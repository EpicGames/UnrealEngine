// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidCookieManager.h"

#if USE_ANDROID_JNI

#include "Android/AndroidJNI.h"
#include "Android/AndroidApplication.h"

#include <jni.h>
#include "Async/TaskGraphInterfaces.h"


FAndroidCookieManager::FAndroidCookieManager()
{
}

FAndroidCookieManager::~FAndroidCookieManager()
{
}

void FAndroidCookieManager::SetCookie(const FString& URL, const FCookie& Cookie, TFunction<void(bool)> Completed)
{
	bool bResult = false;

	JNIEnv* Env = FAndroidApplication::GetJavaEnv();
	if (Env)
	{
		static jmethodID SetCookieFunc = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_CookieManager_SetCookie", "(Ljava/lang/String;Ljava/lang/String;)Z", false);
		if (SetCookieFunc != nullptr)
		{
			FString CookieData = Cookie.Name + FString("=") + Cookie.Value;
			if (Cookie.bHasExpires)
			{
				CookieData += FString(";expires") + Cookie.Expires.ToHttpDate() + FString(";");
			}

			jstring jUrl = Env->NewStringUTF(TCHAR_TO_UTF8(*URL));
			jstring jCookieData = Env->NewStringUTF(TCHAR_TO_UTF8(*CookieData));
			bResult = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, SetCookieFunc, jUrl, jCookieData);
			Env->DeleteLocalRef(jCookieData);
			Env->DeleteLocalRef(jUrl);
		}
	}

	// send result
	if (Completed)
	{
		Completed(bResult);
	}
}

void FAndroidCookieManager::DeleteCookies(const FString& URL, const FString& CookieName, TFunction<void(int)> Completed)
{
	bool bResult = false;

	JNIEnv* Env = FAndroidApplication::GetJavaEnv();
	if (Env)
	{
		static jmethodID RemoveCookiesFunc = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_CookieManager_RemoveCookies", "(Ljava/lang/String;)Z", false);
		if (RemoveCookiesFunc != nullptr)
		{
			jstring jUrl = Env->NewStringUTF(TCHAR_TO_UTF8(*URL));
			bResult = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, RemoveCookiesFunc, jUrl);
			Env->DeleteLocalRef(jUrl);
		}
	}

	// send result
	if (Completed)
	{
		Completed(bResult);
	}
}

#else

FAndroidCookieManager::FAndroidCookieManager()
{
}

FAndroidCookieManager::~FAndroidCookieManager()
{
}

void FAndroidCookieManager::SetCookie(const FString& URL, const FCookie& Cookie, TFunction<void(bool)> Completed)
{
	bool bResult = false;

	// send result
	if (Completed)
	{
		Completed(bResult);
	}
}

void FAndroidCookieManager::DeleteCookies(const FString& URL, const FString& CookieName, TFunction<void(int)> Completed)
{
	bool bResult = false;

	// send result
	if (Completed)
	{
		Completed(bResult);
	}
}

#endif
