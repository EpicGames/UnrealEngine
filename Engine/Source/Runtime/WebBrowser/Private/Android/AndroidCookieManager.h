// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if PLATFORM_ANDROID
#include "IWebBrowserCookieManager.h"

/**
 * Implementation of interface for dealing with a Web Browser cookies for iOS.
 */
class FAndroidCookieManager
	: public IWebBrowserCookieManager
	, public TSharedFromThis<FAndroidCookieManager>
{
public:

	// IWebBrowserCookieManager interface

	virtual void SetCookie(const FString& URL, const FCookie& Cookie, TFunction<void(bool)> Completed = nullptr) override;
	virtual void DeleteCookies(const FString& URL = TEXT(""), const FString& CookieName = TEXT(""), TFunction<void(int)> Completed = nullptr) override;

	// FAndroidCookieManager

	FAndroidCookieManager();
	virtual ~FAndroidCookieManager();
};
#endif
