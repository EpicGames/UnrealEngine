// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidJavaWebBrowser.h"

#if USE_ANDROID_JNI

#include "Android/AndroidApplication.h"

#include <android/bitmap.h>

#if UE_BUILD_SHIPPING
// always clear any exceptions in SHipping
#define CHECK_JNI_RESULT(Id) if (Id == 0) { JEnv->ExceptionClear(); }
#else
#define CHECK_JNI_RESULT(Id) \
if (Id == 0) \
{ \
	if (!bIsOptional) { JEnv->ExceptionDescribe(); checkf(Id != 0, TEXT("Failed to find " #Id)); } \
	JEnv->ExceptionClear(); \
}
#endif

static jfieldID FindField(JNIEnv* JEnv, jclass Class, const ANSICHAR* FieldName, const ANSICHAR* FieldType, bool bIsOptional)
{
	jfieldID Field = Class == NULL ? NULL : JEnv->GetFieldID(Class, FieldName, FieldType);
	CHECK_JNI_RESULT(Field);
	return Field;
}

// From https://developer.android.com/reference/android/view/MotionEvent#constants_1
#define MOTIONEVENT_ACTION_DOWN		0
#define MOTIONEVENT_ACTION_UP		1
#define MOTIONEVENT_ACTION_MOVE		2

FJavaAndroidWebBrowser::FJavaAndroidWebBrowser(bool swizzlePixels, bool vulkanRenderer, int32 width, int32 height,
	jlong widgetPtr, bool bEnableRemoteDebugging, bool bUseTransparency, bool bEnableDomStorage, bool bShouldUseBitmapRender, const FString& UserAgentApplication, bool bEnableFloatingCloseButton, bool bEnablePaymentRequest)
	: TClassObject{widgetPtr, width, height, swizzlePixels, vulkanRenderer, bEnableRemoteDebugging, bUseTransparency, bEnableDomStorage, bShouldUseBitmapRender, *FJavaHelper::ToJavaString(AndroidJavaEnv::GetJavaEnv(), UserAgentApplication), bEnableFloatingCloseButton, bEnablePaymentRequest}
	, ReleaseMethod(GetClassMethod("release", "()V"))
	, GetVideoLastFrameBitmapMethod(GetClassMethod("getVideoLastFrameBitmap", "()Lcom/epicgames/unreal/WebViewControl$FrameUpdateInfo;"))
	, GetVideoLastFrameDataMethod(GetClassMethod("getVideoLastFrameData", "()Lcom/epicgames/unreal/WebViewControl$FrameUpdateInfo;"))
	, GetVideoLastFrameMethod(GetClassMethod("getVideoLastFrame", "(I)Lcom/epicgames/unreal/WebViewControl$FrameUpdateInfo;"))
	, DidResolutionChangeMethod(GetClassMethod("didResolutionChange", "()Z"))
	, UpdateVideoFrameMethod(GetClassMethod("updateVideoFrame", "(I)Lcom/epicgames/unreal/WebViewControl$FrameUpdateInfo;"))
	, UpdateMethod(GetClassMethod("Update", "(IIII)V"))
	, ExecuteJavascriptMethod(GetClassMethod("ExecuteJavascript", "(Ljava/lang/String;)V"))
	, LoadURLMethod(GetClassMethod("LoadURL", "(Ljava/lang/String;)V"))
	, LoadStringMethod(GetClassMethod("LoadString", "(Ljava/lang/String;Ljava/lang/String;)V"))
	, StopLoadMethod(GetClassMethod("StopLoad", "()V"))
	, ReloadMethod(GetClassMethod("Reload", "()V"))
	, CloseMethod(GetClassMethod("Close", "()V"))
	, ShowFloatingCloseButtonMethod(GetClassMethod("ShowFloatingCloseButton", "(ZZ)V"))
	, GoBackOrForwardMethod(GetClassMethod("GoBackOrForward", "(I)V"))
	, SendTouchEventMethod(GetClassMethod("SendTouchEvent", "(IFF)V"))
	, SendKeyEventMethod(GetClassMethod("SendKeyEvent", "(ZI)Z"))
	, SetAndroid3DBrowserMethod(GetClassMethod("SetAndroid3DBrowser", "(Z)V"))
	, SetVisibilityMethod(GetClassMethod("SetVisibility", "(Z)V"))
{
	VideoTexture = nullptr;
	bVideoTextureValid = false;

	JNIEnv* JEnv = FAndroidApplication::GetJavaEnv();

	// get field IDs for FrameUpdateInfo class members
	FrameUpdateInfoClass = FAndroidApplication::FindJavaClassGlobalRef("com/epicgames/unreal/WebViewControl$FrameUpdateInfo");
	FrameUpdateInfo_Buffer = FindField(JEnv, FrameUpdateInfoClass, "Buffer", "Ljava/nio/Buffer;", false);
	FrameUpdateInfo_Bitmap = FindField(JEnv, FrameUpdateInfoClass, "Bitmap", "Landroid/graphics/Bitmap;", false);
	FrameUpdateInfo_FrameReady = FindField(JEnv, FrameUpdateInfoClass, "FrameReady", "Z", false);
	FrameUpdateInfo_RegionChanged = FindField(JEnv, FrameUpdateInfoClass, "RegionChanged", "Z", false);
}

FJavaAndroidWebBrowser::~FJavaAndroidWebBrowser()
{
	if (auto Env = FAndroidApplication::GetJavaEnv())
	{
		Env->DeleteGlobalRef(FrameUpdateInfoClass);
	}
}

void FJavaAndroidWebBrowser::Release()
{
	CallMethod<void>(ReleaseMethod);
}

bool FJavaAndroidWebBrowser::GetVideoLastFrameBitmap(void* outPixels, int64 outCount)
{
	// This can return an exception in some cases
	JNIEnv* JEnv = FAndroidApplication::GetJavaEnv();
	auto Result = NewScopedJavaObject(JEnv, JEnv->CallObjectMethod(Object, GetVideoLastFrameBitmapMethod.Method));
	if (JEnv->ExceptionCheck())
	{
		JEnv->ExceptionDescribe();
		JEnv->ExceptionClear();
		return false;
	}

	if (!Result)
	{
		return false;
	}

	auto bitmap = NewScopedJavaObject(JEnv, JEnv->GetObjectField(*Result, FrameUpdateInfo_Bitmap));
	if (bitmap)
	{
		AndroidBitmapInfo BitmapInfo;

		AndroidBitmap_getInfo(JEnv, *bitmap, &BitmapInfo);
		if (BitmapInfo.format != ANDROID_BITMAP_FORMAT_RGBA_8888)
		{
			return false;
		}
		int32 BitmapBytes = BitmapInfo.stride * BitmapInfo.height;

		BitmapBytes = FMath::Min(outCount, BitmapBytes);
		if (BitmapBytes > 0)
		{
			void* BitmapDataPtr;

			if (AndroidBitmap_lockPixels(JEnv, *bitmap, &BitmapDataPtr) >= 0)
			{
				// unfortunately the order is wrong (RGBA instead of BGRA)
				//memcpy(outPixels, BitmapDataPtr, BitmapBytes);
				int32 NumPixels = BitmapBytes >> 2;
				uint8* SrcData = (uint8*)BitmapDataPtr;
				uint8* DstData = (uint8*)outPixels;
				while (NumPixels--)
				{
					*DstData++ = SrcData[2];
					*DstData++ = SrcData[1];
					*DstData++ = SrcData[0];
					*DstData++ = SrcData[3];
					SrcData += 4;
				}
				AndroidBitmap_unlockPixels(JEnv, *bitmap);
				return true;
			}
		}

		return false;
	}

	return false;
}

bool FJavaAndroidWebBrowser::GetVideoLastFrameData(void* & outPixels, int64 & outCount, bool *bRegionChanged)
{
	// This can return an exception in some cases
	JNIEnv*	JEnv = FAndroidApplication::GetJavaEnv();
	auto Result = NewScopedJavaObject(JEnv, JEnv->CallObjectMethod(Object, GetVideoLastFrameDataMethod.Method));
	if (JEnv->ExceptionCheck())
	{
		JEnv->ExceptionDescribe();
		JEnv->ExceptionClear();
		*bRegionChanged = false;
		return false;
	}

	if (!Result)
	{
		return false;
	}

	auto buffer = NewScopedJavaObject(JEnv, JEnv->GetObjectField(*Result, FrameUpdateInfo_Buffer));
	if (buffer)
	{
		bool bFrameReady = (bool)JEnv->GetBooleanField(*Result, FrameUpdateInfo_FrameReady);
		*bRegionChanged = (bool)JEnv->GetBooleanField(*Result, FrameUpdateInfo_RegionChanged);
		
		outPixels = JEnv->GetDirectBufferAddress(*buffer);
		outCount = JEnv->GetDirectBufferCapacity(*buffer);
		
		return !(nullptr == outPixels || 0 == outCount);
	}
	
	return false;
}

bool FJavaAndroidWebBrowser::DidResolutionChange()
{
	return CallMethod<bool>(DidResolutionChangeMethod);
}

bool FJavaAndroidWebBrowser::UpdateVideoFrame(int32 ExternalTextureId, bool *bRegionChanged)
{
	// This can return an exception in some cases
	JNIEnv*	JEnv = FAndroidApplication::GetJavaEnv();
	auto Result = NewScopedJavaObject(JEnv, JEnv->CallObjectMethod(Object, UpdateVideoFrameMethod.Method, ExternalTextureId));
	if (JEnv->ExceptionCheck())
	{
		JEnv->ExceptionDescribe();
		JEnv->ExceptionClear();
		*bRegionChanged = false;
		return false;
	}

	if (!Result)
	{
		*bRegionChanged = false;
		return false;
	}

	bool bFrameReady = (bool)JEnv->GetBooleanField(*Result, FrameUpdateInfo_FrameReady);
	*bRegionChanged = (bool)JEnv->GetBooleanField(*Result, FrameUpdateInfo_RegionChanged);
	
	return bFrameReady;
}

bool FJavaAndroidWebBrowser::GetVideoLastFrame(int32 destTexture)
{
	// This can return an exception in some cases
	JNIEnv*	JEnv = FAndroidApplication::GetJavaEnv();
	auto Result = NewScopedJavaObject(JEnv, JEnv->CallObjectMethod(Object, GetVideoLastFrameMethod.Method, destTexture));
	if (JEnv->ExceptionCheck())
	{
		JEnv->ExceptionDescribe();
		JEnv->ExceptionClear();
		return false;
	}

	if (!Result)
	{
		return false;
	}

	bool bFrameReady = (bool)JEnv->GetBooleanField(*Result, FrameUpdateInfo_FrameReady);
	
	return bFrameReady;
}

FName FJavaAndroidWebBrowser::GetClassName()
{
	return FName("com/epicgames/unreal/WebViewControl");
}

void FJavaAndroidWebBrowser::Update(const int posX, const int posY, const int sizeX, const int sizeY)
{
	CallMethod<void>(UpdateMethod, posX, posY, sizeX, sizeY);
}

void FJavaAndroidWebBrowser::ExecuteJavascript(const FString& Script)
{
	auto JString = FJavaClassObject::GetJString(Script);
	CallMethod<void>(ExecuteJavascriptMethod, *JString);
}

void FJavaAndroidWebBrowser::LoadURL(const FString& NewURL)
{
	auto JString = FJavaClassObject::GetJString(NewURL);
	CallMethod<void>(LoadURLMethod, *JString);
}

void FJavaAndroidWebBrowser::LoadString(const FString& Contents, const FString& BaseUrl)
{
	auto JContents = FJavaClassObject::GetJString(Contents);
	auto JUrl = FJavaClassObject::GetJString(BaseUrl);
	CallMethod<void>(LoadStringMethod, *JContents, *JUrl);
}

void FJavaAndroidWebBrowser::StopLoad()
{
	CallMethod<void>(StopLoadMethod);
}

void FJavaAndroidWebBrowser::Reload()
{
	CallMethod<void>(ReloadMethod);
}

void FJavaAndroidWebBrowser::Close()
{
	CallMethod<void>(CloseMethod);
}

void FJavaAndroidWebBrowser::ShowFloatingCloseButton(bool bShow, bool bDraggable)
{
	CallMethod<void>(ShowFloatingCloseButtonMethod, bShow, bDraggable);
}

void FJavaAndroidWebBrowser::GoBack()
{
	CallMethod<void>(GoBackOrForwardMethod, -1);
}

void FJavaAndroidWebBrowser::GoForward()
{
	CallMethod<void>(GoBackOrForwardMethod, 1);
}

void FJavaAndroidWebBrowser::SendTouchDown(float x, float y)
{
	CallMethod<void>(SendTouchEventMethod, MOTIONEVENT_ACTION_DOWN, x, y);
}

void FJavaAndroidWebBrowser::SendTouchUp(float x, float y)
{
	CallMethod<void>(SendTouchEventMethod, MOTIONEVENT_ACTION_UP, x, y);
}

void FJavaAndroidWebBrowser::SendTouchMove(float x, float y)
{
	CallMethod<void>(SendTouchEventMethod, MOTIONEVENT_ACTION_MOVE, x, y);
}

bool FJavaAndroidWebBrowser::SendKeyDown(int32 KeyCode)
{
	return CallMethod<bool>(SendKeyEventMethod, true, KeyCode);
}

bool FJavaAndroidWebBrowser::SendKeyUp(int32 KeyCode)
{
	return CallMethod<bool>(SendKeyEventMethod, false, KeyCode);
}

void FJavaAndroidWebBrowser::SetAndroid3DBrowser(bool InIsAndroid3DBrowser)
{
	CallMethod<void>(SetAndroid3DBrowserMethod, InIsAndroid3DBrowser);
}

void FJavaAndroidWebBrowser::SetVisibility(bool InIsVisible)
{
	CallMethod<void>(SetVisibilityMethod, InIsVisible);
}

#endif // USE_ANDROID_JNI

