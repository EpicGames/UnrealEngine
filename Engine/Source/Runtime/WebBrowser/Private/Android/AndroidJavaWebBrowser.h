// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if USE_ANDROID_JNI

#include "AndroidJniWebViewControl.h"
#include "Android/AndroidPlatform.h"
#include "Android/AndroidJava.h"
#include "RHI.h"
#include "RHIResources.h"

// Wrapper for com/epicgames/unreal/WebViewControl.java.
class FJavaAndroidWebBrowser : public UE::Jni::TClassObject<UE::Jni::FWebViewControl>
{
public:
	FJavaAndroidWebBrowser(bool swizzlePixels, bool vulkanRenderer, int32 width, int32 height, jlong widgetPtr, bool bEnableRemoteDebugging, bool bUseTransparency, bool bEnableDomStorage, bool bShouldUseBitmapRender, const FString& UserAgentApplication, bool bEnableFloatingCloseButton, bool bEnablePaymentRequest);
	virtual ~FJavaAndroidWebBrowser();
	void Release();
	bool GetVideoLastFrameBitmap(void* outPixels, int64 outCount);
	bool GetVideoLastFrameData(void* & outPixels, int64 & outCount, bool *bRegionChanged);
	bool GetVideoLastFrame(int32 destTexture);
	bool DidResolutionChange();
	bool UpdateVideoFrame(int32 ExternalTextureId, bool *bRegionChanged);
	void ExecuteJavascript(const FString& Script);
	void LoadURL(const FString& NewURL);
	void LoadString(const FString& Contents, const FString& BaseUrl);
	void StopLoad();
	void Reload();
	void Close();
	void ShowFloatingCloseButton(bool bShow, bool bDraggable);
	void GoBack();
	void GoForward();
	void SendTouchDown(float x, float y);
	void SendTouchUp(float x, float y);
	void SendTouchMove(float x, float y);
	bool SendKeyDown(int32 KeyCode);
	bool SendKeyUp(int32 KeyCode);
	void SetAndroid3DBrowser(bool InIsAndroid3DBrowser);
	void SetVisibility(bool InIsVisible);
	void Update(const int posX, const int posY, const int sizeX, const int sizeY);
private:
	static FName GetClassName();

	FJavaClassMethod ReleaseMethod;
	FJavaClassMethod GetVideoLastFrameBitmapMethod;
	FJavaClassMethod GetVideoLastFrameDataMethod;
	FJavaClassMethod GetVideoLastFrameMethod;
	FJavaClassMethod DidResolutionChangeMethod;
	FJavaClassMethod UpdateVideoFrameMethod;
	FJavaClassMethod UpdateMethod;
	FJavaClassMethod ExecuteJavascriptMethod;
	FJavaClassMethod LoadURLMethod;
	FJavaClassMethod LoadStringMethod;
	FJavaClassMethod StopLoadMethod;
	FJavaClassMethod ReloadMethod;
	FJavaClassMethod CloseMethod;
	FJavaClassMethod ShowFloatingCloseButtonMethod;
	FJavaClassMethod GoBackOrForwardMethod;
	FJavaClassMethod SendTouchEventMethod;
	FJavaClassMethod SendKeyEventMethod;
	FJavaClassMethod SetAndroid3DBrowserMethod;
	FJavaClassMethod SetVisibilityMethod;

	// FrameUpdateInfo member field ids
	jclass FrameUpdateInfoClass;
	jfieldID FrameUpdateInfo_Buffer;
	jfieldID FrameUpdateInfo_Bitmap;
	jfieldID FrameUpdateInfo_FrameReady;
	jfieldID FrameUpdateInfo_RegionChanged;
	
	FTextureRHIRef VideoTexture;
	bool bVideoTextureValid;

public:
	FTextureRHIRef GetVideoTexture()
	{
		return VideoTexture;
	}

	void SetVideoTexture(FTextureRHIRef Texture)
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Fetch RT: SetVideoTexture: %d"), Texture.IsValid());

		VideoTexture = Texture;
	}

	void SetVideoTextureValid(bool Condition)
	{
		bVideoTextureValid = Condition;
	}

	bool IsVideoTextureValid()
	{
		return bVideoTextureValid;
	}

};

#endif // USE_ANDROID_JNI
