// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#if USE_ANDROID_JNI
#include "Android/AndroidJavaEnv.h"

namespace UE::Jni::Android::WebKit
{
	struct FWebView: Java::Lang::FObject
	{
		static constexpr FAnsiStringView ClassName = "android/webkit/WebView";
	};
	
	struct FJsResult: Java::Lang::FObject
	{
		static constexpr FAnsiStringView ClassName = "android/webkit/JsResult";
	};
	
	struct FJsPromptResult: Java::Lang::FObject
	{
		static constexpr FAnsiStringView ClassName = "android/webkit/JsPromptResult";
	};
}

namespace UE::Jni
{
	struct FWebViewControl: Java::Lang::FObject
	{
		struct FViewClient: Java::Lang::FObject
		{
			static constexpr FAnsiStringView ClassName = "com/epicgames/unreal/WebViewControl$ViewClient";

			static Java::Lang::TArray<jbyte>* JNICALL shouldInterceptRequestImpl(JNIEnv* env, jobject thiz, jstring JUrl);
			static jboolean JNICALL shouldOverrideUrlLoading(JNIEnv* env, jobject thiz, Android::WebKit::FWebView* /* ignore */, jstring JUrl);
			static void JNICALL onPageLoad(JNIEnv* env, jobject thiz, jstring JUrl, jboolean bIsLoading, jint HistorySize, jint HistoryPosition);
			static void JNICALL onReceivedError(JNIEnv* env, jobject thiz, Android::WebKit::FWebView* /* ignore */, jint ErrorCode, jstring Description, jstring JUrl);

			static constexpr FNativeMethod NativeMethods[]
			{
				UE_JNI_NATIVE_METHOD(shouldInterceptRequestImpl),
				UE_JNI_NATIVE_METHOD(shouldOverrideUrlLoading),
				UE_JNI_NATIVE_METHOD(onPageLoad),
				UE_JNI_NATIVE_METHOD(onReceivedError)
			};
		};

		struct FChromeClient: Java::Lang::FObject
		{
			static constexpr FAnsiStringView ClassName = "com/epicgames/unreal/WebViewControl$ChromeClient";

			static jboolean JNICALL onJsAlert(JNIEnv* env, jobject thiz, Android::WebKit::FWebView* /* ignore */, jstring JUrl, jstring Message, Android::WebKit::FJsResult* Result);
			static jboolean JNICALL onJsBeforeUnload(JNIEnv* env, jobject thiz, Android::WebKit::FWebView* /* ignore */, jstring JUrl, jstring Message, Android::WebKit::FJsResult* Result);
			static jboolean JNICALL onJsConfirm(JNIEnv* env, jobject thiz, Android::WebKit::FWebView* /* ignore */, jstring JUrl, jstring Message, Android::WebKit::FJsResult* Result);
			static jboolean JNICALL onJsPrompt(JNIEnv* env, jobject thiz, Android::WebKit::FWebView* /* ignore */, jstring JUrl, jstring Message, jstring DefaultValue, Android::WebKit::FJsPromptResult* Result);
			static void JNICALL onReceivedTitle(JNIEnv* env, jobject thiz, Android::WebKit::FWebView* /* ignore */, jstring Title);

			static constexpr FNativeMethod NativeMethods[]
			{
				UE_JNI_NATIVE_METHOD(onJsAlert),
				UE_JNI_NATIVE_METHOD(onJsBeforeUnload),
				UE_JNI_NATIVE_METHOD(onJsConfirm),
				UE_JNI_NATIVE_METHOD(onJsPrompt),
				UE_JNI_NATIVE_METHOD(onReceivedTitle)
			};
		};
		
		static constexpr FAnsiStringView ClassName = "com/epicgames/unreal/WebViewControl";

		inline static TConstructor<FWebViewControl(jlong InNativePtr, jint Width, jint Height, jboolean bSwizzlePixels, jboolean bVulkanRenderer, jboolean bEnableRemoteDebugging, jboolean bUseTransparency, jboolean bEnableDomStorage, jboolean bShouldUseBitmapRender, jstring UserAgentApplication, jboolean bEnableFloatingCloseButton, jboolean bEnablePaymentRequest)> New;

		static void JNICALL FloatingCloseButtonPressed(JNIEnv* JEnv, jobject thiz);
		
		static constexpr FMember Members[]
		{
			UE_JNI_MEMBER(New)
		};

		static constexpr FNativeMethod NativeMethods[]
		{
			UE_JNI_NATIVE_METHOD(FloatingCloseButtonPressed)
		};
	};

	template struct TInitialize<FWebViewControl::FViewClient>;
	template struct TInitialize<FWebViewControl::FChromeClient>;
	template struct TInitialize<FWebViewControl>;
}
#endif
