// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidWebBrowserDialog.h"

#if USE_ANDROID_JNI

#include "Android/AndroidApplication.h"
#include "Android/AndroidJava.h"

#include <jni.h>

namespace
{
	FText GetFText(jstring InString)
	{
		if (InString == nullptr)
		{
			return FText::GetEmpty();
		}

		JNIEnv*	JEnv = FAndroidApplication::GetJavaEnv();
		FString Temp = FJavaHelper::FStringFromParam(JEnv, InString);
		FText Retval = FText::FromString(Temp);
		return Retval;
	}
}

FAndroidWebBrowserDialog::FAndroidWebBrowserDialog(jstring InMessageText, jstring InDefaultPrompt, jobject InCallback)
	: Type(EWebBrowserDialogType::Prompt)
	, MessageText(GetFText(InMessageText))
	, DefaultPrompt(GetFText(InDefaultPrompt))
	, Callback(InCallback)
{
}

FAndroidWebBrowserDialog::FAndroidWebBrowserDialog(EWebBrowserDialogType InDialogType, jstring InMessageText, jobject InCallback)
	: Type(InDialogType)
	, MessageText(GetFText(InMessageText))
	, DefaultPrompt()
	, Callback(InCallback)
{
}

void FAndroidWebBrowserDialog::Continue(bool Success, const FText& UserResponse)
{
	check(Callback != nullptr);
	JNIEnv*	JEnv = FAndroidApplication::GetJavaEnv();
	const char* MethodName = Success?"confirm":"cancel";
	const char* MethodSignature = (Success && Type==EWebBrowserDialogType::Prompt)?"(Ljava/lang/String;)V":"()V";
	auto Class = NewScopedJavaObject(JEnv, JEnv->GetObjectClass(Callback));
	jmethodID MethodId = JEnv->GetMethodID(*Class, MethodName, MethodSignature);

	if (Success && Type==EWebBrowserDialogType::Prompt)
	{
		auto JUserResponse = FJavaClassObject::GetJString(UserResponse.ToString());
		JEnv->CallVoidMethod(Callback, MethodId, *JUserResponse);
	}
	else
	{
		JEnv->CallVoidMethod(Callback, MethodId, nullptr);
	}
}

#endif // USE_ANDROID_JNI
