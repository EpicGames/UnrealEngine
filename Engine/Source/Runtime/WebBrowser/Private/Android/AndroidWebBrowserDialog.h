// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if USE_ANDROID_JNI

#include "IWebBrowserDialog.h"

#include <jni.h>

class SAndroidWebBrowserWidget;

class FAndroidWebBrowserDialog
	: public IWebBrowserDialog
{
public:
	virtual ~FAndroidWebBrowserDialog() = default;

	// IWebBrowserDialog interface:

	virtual EWebBrowserDialogType GetType() override
	{
		return Type;
	}

	virtual const FText& GetMessageText() override
	{
		return MessageText;
	}

	virtual const FText& GetDefaultPrompt() override
	{
		return DefaultPrompt;
	}

	virtual bool IsReload() override
	{
		check(Type == EWebBrowserDialogType::Unload);
		return false; // The android webkit browser does not provide this infomation
	}

	virtual void Continue(bool Success = true, const FText& UserResponse = FText::GetEmpty()) override;

private:

	EWebBrowserDialogType Type;
	FText MessageText;
	FText DefaultPrompt;

	jobject Callback; // Either a reference to a JsResult or a JsPromptResult object depending on Type

	// Create a dialog from OnJSPrompt arguments
	FAndroidWebBrowserDialog(jstring InMessageText, jstring InDefaultPrompt, jobject InCallback);

	// Create a dialog from OnJSAlert|Confirm|BeforeUnload arguments
	FAndroidWebBrowserDialog(EWebBrowserDialogType InDialogType, jstring InMessageText, jobject InCallback);

	friend class FAndroidWebBrowserWindow;
	friend class SAndroidWebBrowserWidget;
};

typedef FAndroidWebBrowserDialog FWebBrowserDialog;

#endif // USE_ANDROID_JNI
