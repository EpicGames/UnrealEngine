// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidWebBrowserWidget.h"

#if USE_ANDROID_JNI

#include "AndroidJniWebViewControl.h"
#include "AndroidWebBrowserWindow.h"
#include "AndroidWebBrowserDialog.h"
#include "MobileJS/MobileJSScripting.h"
#include "Android/AndroidApplication.h"
#include "Android/AndroidWindow.h"
#include "Android/AndroidJava.h"
#include "Android/AndroidEventManager.h"
#include "Async/Async.h"
#include "Misc/ScopeLock.h"
#include "RHICommandList.h"
#include "RenderingThread.h"
#include "ExternalTexture.h"
#include "Slate/SlateTextures.h"
#include "SlateMaterialBrush.h"
#include "Templates/SharedPointer.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "WebBrowserTextureSample.h"
#include "WebBrowserModule.h"
#include "IWebBrowserSingleton.h"
#include "Misc/ConfigCacheIni.h"

// For UrlDecode
#include "Http.h"

FCriticalSection SAndroidWebBrowserWidget::WebControlsCS;
TMap<int64, TWeakPtr<SAndroidWebBrowserWidget>> SAndroidWebBrowserWidget::AllWebControls;

TSharedPtr<SAndroidWebBrowserWidget> SAndroidWebBrowserWidget::GetWidgetPtr(JNIEnv* JEnv, jobject Jobj)
{
	FScopeLock L(&WebControlsCS);

	auto Class = NewScopedJavaObject(JEnv, JEnv->GetObjectClass(Jobj));
	jmethodID JMethod = JEnv->GetMethodID(*Class, "GetNativePtr", "()J");
	check(JMethod != nullptr);

	int64 ObjAddr = JEnv->CallLongMethod(Jobj, JMethod);

	TWeakPtr<SAndroidWebBrowserWidget> WebControl = AllWebControls.FindRef(ObjAddr);
	return (WebControl.IsValid()) ? WebControl.Pin() : TSharedPtr<SAndroidWebBrowserWidget>();
}

SAndroidWebBrowserWidget::~SAndroidWebBrowserWidget()
{
	if (JavaWebBrowser.IsValid())
	{
		if (GSupportsImageExternal && !FAndroidMisc::ShouldUseVulkan())
		{
			// Unregister the external texture on render thread
			FTextureRHIRef VideoTexture = JavaWebBrowser->GetVideoTexture();

			JavaWebBrowser->SetVideoTexture(nullptr);
			JavaWebBrowser->Release();

			struct FReleaseVideoResourcesParams
			{
				FTextureRHIRef VideoTexture;
				FGuid PlayerGuid;
			};

			FReleaseVideoResourcesParams ReleaseVideoResourcesParams = { VideoTexture, WebBrowserTexture->GetExternalTextureGuid() };

			ENQUEUE_RENDER_COMMAND(AndroidWebBrowserWriteVideoSample)(
				[Params = ReleaseVideoResourcesParams](FRHICommandListImmediate& RHICmdList)
				{
					FExternalTextureRegistry::Get().UnregisterExternalTexture(Params.PlayerGuid);
					// @todo: this causes a crash
					//					Params.VideoTexture->Release();
				});
		}
		else
		{
			JavaWebBrowser->SetVideoTexture(nullptr);
			JavaWebBrowser->Release();
		}

	}
	delete TextureSamplePool;
	TextureSamplePool = nullptr;
	
	WebBrowserTextureSamplesQueue->RequestFlush();

	if (WebBrowserBrush.IsValid())
	{
		WebBrowserBrush->SetResourceObject(nullptr);	
	}
	
	if (WebBrowserMaterial != nullptr)
	{
		WebBrowserMaterial->RemoveFromRoot();
		WebBrowserMaterial = nullptr;
	}

	if (WebBrowserTexture != nullptr)
	{
		WebBrowserTexture->RemoveFromRoot();
		WebBrowserTexture = nullptr;
	}

	FScopeLock L(&WebControlsCS);
	AllWebControls.Remove(reinterpret_cast<int64>(this));
}

void SAndroidWebBrowserWidget::Construct(const FArguments& Args)
{
	{
		FScopeLock L(&WebControlsCS);
		AllWebControls.Add(reinterpret_cast<int64>(this), StaticCastSharedRef<SAndroidWebBrowserWidget>(AsShared()));
	}

	WebBrowserWindowPtr = Args._WebBrowserWindow;
	IsAndroid3DBrowser = true;

	bShouldUseBitmapRender = false;
	bMouseCapture = false;

	HistorySize = 0;
	HistoryPosition = 0;

	// Check if DOM storage should be enabled
	bool bEnableDomStorage = false;
	GConfig->GetBool(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("bEnableDomStorage"), bEnableDomStorage, GEngineIni);

	FIntPoint viewportSize = WebBrowserWindowPtr.Pin()->GetViewportSize();
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("SAndroidWebBrowserWidget::Construct viewport=%d x %d"), viewportSize.X, viewportSize.Y);

	bool bEnableFloatingCloseButton = false;
	GConfig->GetBool(TEXT("Browser"), TEXT("bEnableFloatingCloseButton"), bEnableFloatingCloseButton, GEngineIni);

	JavaWebBrowser = MakeShared<FJavaAndroidWebBrowser, ESPMode::ThreadSafe>(false, FAndroidMisc::ShouldUseVulkan(), viewportSize.X, viewportSize.Y,
		reinterpret_cast<jlong>(this), !(UE_BUILD_SHIPPING || UE_BUILD_TEST), Args._UseTransparency, bEnableDomStorage, bShouldUseBitmapRender, Args._UserAgentApplication, bEnableFloatingCloseButton, Args._EnablePaymentRequest);

	TextureSamplePool = new FWebBrowserTextureSamplePool();
	WebBrowserTextureSamplesQueue = MakeShared<FWebBrowserTextureSampleQueue, ESPMode::ThreadSafe>();
	WebBrowserTexture = nullptr;
	WebBrowserMaterial = nullptr;
	WebBrowserBrush = nullptr;

	// create external texture
	WebBrowserTexture = NewObject<UWebBrowserTexture>((UObject*)GetTransientPackage(), NAME_None, RF_Transient | RF_Public);

	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("SAndroidWebBrowserWidget::Construct0"));
	if (WebBrowserTexture != nullptr)
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("SAndroidWebBrowserWidget::Construct01"));

		WebBrowserTexture->UpdateResource();
		WebBrowserTexture->AddToRoot();
	}

	// create wrapper material
	IWebBrowserSingleton* WebBrowserSingleton = IWebBrowserModule::Get().GetSingleton();
	
	UMaterialInterface* DefaultWBMaterial = Args._UseTransparency? WebBrowserSingleton->GetDefaultTranslucentMaterial(): WebBrowserSingleton->GetDefaultMaterial();
	if (WebBrowserSingleton && DefaultWBMaterial)
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("SAndroidWebBrowserWidget::Construct1"));
		// create wrapper material
		WebBrowserMaterial = UMaterialInstanceDynamic::Create(DefaultWBMaterial, nullptr);

		if (WebBrowserMaterial)
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("SAndroidWebBrowserWidget::Construct2"));
			WebBrowserMaterial->SetTextureParameterValue("SlateUI", WebBrowserTexture);
			WebBrowserMaterial->AddToRoot();

			// create Slate brush
			WebBrowserBrush = MakeShareable(new FSlateBrush());
			{
				WebBrowserBrush->SetResourceObject(WebBrowserMaterial);
			}
		}
	}
	
	check(JavaWebBrowser.IsValid());

	JavaWebBrowser->LoadURL(Args._InitialURL);
}

void SAndroidWebBrowserWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if(WebBrowserWindowPtr.IsValid())
	{
		WebBrowserWindowPtr.Pin()->SetTickLastFrame();
		if (WebBrowserWindowPtr.Pin()->GetParentWindow().IsValid())
		{
			bool ShouldSetAndroid3DBrowser = WebBrowserWindowPtr.Pin()->GetParentWindow().Get()->IsVirtualWindow();
			if (IsAndroid3DBrowser != ShouldSetAndroid3DBrowser)
			{
				IsAndroid3DBrowser = ShouldSetAndroid3DBrowser;
				JavaWebBrowser->SetAndroid3DBrowser(IsAndroid3DBrowser);
			}
		}
	}

	if (!JavaWebBrowser.IsValid())
	{	
		return;
	}
	// deal with resolution changes (usually from streams)
	if (JavaWebBrowser->DidResolutionChange())
	{
		JavaWebBrowser->SetVideoTextureValid(false);
	}


	FIntPoint viewportSize = WebBrowserWindowPtr.Pin()->GetViewportSize();

	if (IsAndroid3DBrowser)
	{
		//FVector2D LocalSize = AllottedGeometry.GetLocalSize();
		//FIntPoint IntLocalSize = FIntPoint((int32)LocalSize.X, (int32)LocalSize.Y);

		//if (viewportSize != IntLocalSize)
		//{
		//	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("SAndroidWebBrowser::Tick: updating viewport to localSize = %d x %d"), IntLocalSize.X, IntLocalSize.Y);
		//	WebBrowserWindowPtr.Pin()->SetViewportSize(IntLocalSize, FIntPoint(0, 0));
		//}

		JavaWebBrowser->Update(0, 0, viewportSize.X, viewportSize.Y);
	}
	else
	{
		// Calculate UIScale, which can vary frame-to-frame thanks to device rotation
		// UI Scale is calculated relative to vertical axis of 1280x720 / 720x1280
		float UIScale;
		FPlatformRect ScreenRect = FAndroidWindow::GetScreenRect();
		int32_t ScreenWidth, ScreenHeight;
		FAndroidWindow::CalculateSurfaceSize(ScreenWidth, ScreenHeight);
		if (ScreenWidth > ScreenHeight)
		{
			UIScale = (float)ScreenHeight / (ScreenRect.Bottom - ScreenRect.Top);
		}
		else
		{
			UIScale = (float)ScreenHeight / (ScreenRect.Bottom - ScreenRect.Top);
		}

		FVector2D Position = AllottedGeometry.GetAccumulatedRenderTransform().GetTranslation() * UIScale;
		FVector2D Size = TransformVector(AllottedGeometry.GetAccumulatedRenderTransform(), AllottedGeometry.GetLocalSize()) * UIScale;

		// Convert position to integer coordinates
		FIntPoint IntPos(FMath::RoundToInt(Position.X), FMath::RoundToInt(Position.Y));
		// Convert size to integer taking the rounding of position into account to avoid double round-down or double round-up causing a noticeable error.
		FIntPoint IntSize = FIntPoint(FMath::RoundToInt(Position.X + Size.X), FMath::RoundToInt(Size.Y + Position.Y)) - IntPos;

		JavaWebBrowser->Update(IntPos.X, IntPos.Y, IntSize.X, IntSize.Y);
	}

	if (IsAndroid3DBrowser)
	{
		if (WebBrowserTexture)
		{
			TSharedPtr<FWebBrowserTextureSample, ESPMode::ThreadSafe> WebBrowserTextureSample;
			WebBrowserTextureSamplesQueue->Peek(WebBrowserTextureSample);

			WebBrowserTexture->TickResource(WebBrowserTextureSample);
		}

		if (FAndroidMisc::ShouldUseVulkan())
		{
			// create new video sample
			auto NewTextureSample = TextureSamplePool->AcquireShared();

			if (!NewTextureSample->Initialize(viewportSize))
			{
				return;
			}

			struct FWriteWebBrowserParams
			{
				TWeakPtr<FJavaAndroidWebBrowser, ESPMode::ThreadSafe> JavaWebBrowserPtr;
				TWeakPtr<FWebBrowserTextureSampleQueue, ESPMode::ThreadSafe> WebBrowserTextureSampleQueuePtr;
				TSharedRef<FWebBrowserTextureSample, ESPMode::ThreadSafe> NewTextureSamplePtr;
				int32 SampleCount;
			}
			WriteWebBrowserParams = { JavaWebBrowser, WebBrowserTextureSamplesQueue, NewTextureSample, (int32)(viewportSize.X * viewportSize.Y * sizeof(int32)) };

			if (bShouldUseBitmapRender)
			{
				ENQUEUE_RENDER_COMMAND(WriteAndroidWebBrowser)(
					[Params = WriteWebBrowserParams](FRHICommandListImmediate& RHICmdList)
					{
						auto PinnedJavaWebBrowser = Params.JavaWebBrowserPtr.Pin();
						auto PinnedSamples = Params.WebBrowserTextureSampleQueuePtr.Pin();

						if (!PinnedJavaWebBrowser.IsValid() || !PinnedSamples.IsValid())
						{
							return;
						}

						int32 SampleBufferSize = Params.NewTextureSamplePtr->InitializeBufferForCopy();
						void* Buffer = (void*)Params.NewTextureSamplePtr->GetBuffer();

						if (!PinnedJavaWebBrowser->GetVideoLastFrameBitmap(Buffer, SampleBufferSize))
						{
							return;
						}

						PinnedSamples->RequestFlush();
						PinnedSamples->Enqueue(Params.NewTextureSamplePtr);
					});
			}
			else
			{
				ENQUEUE_RENDER_COMMAND(WriteAndroidWebBrowser)(
					[Params = WriteWebBrowserParams](FRHICommandListImmediate& RHICmdList)
					{
						auto PinnedJavaWebBrowser = Params.JavaWebBrowserPtr.Pin();
						auto PinnedSamples = Params.WebBrowserTextureSampleQueuePtr.Pin();

						if (!PinnedJavaWebBrowser.IsValid() || !PinnedSamples.IsValid())
						{
							return;
						}

						bool bRegionChanged = false;

						// write frame into buffer
						void* Buffer = nullptr;
						int64 SampleCount = 0;

						if (!PinnedJavaWebBrowser->GetVideoLastFrameData(Buffer, SampleCount, &bRegionChanged))
						{
							//FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Fetch RT: ShouldUseVulkan couldn't get texture buffer"));
							return;
						}

						if (SampleCount != Params.SampleCount)
						{
							FPlatformMisc::LowLevelOutputDebugStringf(TEXT("SAndroidWebBrowserWidget::Fetch: Sample count mismatch (Buffer=%llu, Available=%llu"), Params.SampleCount, SampleCount);
						}
						check(Params.SampleCount <= SampleCount);

						// must make a copy (buffer is owned by Java, not us!)
						Params.NewTextureSamplePtr->InitializeBuffer(Buffer, true);

						PinnedSamples->RequestFlush();
						PinnedSamples->Enqueue(Params.NewTextureSamplePtr);
					});
			}
		}
		else if (GSupportsImageExternal && WebBrowserTexture != nullptr)
		{
			struct FWriteWebBrowserParams
			{
				TWeakPtr<FJavaAndroidWebBrowser, ESPMode::ThreadSafe> JavaWebBrowserPtr;
				FGuid PlayerGuid;
				FIntPoint Size;
			};

			FWriteWebBrowserParams WriteWebBrowserParams = { JavaWebBrowser, WebBrowserTexture->GetExternalTextureGuid(), viewportSize };
			ENQUEUE_RENDER_COMMAND(WriteAndroidWebBrowser)(
				[Params = WriteWebBrowserParams](FRHICommandListImmediate& RHICmdList)
				{
					auto PinnedJavaWebBrowser = Params.JavaWebBrowserPtr.Pin();

					if (!PinnedJavaWebBrowser.IsValid())
					{
						return;
					}

					FTextureRHIRef VideoTexture = PinnedJavaWebBrowser->GetVideoTexture();
					if (VideoTexture == nullptr)
					{
						const FIntPoint LocalSize = Params.Size;

						const FRHITextureCreateDesc Desc =
							FRHITextureCreateDesc::Create2D(TEXT("VideoTexture"), LocalSize, PF_R8G8B8A8)
							.SetFlags(ETextureCreateFlags::External);

						VideoTexture = RHICmdList.CreateTexture(Desc);
						PinnedJavaWebBrowser->SetVideoTexture(VideoTexture);

						if (VideoTexture == nullptr)
						{
							UE_LOGF(LogAndroid, Warning, "CreateTexture failed!");
							return;
						}

						PinnedJavaWebBrowser->SetVideoTextureValid(false);
						FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Fetch RT: Created VideoTexture: %d - %s (%d, %d)"), *reinterpret_cast<int32*>(VideoTexture->GetNativeResource()), *Params.PlayerGuid.ToString(), LocalSize.X, LocalSize.Y);
					}

					int32 TextureId = *reinterpret_cast<int32*>(VideoTexture->GetNativeResource());
					bool bRegionChanged = false;
					if (PinnedJavaWebBrowser->UpdateVideoFrame(TextureId, &bRegionChanged))
					{
						// if region changed, need to reregister UV scale/offset
						FPlatformMisc::LowLevelOutputDebugStringf(TEXT("UpdateVideoFrame RT: %s"), *Params.PlayerGuid.ToString());
						if (bRegionChanged)
						{
							PinnedJavaWebBrowser->SetVideoTextureValid(false);
							FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Fetch RT: %s"), *Params.PlayerGuid.ToString());
						}
					}

					if (!PinnedJavaWebBrowser->IsVideoTextureValid())
					{
						FSamplerStateInitializerRHI SamplerStateInitializer(SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp);
						FSamplerStateRHIRef SamplerStateRHI = RHICreateSamplerState(SamplerStateInitializer);
						FExternalTextureRegistry::Get().RegisterExternalTexture(Params.PlayerGuid, VideoTexture, SamplerStateRHI);
						FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Fetch RT: Register Guid: %s"), *Params.PlayerGuid.ToString());

						PinnedJavaWebBrowser->SetVideoTextureValid(true);
					}
				});
		}
		else
		{
			// create new video sample
			auto NewTextureSample = TextureSamplePool->AcquireShared();

			if (!NewTextureSample->Initialize(viewportSize))
			{
				return;
			}

			// populate & add sample (on render thread)
			struct FWriteWebBrowserParams
			{
				TWeakPtr<FJavaAndroidWebBrowser, ESPMode::ThreadSafe> JavaWebBrowserPtr;
				TWeakPtr<FWebBrowserTextureSampleQueue, ESPMode::ThreadSafe> WebBrowserTextureSampleQueuePtr;
				TSharedRef<FWebBrowserTextureSample, ESPMode::ThreadSafe> NewTextureSamplePtr;
				int32 SampleCount;
			}
			WriteWebBrowserParams = { JavaWebBrowser, WebBrowserTextureSamplesQueue, NewTextureSample, (int32)(viewportSize.X * viewportSize.Y * sizeof(int32)) };

			ENQUEUE_RENDER_COMMAND(WriteAndroidWebBrowser)(
				[Params = WriteWebBrowserParams](FRHICommandListImmediate& RHICmdList)
				{
					auto PinnedJavaWebBrowser = Params.JavaWebBrowserPtr.Pin();
					auto PinnedSamples = Params.WebBrowserTextureSampleQueuePtr.Pin();

					if (!PinnedJavaWebBrowser.IsValid() || !PinnedSamples.IsValid())
					{
						return;
					}

					// write frame into texture
					FRHITexture* Texture = Params.NewTextureSamplePtr->InitializeTexture(RHICmdList);

					if (Texture != nullptr)
					{
						int32 Resource = *reinterpret_cast<int32*>(Texture->GetNativeResource());
						if (!PinnedJavaWebBrowser->GetVideoLastFrame(Resource))
						{
							return;
						}
					}

					PinnedSamples->RequestFlush();
					PinnedSamples->Enqueue(Params.NewTextureSamplePtr);
				});
		}
	}
}


int32 SAndroidWebBrowserWidget::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	bool bIsVisible = !WebBrowserWindowPtr.IsValid() || WebBrowserWindowPtr.Pin()->IsVisible();

	if (bIsVisible && IsAndroid3DBrowser && WebBrowserBrush.IsValid())
	{
		FSlateDrawElement::MakeBox(OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(), WebBrowserBrush.Get(), ESlateDrawEffect::None);
	}
	return LayerId;
}

FVector2D SAndroidWebBrowserWidget::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	return FVector2D(640, 480);
}

void SAndroidWebBrowserWidget::ExecuteJavascript(const FString& Script)
{
	JavaWebBrowser->ExecuteJavascript(Script);
}

void SAndroidWebBrowserWidget::LoadURL(const FString& NewURL)
{
	JavaWebBrowser->LoadURL(NewURL);
}

void SAndroidWebBrowserWidget::LoadString(const FString& Contents, const FString& BaseUrl)
{
	JavaWebBrowser->LoadString(Contents, BaseUrl);
}

void SAndroidWebBrowserWidget::StopLoad()
{
	JavaWebBrowser->StopLoad();
}

void SAndroidWebBrowserWidget::Reload()
{
	JavaWebBrowser->Reload();
}

void SAndroidWebBrowserWidget::Close()
{
	JavaWebBrowser->Release();
	WebBrowserWindowPtr.Reset();
}

void SAndroidWebBrowserWidget::ShowFloatingCloseButton(bool bShow, bool bDraggable)
{
	JavaWebBrowser->ShowFloatingCloseButton(bShow, bDraggable);
}

void SAndroidWebBrowserWidget::GoBack()
{
	JavaWebBrowser->GoBack();
}

void SAndroidWebBrowserWidget::GoForward()
{
	JavaWebBrowser->GoForward();
}

bool SAndroidWebBrowserWidget::CanGoBack()
{
	return HistoryPosition > 1;
}

bool SAndroidWebBrowserWidget::CanGoForward()
{
	return HistoryPosition < HistorySize-1;
}

void SAndroidWebBrowserWidget::SendTouchDown(FVector2D Position)
{
	FVector2D WidgetSize = GetCachedGeometry().GetLocalSize();
	JavaWebBrowser->SendTouchDown(Position.X / WidgetSize.X, Position.Y / WidgetSize.Y);
}

void SAndroidWebBrowserWidget::SendTouchUp(FVector2D Position)
{
	FVector2D WidgetSize = GetCachedGeometry().GetLocalSize();
	JavaWebBrowser->SendTouchUp(Position.X / WidgetSize.X, Position.Y / WidgetSize.Y);
}

void SAndroidWebBrowserWidget::SendTouchMove(FVector2D Position)
{
	FVector2D WidgetSize = GetCachedGeometry().GetLocalSize();
	JavaWebBrowser->SendTouchMove(Position.X / WidgetSize.X, Position.Y / WidgetSize.Y);
}

FVector2D SAndroidWebBrowserWidget::ConvertMouseEventToLocal(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FGeometry MouseGeometry = MyGeometry;

	float DPIScale = MouseGeometry.Scale;
	FVector2D LocalPos = MouseGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()) * DPIScale;

	return LocalPos;
}

FReply SAndroidWebBrowserWidget::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = FReply::Unhandled();

	if (IsAndroid3DBrowser)
	{
		FKey Button = MouseEvent.GetEffectingButton();
		bool bSupportedButton = (Button == EKeys::LeftMouseButton); // || Button == EKeys::RightMouseButton || Button == EKeys::MiddleMouseButton);

		if (bSupportedButton)
		{
			Reply = FReply::Handled();
			SendTouchDown(ConvertMouseEventToLocal(MyGeometry, MouseEvent));
			bMouseCapture = true;
		}
	}

	return Reply;
}

FReply SAndroidWebBrowserWidget::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = FReply::Unhandled();

	if (IsAndroid3DBrowser)
	{
		FKey Button = MouseEvent.GetEffectingButton();
		bool bSupportedButton = (Button == EKeys::LeftMouseButton); // || Button == EKeys::RightMouseButton || Button == EKeys::MiddleMouseButton);

		if (bSupportedButton)
		{
			Reply = FReply::Handled();
			SendTouchUp(ConvertMouseEventToLocal(MyGeometry, MouseEvent));
			bMouseCapture = false;
		}
	}

	return Reply;
}

FReply SAndroidWebBrowserWidget::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = FReply::Unhandled();

	if (IsAndroid3DBrowser && bMouseCapture)
	{
		Reply = FReply::Handled();
		SendTouchMove(ConvertMouseEventToLocal(MyGeometry, MouseEvent));
	}

	return Reply;
}

FReply SAndroidWebBrowserWidget::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (!IsAndroid3DBrowser)
	{
		return FReply::Unhandled();
	}

//	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("SAndroidWebBrowserWidget::OnKeyDown: %d"), InKeyEvent.GetCharacter());
	return JavaWebBrowser->SendKeyDown(InKeyEvent.GetCharacter()) ? FReply::Handled() : FReply::Unhandled();
}

FReply SAndroidWebBrowserWidget::OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (!IsAndroid3DBrowser)
	{
		return FReply::Unhandled();
	}

//	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("SAndroidWebBrowserWidget::OnKeyUp: %d"), InKeyEvent.GetCharacter());
	return JavaWebBrowser->SendKeyUp(InKeyEvent.GetCharacter()) ? FReply::Handled() : FReply::Unhandled();
}

FReply SAndroidWebBrowserWidget::OnKeyChar(const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent)
{
//	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("SAndroidWebBrowserWidget::OnKeyChar: %d"), (int32)InCharacterEvent.GetCharacter());
	if (IsAndroid3DBrowser)
	{
		if (JavaWebBrowser->SendKeyDown(InCharacterEvent.GetCharacter()))
		{
			JavaWebBrowser->SendKeyUp(InCharacterEvent.GetCharacter());
			return FReply::Handled();
		}
	}
	return FReply::Unhandled();
}

void SAndroidWebBrowserWidget::SetWebBrowserVisibility(bool InIsVisible)
{
	JavaWebBrowser->SetVisibility(InIsVisible);
}

jbyteArray SAndroidWebBrowserWidget::HandleShouldInterceptRequest(jstring JUrl)
{
	JNIEnv*	JEnv = FAndroidApplication::GetJavaEnv();

	FString Url = FJavaHelper::FStringFromParam(JEnv, JUrl);

	FString Response;
	bool bOverrideResponse = false;
	int32 Position = Url.Find(*FMobileJSScripting::JSMessageTag, ESearchCase::CaseSensitive);
	if (Position >= 0)
	{
		AsyncTask(ENamedThreads::GameThread, [Url, Position, this]()
		{
			if (WebBrowserWindowPtr.IsValid())
			{
				TSharedPtr<FAndroidWebBrowserWindow> BrowserWindow = WebBrowserWindowPtr.Pin();
				if (BrowserWindow.IsValid())
				{
					FString Origin = Url.Left(Position);
					FString Message = Url.RightChop(Position + FMobileJSScripting::JSMessageTag.Len());

					TArray<FString> Params;
					Message.ParseIntoArray(Params, TEXT("/"), false);
					if (Params.Num() > 0)
					{
						for (int I = 0; I < Params.Num(); I++)
						{
							Params[I] = FPlatformHttp::UrlDecode(Params[I]);
						}

						FString Command = Params[0];
						Params.RemoveAt(0, 1);
						BrowserWindow->OnJsMessageReceived(Command, Params, Origin);
					}
					else
					{
						GLog->Logf(ELogVerbosity::Error, TEXT("Invalid message from browser view: %s"), *Message);
					}
				}
			}
		});
		bOverrideResponse = true;
	}
	else
	{
	    FGraphEventRef OnLoadUrl = FFunctionGraphTask::CreateAndDispatchWhenReady([&]()
	    {
			if (WebBrowserWindowPtr.IsValid())
			{
				TSharedPtr<FAndroidWebBrowserWindow> BrowserWindow = WebBrowserWindowPtr.Pin();
				if (BrowserWindow.IsValid() && BrowserWindow->OnLoadUrl().IsBound())
				{
					FString Method; // We don't support passing anything but the requested URL
					bOverrideResponse = BrowserWindow->OnLoadUrl().Execute(Method, Url, Response);
				}
			}
		}, TStatId(), NULL, ENamedThreads::GameThread);

		FAppEventManager::GameThreadTicker Ticker;
		FTaskGraphInterface::Get().WaitUntilTaskCompletes(OnLoadUrl);
	}

	if ( bOverrideResponse )
	{
		FTCHARToUTF8 Converter(*Response);
		jbyteArray Buffer = JEnv->NewByteArray(Converter.Length());
		JEnv->SetByteArrayRegion(Buffer, 0, Converter.Length(), reinterpret_cast<const jbyte *>(Converter.Get()));
		return Buffer;
	}
	return nullptr;
}

bool SAndroidWebBrowserWidget::HandleShouldOverrideUrlLoading(jstring JUrl)
{
	JNIEnv*	JEnv = FAndroidApplication::GetJavaEnv();

	FString Url = FJavaHelper::FStringFromParam(JEnv, JUrl);
	bool Retval = false;
	FGraphEventRef OnBeforeBrowse = FFunctionGraphTask::CreateAndDispatchWhenReady([&]()
	{
		if (WebBrowserWindowPtr.IsValid())
		{
			TSharedPtr<FAndroidWebBrowserWindow> BrowserWindow = WebBrowserWindowPtr.Pin();
			if (BrowserWindow.IsValid())
			{
				if (BrowserWindow->OnBeforeBrowse().IsBound())
				{
					FWebNavigationRequest RequestDetails;
					RequestDetails.bIsRedirect = false;
					RequestDetails.bIsMainFrame = true; // shouldOverrideUrlLoading is only called on the main frame

					Retval = BrowserWindow->OnBeforeBrowse().Execute(Url, RequestDetails);
				}
			}
		}
	}, TStatId(), NULL, ENamedThreads::GameThread);

	FAppEventManager::GameThreadTicker Ticker;
	FTaskGraphInterface::Get().WaitUntilTaskCompletes(OnBeforeBrowse);

	return Retval;
}

bool SAndroidWebBrowserWidget::HandleJsDialog(TSharedPtr<IWebBrowserDialog>& Dialog)
{
	bool Retval = false;
	FGraphEventRef OnShowDialog = FFunctionGraphTask::CreateAndDispatchWhenReady([&]()
	{
		if (WebBrowserWindowPtr.IsValid())
		{
			TSharedPtr<FAndroidWebBrowserWindow> BrowserWindow = WebBrowserWindowPtr.Pin();
			if (BrowserWindow.IsValid() && BrowserWindow->OnShowDialog().IsBound())
			{
				EWebBrowserDialogEventResponse EventResponse = BrowserWindow->OnShowDialog().Execute(TWeakPtr<IWebBrowserDialog>(Dialog));
				switch (EventResponse)
				{
				case EWebBrowserDialogEventResponse::Handled:
					Retval = true;
					break;
				case EWebBrowserDialogEventResponse::Continue:
					Dialog->Continue(true, (Dialog->GetType() == EWebBrowserDialogType::Prompt) ? Dialog->GetDefaultPrompt() : FText::GetEmpty());
					Retval = true;
					break;
				case EWebBrowserDialogEventResponse::Ignore:
					Dialog->Continue(false);
					Retval = true;
					break;
				case EWebBrowserDialogEventResponse::Unhandled:
				default:
					Retval = false;
					break;
				}
			}
		}
	}, TStatId(), NULL, ENamedThreads::GameThread);

	FAppEventManager::GameThreadTicker Ticker;
	FTaskGraphInterface::Get().WaitUntilTaskCompletes(OnShowDialog);

	return Retval;
}

void SAndroidWebBrowserWidget::HandleReceivedTitle(jstring JTitle)
{
	JNIEnv*	JEnv = FAndroidApplication::GetJavaEnv();

	FString Title = FJavaHelper::FStringFromParam(JEnv, JTitle);

	if (WebBrowserWindowPtr.IsValid())
	{
		TSharedPtr<FAndroidWebBrowserWindow> BrowserWindow = WebBrowserWindowPtr.Pin();
		if (BrowserWindow.IsValid())
		{
			FGraphEventRef OnSetTitle = FFunctionGraphTask::CreateAndDispatchWhenReady([&]()
			{
				BrowserWindow->SetTitle(Title);
			}, TStatId(), NULL, ENamedThreads::GameThread);

			FAppEventManager::GameThreadTicker Ticker;
			FTaskGraphInterface::Get().WaitUntilTaskCompletes(OnSetTitle);
		}
	}
}

void SAndroidWebBrowserWidget::HandlePageLoad(jstring JUrl, bool bIsLoading, int InHistorySize, int InHistoryPosition)
{
	HistorySize = InHistorySize;
	HistoryPosition = InHistoryPosition;

	JNIEnv*	JEnv = FAndroidApplication::GetJavaEnv();

	FString Url = FJavaHelper::FStringFromParam(JEnv, JUrl);
	if (WebBrowserWindowPtr.IsValid())
	{
		TSharedPtr<FAndroidWebBrowserWindow> BrowserWindow = WebBrowserWindowPtr.Pin();
		if (BrowserWindow.IsValid())
		{
			FGraphEventRef OnNotifyDocumentLoadingStateChange = FFunctionGraphTask::CreateAndDispatchWhenReady([&]()
			{
				BrowserWindow->NotifyDocumentLoadingStateChange(Url, bIsLoading);
			}, TStatId(), NULL, ENamedThreads::GameThread);

			FAppEventManager::GameThreadTicker Ticker;
			FTaskGraphInterface::Get().WaitUntilTaskCompletes(OnNotifyDocumentLoadingStateChange);
		}
	}
}

void SAndroidWebBrowserWidget::HandleReceivedError(jint ErrorCode, jstring /* ignore */, jstring JUrl)
{
	JNIEnv*	JEnv = FAndroidApplication::GetJavaEnv();

	FString Url = FJavaHelper::FStringFromParam(JEnv, JUrl);
	if (WebBrowserWindowPtr.IsValid())
	{
		TSharedPtr<FAndroidWebBrowserWindow> BrowserWindow = WebBrowserWindowPtr.Pin();
		if (BrowserWindow.IsValid())
		{
			FGraphEventRef OnNotifyDocumentError = FFunctionGraphTask::CreateAndDispatchWhenReady([&]()
			{
				BrowserWindow->NotifyDocumentError(Url, ErrorCode);
			}, TStatId(), NULL, ENamedThreads::GameThread);

			FAppEventManager::GameThreadTicker Ticker;
			FTaskGraphInterface::Get().WaitUntilTaskCompletes(OnNotifyDocumentError);
		}
	}
}

void SAndroidWebBrowserWidget::FloatingCloseButtonPressed()
{
	if (WebBrowserWindowPtr.IsValid())
	{
		TSharedPtr<FAndroidWebBrowserWindow> BrowserWindow = WebBrowserWindowPtr.Pin();
		if (BrowserWindow.IsValid())
		{
			FGraphEventRef OnRequestClose = FFunctionGraphTask::CreateAndDispatchWhenReady([&]()
				{
					BrowserWindow->FloatingCloseButtonPressed();
				}, TStatId(), NULL, ENamedThreads::GameThread);

			FAppEventManager::GameThreadTicker Ticker;
			FTaskGraphInterface::Get().WaitUntilTaskCompletes(OnRequestClose);
		}
	}
}

// Native method implementations:

void JNICALL UE::Jni::FWebViewControl::FloatingCloseButtonPressed(JNIEnv* JEnv, jobject Client)
{
	TSharedPtr<SAndroidWebBrowserWidget> Widget = SAndroidWebBrowserWidget::GetWidgetPtr(JEnv, Client);
	if (Widget.IsValid())
	{
		Widget->FloatingCloseButtonPressed();
	}
}

UE::Jni::Java::Lang::TArray<jbyte>* JNICALL UE::Jni::FWebViewControl::FViewClient::shouldInterceptRequestImpl(JNIEnv* JEnv, jobject Client, jstring JUrl)
{
	TSharedPtr<SAndroidWebBrowserWidget> Widget = SAndroidWebBrowserWidget::GetWidgetPtr(JEnv, Client);
	if (Widget.IsValid())
	{
		return reinterpret_cast<Java::Lang::TArray<jbyte>*>(Widget->HandleShouldInterceptRequest(JUrl));
	}
	else
	{
		return nullptr;
	}
}

jboolean JNICALL UE::Jni::FWebViewControl::FViewClient::shouldOverrideUrlLoading(JNIEnv* JEnv, jobject Client, Android::WebKit::FWebView* /* ignore */, jstring JUrl)
{
	TSharedPtr<SAndroidWebBrowserWidget> Widget = SAndroidWebBrowserWidget::GetWidgetPtr(JEnv, Client);
	if (Widget.IsValid())
	{
		return Widget->HandleShouldOverrideUrlLoading(JUrl);
	}
	else
	{
		return false;
	}
}

void JNICALL UE::Jni::FWebViewControl::FViewClient::onPageLoad(JNIEnv* JEnv, jobject Client, jstring JUrl, jboolean bIsLoading, jint HistorySize, jint HistoryPosition)
{
	TSharedPtr<SAndroidWebBrowserWidget> Widget = SAndroidWebBrowserWidget::GetWidgetPtr(JEnv, Client);
	if (Widget.IsValid())
	{
		Widget->HandlePageLoad(JUrl, bIsLoading, HistorySize, HistoryPosition);
	}
}

void JNICALL UE::Jni::FWebViewControl::FViewClient::onReceivedError(JNIEnv* JEnv, jobject Client, Android::WebKit::FWebView* /* ignore */, jint ErrorCode, jstring Description, jstring JUrl)
{
	TSharedPtr<SAndroidWebBrowserWidget> Widget = SAndroidWebBrowserWidget::GetWidgetPtr(JEnv, Client);
	if (Widget.IsValid())
	{
		Widget->HandleReceivedError(ErrorCode, Description, JUrl);
	}
}

jboolean JNICALL UE::Jni::FWebViewControl::FChromeClient::onJsAlert(JNIEnv* JEnv, jobject Client, Android::WebKit::FWebView* /* ignore */, jstring JUrl, jstring Message, Android::WebKit::FJsResult* Result)
{
	TSharedPtr<SAndroidWebBrowserWidget> Widget = SAndroidWebBrowserWidget::GetWidgetPtr(JEnv, Client);
	if (Widget.IsValid())
	{
		return Widget->HandleJsDialog(EWebBrowserDialogType::Alert, JUrl, Message, Result);
	}
	else
	{
		return false;
	}
}

jboolean JNICALL UE::Jni::FWebViewControl::FChromeClient::onJsBeforeUnload(JNIEnv* JEnv, jobject Client, Android::WebKit::FWebView* /* ignore */, jstring JUrl, jstring Message, Android::WebKit::FJsResult* Result)
{
	TSharedPtr<SAndroidWebBrowserWidget> Widget = SAndroidWebBrowserWidget::GetWidgetPtr(JEnv, Client);
	if (Widget.IsValid())
	{
		return Widget->HandleJsDialog(EWebBrowserDialogType::Unload, JUrl, Message, Result);
	}
	else
	{
		return false;
	}
}

jboolean JNICALL UE::Jni::FWebViewControl::FChromeClient::onJsConfirm(JNIEnv* JEnv, jobject Client, Android::WebKit::FWebView* /* ignore */, jstring JUrl, jstring Message, Android::WebKit::FJsResult* Result)
{
	TSharedPtr<SAndroidWebBrowserWidget> Widget = SAndroidWebBrowserWidget::GetWidgetPtr(JEnv, Client);
	if (Widget.IsValid())
	{
		return Widget->HandleJsDialog(EWebBrowserDialogType::Confirm, JUrl, Message, Result);
	}
	else
	{
		return false;
	}
}

jboolean JNICALL UE::Jni::FWebViewControl::FChromeClient::onJsPrompt(JNIEnv* JEnv, jobject Client, Android::WebKit::FWebView* /* ignore */, jstring JUrl, jstring Message, jstring DefaultValue, Android::WebKit::FJsPromptResult* Result)
{
	TSharedPtr<SAndroidWebBrowserWidget> Widget = SAndroidWebBrowserWidget::GetWidgetPtr(JEnv, Client);
	if (Widget.IsValid())
	{
		return Widget->HandleJsPrompt(JUrl, Message, DefaultValue, Result);
	}
	else
	{
		return false;
	}
}

void JNICALL UE::Jni::FWebViewControl::FChromeClient::onReceivedTitle(JNIEnv* JEnv, jobject Client, Android::WebKit::FWebView* /* ignore */, jstring Title)
{
	TSharedPtr<SAndroidWebBrowserWidget> Widget = SAndroidWebBrowserWidget::GetWidgetPtr(JEnv, Client);
	if (Widget.IsValid())
	{
		Widget->HandleReceivedTitle(Title);
	}
}

#endif // USE_ANDROID_JNI
