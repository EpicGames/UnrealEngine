// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if USE_ANDROID_JNI

#include "Widgets/SLeafWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "AndroidWebBrowserWindow.h"
#include "AndroidWebBrowserDialog.h"
#include "Android/AndroidJava.h"
#include "RHI.h"
#include "RHIResources.h"
#include "UObject/Class.h"
#include "UObject/UObjectGlobals.h"
#include "AndroidJavaWebBrowser.h"
#include "Engine/Texture2D.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "WebBrowserTexture.h"

#include <jni.h>

class UMaterialExpressionTextureSample;
class FWebBrowserTextureSamplePool;

class SAndroidWebBrowserWidget : public SViewport
{
	SLATE_BEGIN_ARGS(SAndroidWebBrowserWidget)
		: _InitialURL("about:blank")
		, _UseTransparency(false)
		, _EnablePaymentRequest(false)
	{ }

		SLATE_ARGUMENT(FString, InitialURL);
		SLATE_ARGUMENT(bool, UseTransparency);
		SLATE_ARGUMENT(TSharedPtr<FAndroidWebBrowserWindow>, WebBrowserWindow);
		SLATE_ARGUMENT(FString, UserAgentApplication);
		SLATE_ARGUMENT(bool, EnablePaymentRequest);

	SLATE_END_ARGS()

public:
	virtual ~SAndroidWebBrowserWidget();

	void Construct(const FArguments& Args);

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FVector2D ComputeDesiredSize(float) const override;

	void ExecuteJavascript(const FString& Script);
	void LoadURL(const FString& NewURL);
	void LoadString(const FString& Content, const FString& BaseUrl);

	void StopLoad();
	void Reload();
	void Close();
	void ShowFloatingCloseButton(bool bShow, bool bDraggable);
	void GoBack();
	void GoForward();
	bool CanGoBack();
	bool CanGoForward();
	void FloatingCloseButtonPressed();

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	void SendTouchDown(FVector2D Position);
	void SendTouchUp(FVector2D Position);
	void SendTouchMove(FVector2D Position);

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FReply OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FReply OnKeyChar(const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent) override;

	// WebViewClient callbacks

	jbyteArray HandleShouldInterceptRequest(jstring JUrl);

	bool HandleShouldOverrideUrlLoading(jstring JUrl);
	bool HandleJsDialog(EWebBrowserDialogType Type, jstring JUrl, jstring MessageText, jobject ResultCallback)
	{
		TSharedPtr<IWebBrowserDialog> Dialog(new FWebBrowserDialog(Type, MessageText, ResultCallback));
		return HandleJsDialog(Dialog);
	}

	bool HandleJsPrompt(jstring JUrl, jstring MessageText, jstring DefaultPrompt, jobject ResultCallback)
	{
		TSharedPtr<IWebBrowserDialog> Dialog(new FWebBrowserDialog(MessageText, DefaultPrompt, ResultCallback));
		return HandleJsDialog(Dialog);
	}

	void HandleReceivedTitle(jstring JTitle);
	void HandlePageLoad(jstring JUrl, bool bIsLoading, int InHistorySize, int InHistoryPosition);
	void HandleReceivedError(jint ErrorCode, jstring JMessage, jstring JUrl);

	// Helper to get the native widget pointer from a Java callback.
	// Jobj can either be a WebViewControl, a WebViewControl.ViewClient or WebViewControl.ChromeClient instance
	static TSharedPtr<SAndroidWebBrowserWidget> GetWidgetPtr(JNIEnv* JEnv, jobject Jobj);

	//set the native control's visibility
	void SetWebBrowserVisibility(bool InIsVisible);
protected:
	static FCriticalSection WebControlsCS;
	static TMap<int64, TWeakPtr<SAndroidWebBrowserWidget>> AllWebControls;

	bool HandleJsDialog(TSharedPtr<IWebBrowserDialog>& Dialog);
	int HistorySize;
	int HistoryPosition;

	TWeakPtr<FAndroidWebBrowserWindow> WebBrowserWindowPtr;
private:
	FVector2D ConvertMouseEventToLocal(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/** Enable 3D appearance for Android. */
	bool IsAndroid3DBrowser;

	/** Should use bitmap rendering for 3D. */
	bool bShouldUseBitmapRender;

	/** Mouse captured */
	bool bMouseCapture;

	/** The Java side webbrowser interface. */
	TSharedPtr<FJavaAndroidWebBrowser, ESPMode::ThreadSafe> JavaWebBrowser;

	/** The external texture to render the webbrowser output. */
	UWebBrowserTexture* WebBrowserTexture;

	/** The material for the external texture. */
	UMaterialInstanceDynamic* WebBrowserMaterial;

	/** The Slate brush that renders the material. */
	TSharedPtr<FSlateBrush> WebBrowserBrush;

	/** The sample queue. */
	TSharedPtr<FWebBrowserTextureSampleQueue, ESPMode::ThreadSafe> WebBrowserTextureSamplesQueue;

	/** Texture sample object pool. */
	FWebBrowserTextureSamplePool* TextureSamplePool;
};

#endif // USE_ANDROID_JNI
