// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidWebBrowserWindow.h"

#if USE_ANDROID_JNI

#include "AndroidWebBrowserDialog.h"
#include "AndroidWebBrowserWidget.h"
#include "Android/AndroidApplication.h"
#include "Android/AndroidWindow.h"
#include "Android/AndroidJava.h"
#include "Misc/ConfigCacheIni.h"

#include <jni.h>


namespace {

	static const FString JSGetSourceCommand = "GetSource";
	static const FString JSMessageGetSourceScript =
		TEXT("document.location = '") + FMobileJSScripting::JSMessageTag + JSGetSourceCommand +
		TEXT("/' + encodeURIComponent(document.documentElement.innerHTML);");

}

FAndroidWebBrowserWindow::FAndroidWebBrowserWindow(FString InUrl, TOptional<FString> InContentsToLoad, bool InShowErrorMessage, bool InThumbMouseButtonNavigation, bool InUseTransparency, bool bInJSBindingToLoweringEnabled, const FString& InUserAgentApplication, bool bMobileJSReturnInDict, bool bInEnablePaymentRequest)
	: CurrentUrl(MoveTemp(InUrl))
	, ContentsToLoad(MoveTemp(InContentsToLoad))
	, UserAgentApplication(InUserAgentApplication)
	, bUseTransparency(InUseTransparency)
	, bEnablePaymentRequest(bInEnablePaymentRequest)
	, DocumentState(EWebBrowserDocumentState::NoDocument)
	, ErrorCode(0)
	, AndroidWindowSize(FIntPoint(1024, 768))
	, bIsDisabled(false)
	, bIsVisible(true)
	, bTickedLastFrame(true)
{
	bool bInjectJSOnPageStarted = false;
	GConfig->GetBool(TEXT("Browser"), TEXT("bInjectJSOnPageStarted"), bInjectJSOnPageStarted, GEngineIni);
	Scripting = MakeShared<FMobileJSScripting>(bInJSBindingToLoweringEnabled, bInjectJSOnPageStarted, bMobileJSReturnInDict);
	// Deal with optional texture resolution override
	FString WebViewTextureSize;
	if (GConfig->GetString(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("WebViewTextureSize"), WebViewTextureSize, GEngineIni))
	{
		TArray<FString> ResolutionVector;
		int ParseCount = WebViewTextureSize.ParseIntoArray(ResolutionVector, TEXT(","), false);
		ensureMsgf(ParseCount == 2, TEXT("WebViewTextureSize variable not properly formatted."));

		if (ParseCount == 2)
		{
			int32 Width = FCString::Atof(*ResolutionVector[0]);
			int32 Height = FCString::Atof(*ResolutionVector[1]);

			AndroidWindowSize.X = FMath::Max(1, Width);
			AndroidWindowSize.Y = FMath::Max(1, Height);
		}
	}
}

FAndroidWebBrowserWindow::~FAndroidWebBrowserWindow()
{
	CloseBrowser(true, false);
}

void FAndroidWebBrowserWindow::LoadURL(FString NewURL)
{
	BrowserWidget->LoadURL(NewURL);
}

void FAndroidWebBrowserWindow::LoadString(FString Contents, FString DummyURL)
{
	BrowserWidget->LoadString(Contents, DummyURL);
}

TSharedRef<SWidget> FAndroidWebBrowserWindow::CreateWidget()
{
	TSharedRef<SAndroidWebBrowserWidget> BrowserWidgetRef =
		SNew(SAndroidWebBrowserWidget)
		.UseTransparency(bUseTransparency)
		.InitialURL(CurrentUrl)
		.UserAgentApplication(UserAgentApplication)
		.WebBrowserWindow(SharedThis(this))
		.EnablePaymentRequest(bEnablePaymentRequest);

	BrowserWidget = BrowserWidgetRef;

	Scripting->SetWindow(SharedThis(this));
		
	return BrowserWidgetRef;
}

void FAndroidWebBrowserWindow::SetViewportSize(FIntPoint WindowSize, FIntPoint WindowPos)
{
	AndroidWindowSize = WindowSize;
}

FIntPoint FAndroidWebBrowserWindow::GetViewportSize() const
{
	return AndroidWindowSize;
}

FSlateShaderResource* FAndroidWebBrowserWindow::GetTexture(bool bIsPopup /*= false*/)
{
	return nullptr;
}

bool FAndroidWebBrowserWindow::IsValid() const
{
	return BrowserWidget.IsValid();
}

bool FAndroidWebBrowserWindow::IsInitialized() const
{
	return true;
}

bool FAndroidWebBrowserWindow::IsClosing() const
{
	return false;
}

EWebBrowserDocumentState FAndroidWebBrowserWindow::GetDocumentLoadingState() const
{
	return DocumentState;
}

FString FAndroidWebBrowserWindow::GetTitle() const
{
	return Title;
}

FString FAndroidWebBrowserWindow::GetUrl() const
{
	return CurrentUrl;
}

bool FAndroidWebBrowserWindow::OnKeyDown(const FKeyEvent& InKeyEvent)
{
//	return BrowserWidget->OnKeyDown(FGeometry(), InKeyEvent).IsEventHandled();
	return false;
}

bool FAndroidWebBrowserWindow::OnKeyUp(const FKeyEvent& InKeyEvent)
{
//	return BrowserWidget->OnKeyUp(FGeometry(), InKeyEvent).IsEventHandled();
	return false;
}

bool FAndroidWebBrowserWindow::OnKeyChar(const FCharacterEvent& InCharacterEvent)
{
//	return BrowserWidget->OnKeyChar(FGeometry(), InCharacterEvent).IsEventHandled();
	return false;
}

FVector2D FAndroidWebBrowserWindow::ConvertMouseEventToLocal(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup)
{
	FGeometry MouseGeometry = MyGeometry;

	float DPIScale = MouseGeometry.Scale;
	FVector2D LocalPos = MouseGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()) * DPIScale;

	return LocalPos;
}

FReply FAndroidWebBrowserWindow::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup)
{
	FReply Reply = FReply::Unhandled();
/*
	FKey Button = MouseEvent.GetEffectingButton();
	bool bSupportedButton = (Button == EKeys::LeftMouseButton); // || Button == EKeys::RightMouseButton || Button == EKeys::MiddleMouseButton);

	if (bSupportedButton)
	{
		Reply = FReply::Handled();
		BrowserWidget->SendTouchDown(ConvertMouseEventToLocal(MyGeometry, MouseEvent, bIsPopup));
	}
*/
	return Reply;
}

FReply FAndroidWebBrowserWindow::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup)
{
	FReply Reply = FReply::Unhandled();
/*
	FKey Button = MouseEvent.GetEffectingButton();
	bool bSupportedButton = (Button == EKeys::LeftMouseButton); // || Button == EKeys::RightMouseButton || Button == EKeys::MiddleMouseButton);

	if (bSupportedButton)
	{
		Reply = FReply::Handled();
		BrowserWidget->SendTouchUp(ConvertMouseEventToLocal(MyGeometry, MouseEvent, bIsPopup));
	}
*/
	return Reply;
}

FReply FAndroidWebBrowserWindow::OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup)
{
	return FReply::Unhandled();
}

FReply FAndroidWebBrowserWindow::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup)
{
	FReply Reply = FReply::Unhandled();
/*
	FKey Button = MouseEvent.GetEffectingButton();
	bool bSupportedButton = (Button == EKeys::LeftMouseButton); // || Button == EKeys::RightMouseButton || Button == EKeys::MiddleMouseButton);

	if (bSupportedButton)
	{
		Reply = FReply::Handled();
		BrowserWidget->SendTouchMove(ConvertMouseEventToLocal(MyGeometry, MouseEvent, bIsPopup));
	}
*/
	return Reply;
}

void FAndroidWebBrowserWindow::OnMouseLeave(const FPointerEvent& MouseEvent)
{
}

void FAndroidWebBrowserWindow::SetSupportsMouseWheel(bool bValue)
{
}

bool FAndroidWebBrowserWindow::GetSupportsMouseWheel() const
{
	return false;
}

FReply FAndroidWebBrowserWindow::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup)
{
	return FReply::Unhandled();
}

FReply FAndroidWebBrowserWindow::OnTouchGesture(const FGeometry& MyGeometry, const FPointerEvent& GestureEvent, bool bIsPopup)
{
	return FReply::Unhandled();
}

void FAndroidWebBrowserWindow::OnFocus(bool SetFocus, bool bIsPopup)
{
}

void FAndroidWebBrowserWindow::OnCaptureLost()
{
}

bool FAndroidWebBrowserWindow::CanGoBack() const
{
	return BrowserWidget->CanGoBack();
}

void FAndroidWebBrowserWindow::GoBack()
{
	BrowserWidget->GoBack();
}

bool FAndroidWebBrowserWindow::CanGoForward() const
{
	return BrowserWidget->CanGoForward();
}

void FAndroidWebBrowserWindow::GoForward()
{
	BrowserWidget->GoForward();
}

bool FAndroidWebBrowserWindow::IsLoading() const
{
	return DocumentState != EWebBrowserDocumentState::Loading;
}

void FAndroidWebBrowserWindow::Reload()
{
	BrowserWidget->Reload();
}

void FAndroidWebBrowserWindow::StopLoad()
{
	BrowserWidget->StopLoad();
}

void FAndroidWebBrowserWindow::GetSource(TFunction<void (const FString&)> Callback) const
{
	//@todo: decide what to do about multiple pending requests
	GetPageSourceCallback.Emplace(Callback);

	// Ugly hack: Work around the fact that ExecuteJavascript is non-const.
	const_cast<FAndroidWebBrowserWindow*>(this)->ExecuteJavascript(JSMessageGetSourceScript);
}

int FAndroidWebBrowserWindow::GetLoadError()
{
	return ErrorCode;
}

void FAndroidWebBrowserWindow::NotifyDocumentError(const FString& InCurrentUrl, int InErrorCode)
{
	if(!CurrentUrl.Equals(InCurrentUrl, ESearchCase::CaseSensitive))
	{
		CurrentUrl = InCurrentUrl;
		UrlChangedEvent.Broadcast(CurrentUrl);
	}

	ErrorCode = InErrorCode;
	DocumentState = EWebBrowserDocumentState::Error;
	DocumentStateChangedEvent.Broadcast(DocumentState);
}

void FAndroidWebBrowserWindow::NotifyDocumentLoadingStateChange(const FString& InCurrentUrl, bool IsLoading)
{
	// This method may called several times with IsLoading = false (one time for each frame in the page)
	// Ignore a load completed notification if there was an error.
	// For load started, reset any errors from previous page load.
	bool bIsNotError = DocumentState != EWebBrowserDocumentState::Error;
	if (IsLoading || bIsNotError)
	{
		if (!CurrentUrl.Equals(InCurrentUrl, ESearchCase::CaseSensitive))
		{
			CurrentUrl = InCurrentUrl;
			UrlChangedEvent.Broadcast(CurrentUrl);
		}

		if (bIsNotError && !InCurrentUrl.StartsWith("javascript:"))
		{
			if (IsLoading)
			{
				Scripting->PageStarted(SharedThis(this));
			}
			else
			{
				Scripting->PageLoaded(SharedThis(this));
			}
		}
		ErrorCode = 0;
		DocumentState = IsLoading
			? EWebBrowserDocumentState::Loading
			: EWebBrowserDocumentState::Completed;
		DocumentStateChangedEvent.Broadcast(DocumentState);
	}

}

void FAndroidWebBrowserWindow::SetIsDisabled(bool bValue)
{
	bIsDisabled = bValue;
}

TSharedPtr<SWindow> FAndroidWebBrowserWindow::GetParentWindow() const
{
	return ParentWindow;
}

void FAndroidWebBrowserWindow::SetParentWindow(TSharedPtr<SWindow> Window)
{
	ParentWindow = Window;
}

void FAndroidWebBrowserWindow::ShowFloatingCloseButton(bool bShow, bool bDraggable)
{
	BrowserWidget->ShowFloatingCloseButton(bShow, bDraggable);
}

void FAndroidWebBrowserWindow::ExecuteJavascript(const FString& Script)
{
	BrowserWidget->ExecuteJavascript(Script);
}

void FAndroidWebBrowserWindow::CloseBrowser(bool bForce, bool bBlockTillClosed /* ignored */)
{
	BrowserWidget->Close();
}

bool FAndroidWebBrowserWindow::OnJsMessageReceived(const FString& Command, const TArray<FString>& Params, const FString& Origin)
{
	if( Command.Equals(JSGetSourceCommand, ESearchCase::CaseSensitive) && GetPageSourceCallback.IsSet() && Params.Num() == 1)
	{
		GetPageSourceCallback.GetValue()(Params[0]);
		GetPageSourceCallback.Reset();
		return true;
	}
	return Scripting->OnJsMessageReceived(Command, Params, Origin);
}

void FAndroidWebBrowserWindow::BindUObject(const FString& Name, UObject* Object, bool bIsPermanent /*= true*/)
{
	Scripting->BindUObject(SharedThis(this), Name, Object, bIsPermanent);
}

void FAndroidWebBrowserWindow::UnbindUObject(const FString& Name, UObject* Object /*= nullptr*/, bool bIsPermanent /*= true*/)
{
	Scripting->UnbindUObject(SharedThis(this), Name, Object, bIsPermanent);
}

void FAndroidWebBrowserWindow::CheckTickActivity()
{
	if (bIsVisible != bTickedLastFrame)
	{
		bIsVisible = bTickedLastFrame;
		BrowserWidget->SetWebBrowserVisibility(bIsVisible);
	}

	bTickedLastFrame = false;
}

void FAndroidWebBrowserWindow::SetTickLastFrame()
{
	bTickedLastFrame = !bIsDisabled;
}

bool FAndroidWebBrowserWindow::IsVisible()
{
	return bIsVisible;
}

void FAndroidWebBrowserWindow::FloatingCloseButtonPressed()
{
	if (OnFloatingCloseButtonPressed().IsBound())
	{
		OnFloatingCloseButtonPressed().Execute();
	}
}

void FAndroidWebBrowserWindow::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	// Not supported on this platform
}

FReply FAndroidWebBrowserWindow::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	// Not supported on this platform
	return FReply::Unhandled();
}

void FAndroidWebBrowserWindow::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	// Not supported on this platform
}

FReply FAndroidWebBrowserWindow::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	// Not supported on this platform
	return FReply::Unhandled();
}

#endif // USE_ANDROID_JNI
