// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if USE_ANDROID_JNI

#include "IWebBrowserWindow.h"
#include "Widgets/SWidget.h"
#include "MobileJS/MobileJSScripting.h"

class SAndroidWebBrowserWidget;
class SWebBrowserView;

/**
* Implementation of interface for dealing with a Web Browser window.
*/
class FAndroidWebBrowserWindow
	: public IWebBrowserWindow
	, public TSharedFromThis<FAndroidWebBrowserWindow>
{
	// The WebBrowserSingleton should be the only one creating instances of this class
	friend class FWebBrowserSingleton;
	// CreateWidget should only be called by the WebBrowserView
	friend class SWebBrowserView;
	friend class SAndroidWebBrowserWidget;

private:
	/**
	 * Creates and initializes a new instance.
	 *
	 * @param Url The Initial URL that will be loaded.
	 * @param ContentsToLoad Optional string to load as a web page.
	 * @param bShowErrorMessage Whether to show an error message in case of loading errors.
	 * @param bThumbMouseButtonNavigation Whether to allow forward and back navigation via the mouse thumb buttons.
	 * @param bUseTransparency Whether to enable transparency.
	 * @param bJSBindingToLoweringEnabled Whether we ToLower all JavaScript member names.
	 * @param bMobileJSReturnInDict Whether to pass a UObject wrapper function's return value in a dictionary with the property's name or not.
	 * @param bInEnablePaymentRequest Whether to enable the W3C PaymentRequest API on this window
	 */
	FAndroidWebBrowserWindow(FString Url, TOptional<FString> ContentsToLoad, bool bShowErrorMessage, bool bThumbMouseButtonNavigation, bool bUseTransparency, bool bJSBindingToLoweringEnabled, const FString& InUserAgentApplication, bool bMobileJSReturnInDict, bool bInEnablePaymentRequest);

	/**
	 * Create the SWidget for this WebBrowserWindow
	 */
	TSharedRef<SWidget> CreateWidget();

public:
	/** Virtual Destructor. */
	virtual ~FAndroidWebBrowserWindow();

public:

	// IWebBrowserWindow Interface

	virtual void LoadURL(FString NewURL) override;
	virtual void LoadString(FString Contents, FString DummyURL) override;
	virtual void SetViewportSize(FIntPoint WindowSize, FIntPoint WindowPos) override;
	virtual FIntPoint GetViewportSize() const override;
	virtual FSlateShaderResource* GetTexture(bool bIsPopup = false) override;
	virtual bool IsValid() const override;
	virtual bool IsInitialized() const override;
	virtual bool IsClosing() const override;
	virtual EWebBrowserDocumentState GetDocumentLoadingState() const override;
	virtual FString GetTitle() const override;
	virtual FString GetUrl() const override;
	virtual void GetSource(TFunction<void (const FString&)> Callback) const override;
	virtual bool OnKeyDown(const FKeyEvent& InKeyEvent) override;
	virtual bool OnKeyUp(const FKeyEvent& InKeyEvent) override;
	virtual bool OnKeyChar(const FCharacterEvent& InCharacterEvent) override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup) override;
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup) override;
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	virtual void SetSupportsMouseWheel(bool bValue) override;
	virtual bool GetSupportsMouseWheel() const override;
	virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup) override;
	virtual FReply OnTouchGesture(const FGeometry& MyGeometry, const FPointerEvent& GestureEvent, bool bIsPopup) override;
	virtual void OnFocus(bool SetFocus, bool bIsPopup) override;
	virtual void OnCaptureLost() override;
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual bool CanGoBack() const override;
	virtual void GoBack() override;
	virtual bool CanGoForward() const override;
	virtual void GoForward() override;
	virtual bool IsLoading() const override;
	virtual void Reload() override;
	virtual void StopLoad() override;
	virtual void ExecuteJavascript(const FString& Script) override;
	virtual void CloseBrowser(bool bForce, bool bBlockTillClosed) override;
	virtual void BindUObject(const FString& Name, UObject* Object, bool bIsPermanent = true) override;
	virtual void UnbindUObject(const FString& Name, UObject* Object = nullptr, bool bIsPermanent = true) override;
	virtual int GetLoadError() override;
	virtual void SetIsDisabled(bool bValue) override;
	virtual TSharedPtr<SWindow> GetParentWindow() const override;
	virtual void SetParentWindow(TSharedPtr<SWindow> Window) override;
	virtual void ShowFloatingCloseButton(bool bShow, bool bDraggable) override;

	DECLARE_DERIVED_EVENT(FAndroidWebBrowserWindow, IWebBrowserWindow::FOnDocumentStateChanged, FOnDocumentStateChanged);
	virtual FOnDocumentStateChanged& OnDocumentStateChanged() override
	{
		return DocumentStateChangedEvent;
	}

	DECLARE_DERIVED_EVENT(FAndroidWebBrowserWindow, IWebBrowserWindow::FOnTitleChanged, FOnTitleChanged);
	virtual FOnTitleChanged& OnTitleChanged() override
	{
		return TitleChangedEvent;
	}

	DECLARE_DERIVED_EVENT(FAndroidWebBrowserWindow, IWebBrowserWindow::FOnUrlChanged, FOnUrlChanged);
	virtual FOnUrlChanged& OnUrlChanged() override
	{
		return UrlChangedEvent;
	}

	DECLARE_DERIVED_EVENT(FAndroidWebBrowserWindow, IWebBrowserWindow::FOnToolTip, FOnToolTip);
	virtual FOnToolTip& OnToolTip() override
	{
		return ToolTipEvent;
	}

	DECLARE_DERIVED_EVENT(FAndroidWebBrowserWindow, IWebBrowserWindow::FOnNeedsRedraw, FOnNeedsRedraw);
	virtual FOnNeedsRedraw& OnNeedsRedraw() override
	{
		return NeedsRedrawEvent;
	}

	virtual FOnBeforeBrowse& OnBeforeBrowse() override
	{
		return BeforeBrowseDelegate;
	}

	virtual FOnLoadUrl& OnLoadUrl() override
	{
		return LoadUrlDelegate;
	}

	virtual FOnCreateWindow& OnCreateWindow() override
	{
		return CreateWindowDelegate;
	}

	virtual FOnCloseWindow& OnCloseWindow() override
	{
		return CloseWindowDelegate;
	}

	virtual FOnFloatingCloseButtonPressed& OnFloatingCloseButtonPressed() override
	{
		return FloatingCloseButtonPressedDelegate;
	}

	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) override
	{
		return FCursorReply::Unhandled();
	}

	virtual FOnBeforePopupDelegate& OnBeforePopup() override
	{
		return BeforePopupDelegate;
	}

	virtual FOnBeforeResourceLoadDelegate& OnBeforeResourceLoad() override
	{
		return BeforeResourceLoadDelegate;
	}

	virtual FOnResourceLoadCompleteDelegate& OnResourceLoadComplete() override
	{
		return ResourceLoadCompleteDelegate;
	}

	virtual FOnConsoleMessageDelegate& OnConsoleMessage() override
	{
		return ConsoleMessageDelegate;
	}

	DECLARE_DERIVED_EVENT(FAndroidWebBrowserWindow, IWebBrowserWindow::FOnShowPopup, FOnShowPopup);
	virtual FOnShowPopup& OnShowPopup() override
	{
		return ShowPopupEvent;
	}

	DECLARE_DERIVED_EVENT(FAndroidWebBrowserWindow, IWebBrowserWindow::FOnDismissPopup, FOnDismissPopup);
	virtual FOnDismissPopup& OnDismissPopup() override
	{
		return DismissPopupEvent;
	}

	virtual FOnShowDialog& OnShowDialog() override
	{
		return ShowDialogDelegate;
	}

	virtual FOnDismissAllDialogs& OnDismissAllDialogs() override
	{
		return DismissAllDialogsDelegate;
	}

	virtual FOnSuppressContextMenu& OnSuppressContextMenu() override
	{
		return SuppressContextMenuDelgate;
	}

	virtual FOnDragWindow& OnDragWindow() override
	{
		return DragWindowDelegate;
	}
	
	virtual FOnUnhandledKeyDown& OnUnhandledKeyDown() override
	{
		return UnhandledKeyDownDelegate;
	}

	virtual FOnUnhandledKeyUp& OnUnhandledKeyUp() override
	{
		return UnhandledKeyUpDelegate;
	}

	virtual FOnUnhandledKeyChar& OnUnhandledKeyChar() override
	{
		return UnhandledKeyCharDelegate;
	}

public:

	/**
	* Called from the WebBrowserSingleton tick event. Should test whether the widget got a tick from Slate last frame and set the state to hidden if not.
	*/
	void CheckTickActivity() override;

	/**
	* Signal from the widget, meaning that the widget is still active
	*/
	void SetTickLastFrame();

	/**
	* Browser's visibility
	*/
	bool IsVisible();

	/**
	* Webvew floating close button was pressed
	*/
	void FloatingCloseButtonPressed();
private:

	void SetTitle(const FString& InTitle)
	{
		Title = InTitle;
		OnTitleChanged().Broadcast(Title);
	}

	void NotifyDocumentError(const FString& InCurrentUrl, int InErrorCode);
	void NotifyDocumentLoadingStateChange(const FString& InCurrentUrl, bool IsLoading);

	bool OnJsMessageReceived(const FString& Command, const TArray<FString>& Params, const FString& Origin);

	FVector2D ConvertMouseEventToLocal(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup);

	TSharedPtr<SAndroidWebBrowserWidget> BrowserWidget;

	/** Current title of this window. */
	FString Title;

	/** Current Url of this window. */
	FString CurrentUrl;

	/** Optional text to load as a web page. */
	TOptional<FString> ContentsToLoad;

	/** User Agent Application to report */
	FString UserAgentApplication;

	/** Whether to enable background transparency */
	bool bUseTransparency;

	/** Whether to enable the W3C PaymentRequest API on this window. */
	bool bEnablePaymentRequest;

	/** Delegate for broadcasting load state changes. */
	FOnDocumentStateChanged DocumentStateChangedEvent;

	/** Delegate for broadcasting title changes. */
	FOnTitleChanged TitleChangedEvent;

	/** Delegate for broadcasting address changes. */
	FOnUrlChanged UrlChangedEvent;

	/** Delegate for broadcasting when the browser wants to show a tool tip. */
	FOnToolTip ToolTipEvent;

	/** Delegate for notifying that the window needs refreshing. */
	FOnNeedsRedraw NeedsRedrawEvent;

	/** Delegate that is executed prior to browser navigation. */
	FOnBeforeBrowse BeforeBrowseDelegate;

	/** Delegate for overriding Url contents. */
	FOnLoadUrl LoadUrlDelegate;

	/** Delegate for notifying that a popup window is attempting to open. */
	FOnBeforePopupDelegate BeforePopupDelegate;

	/** Delegate for notifying that the browser is about to load a resource. */
	FOnBeforeResourceLoadDelegate BeforeResourceLoadDelegate;

	/** Delegate that allows for responses to resource loads */
	FOnResourceLoadCompleteDelegate ResourceLoadCompleteDelegate;

	/** Delegate that allows for response to console logs.  Typically used to capture and mirror web logs in client application logs. */
	FOnConsoleMessageDelegate ConsoleMessageDelegate;

	/** Delegate for handling requests to create new windows. */
	FOnCreateWindow CreateWindowDelegate;

	/** Delegate for handling requests to close new windows that were created. */
	FOnCloseWindow CloseWindowDelegate;

	/** Delegate for handling requests to close from the webview floating close button. */
	FOnFloatingCloseButtonPressed FloatingCloseButtonPressedDelegate;

	/** Delegate for handling requests to show the popup menu. */
	FOnShowPopup ShowPopupEvent;

	/** Delegate for handling requests to dismiss the current popup menu. */
	FOnDismissPopup DismissPopupEvent;

	/** Delegate for showing dialogs. */
	FOnShowDialog ShowDialogDelegate;

	/** Delegate for dismissing all dialogs. */
	FOnDismissAllDialogs DismissAllDialogsDelegate;

	/** Delegate for suppressing context menu */
	FOnSuppressContextMenu SuppressContextMenuDelgate;
	
	/** Delegate for handling key down events not handled by the browser. */
	FOnUnhandledKeyDown UnhandledKeyDownDelegate;

	/** Delegate for handling key up events not handled by the browser. */
	FOnUnhandledKeyUp UnhandledKeyUpDelegate;

	/** Delegate for handling key char events not handled by the browser. */
	FOnUnhandledKeyChar UnhandledKeyCharDelegate;

	/** Delegate that is executed when a drag event is detected in an area of the web page tagged as a drag region. */
	FOnDragWindow DragWindowDelegate;

	/** Current state of the document being loaded. */
	EWebBrowserDocumentState DocumentState;
	int ErrorCode;

	FMobileJSScriptingPtr Scripting;

	mutable TOptional<TFunction<void (const FString&)>> GetPageSourceCallback;

	TSharedPtr<SWindow> ParentWindow;

	FIntPoint AndroidWindowSize;

	/** Tracks whether the widget is currently disabled or not*/
	bool bIsDisabled;

	/** Tracks whether the widget is currently visible or not*/
	bool bIsVisible;

	/** Used to detect when the widget is hidden*/
	bool bTickedLastFrame;
};

typedef FAndroidWebBrowserWindow FWebBrowserWindow;

#endif // USE_ANDROID_JNI
