// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppleCookieManager.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#if PLATFORM_IOS || PLATFORM_MAC
#import <Foundation/Foundation.h>
#include "Apple/AppleAsyncTask.h"

#if !PLATFORM_TVOS
#import "WebKit/WebKit.h"
#endif

NSUUID *FAppleCookieManager::CookieManagerIdentifier = [[NSUUID alloc] initWithUUIDString: @"0986df82-fc1b-4dd1-864d-e149a7e79119"];

FAppleCookieManager::FAppleCookieManager()
{
}

FAppleCookieManager::~FAppleCookieManager()
{
}

void FAppleCookieManager::SetCookie(const FString& URL, const FCookie& Cookie, TFunction<void(bool)> Completed)
{
	// not implemented
	if (Completed)
	{
		Completed(false);
	}
}

void FAppleCookieManager::DeleteCookies(const FString& URL, const FString& CookieName, TFunction<void(int)> Completed)
{
#if !PLATFORM_TVOS
	FString CapturedURL = URL;
	UE_LOGF(LogTemp, Warning, "Deleting cookies for %ls", *URL);
	
	dispatch_async(dispatch_get_main_queue(), ^
	{
		// Delete matching cookies
		WKHTTPCookieStore* Storage = [WKWebsiteDataStore dataStoreForIdentifier: CookieManagerIdentifier].httpCookieStore;
		bool CapturedURLEmpty = CapturedURL.IsEmpty();
		
		[Storage getAllCookies: ^(NSArray<NSHTTPCookie*>* Cookies) 
		{
			for (NSHTTPCookie* Cookie in Cookies)
			{
				FString Domain([Cookie domain]);
				FString Path([Cookie path]);
				FString CookieUrl = Domain + Path;
				FString URLTest = CapturedURL;
				
				if (CookieUrl.Contains(CapturedURL) || CapturedURLEmpty)
				{
					[Storage deleteCookie:Cookie completionHandler:nil];
				}
			}
			
			// Notify on the game thread
			[FAppleAsyncTask CreateTaskWithBlock:^bool(void)
			 {
				 if (Completed)
				 {
					 Completed(true);
				 }
				 return true;
			 }];
		}];
		
	});
#endif
}

#endif
