// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if PLATFORM_IOS || PLATFORM_MAC
#include "IWebBrowserCookieManager.h"

@class NSUUID;
/**
 * Implementation of interface for dealing with a Web Browser cookies for iOS.
 */
class FAppleCookieManager
	: public IWebBrowserCookieManager
	, public TSharedFromThis<FAppleCookieManager>
{
public:
	// This is a hack for now until I figure out how to persist cookie managers' UUIDs
	static NSUUID *CookieManagerIdentifier;
	
	// IWebBrowserCookieManager interface

	virtual void SetCookie(const FString& URL, const FCookie& Cookie, TFunction<void(bool)> Completed = nullptr) override;
	virtual void DeleteCookies(const FString& URL = TEXT(""), const FString& CookieName = TEXT(""), TFunction<void(int)> Completed = nullptr) override;

	// FAppleCookieManager

	FAppleCookieManager();
	virtual ~FAppleCookieManager();
};
#endif
