// Copyright Epic Games, Inc. All Rights Reserved.

#include "ApplePlatformWebBrowser.h"

#if !WITH_CEF3 && (PLATFORM_IOS || PLATFORM_MAC)

#if PLATFORM_IOS
#include "IOS/IOSView.h"
#include "IOS/IOSAppDelegate.h"
#else
#include "Mac/CocoaWindow.h"
#include "Mac/CocoaThread.h"
#include "Mac/MacWindow.h"
#include "Mac/MacApplication.h"
#endif

#include "Async/Async.h"
#include "Widgets/SLeafWidget.h"
#include "PlatformHttp.h"
#include "HAL/PlatformProcess.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "Rendering/SlateRenderer.h"
#include "Textures/SlateUpdatableTexture.h"
#include "Widgets/Docking/SDockTab.h"

#import <MetalKit/MetalKit.h>
#include "ExternalTexture.h"
#include "WebBrowserModule.h"
#include "WebViewCloseButton.h"
#include "IWebBrowserSingleton.h"

#include "Apple/AppleAsyncTask.h"
#include "AppleCookieManager.h"
#include "WebView3DInputHandler.h"

#include "Apple/AppleStringUtils.h"

DEFINE_LOG_CATEGORY(LogAppleWebBrowser);

#if !UE_BUILD_SHIPPING
TAutoConsoleVariable<bool> CVarWebViewEnableInspectable(
	TEXT("webview.EnableInspectable"),
	false,
	TEXT("Allows webview to be debuggable through Safari Web Inspector."),
	ECVF_Default
);
#endif

class SAppleWebBrowserWidget : public SViewport
{
	SLATE_BEGIN_ARGS(SAppleWebBrowserWidget)
		: _InitialURL("about:blank")
		, _UseTransparency(false)
		, _AllowScreenshots(true)
	{ }

	SLATE_ARGUMENT(FString, InitialURL);
	SLATE_ARGUMENT(bool, UseTransparency);
	SLATE_ARGUMENT(bool, AllowScreenshots);
	SLATE_ARGUMENT(TSharedPtr<FWebBrowserWindow>, WebBrowserWindow);
	SLATE_ARGUMENT(FString, UserAgentApplication);

	SLATE_END_ARGS()

	SAppleWebBrowserWidget()
		: WebViewWrapper(nil)
	{}

	void Construct(const FArguments& Args)
	{
		WebBrowserWindowPtr = Args._WebBrowserWindow;
		Is3DBrowser = false;
		bAcquiredInitialFocus = false;
		bFocused = false;

		bool bEnableFloatingCloseButton = false;
		GConfig->GetBool(TEXT("Browser"), TEXT("bEnableFloatingCloseButton"), bEnableFloatingCloseButton, GEngineIni);

		WebViewWrapper = [AppleWebViewWrapper alloc];
		[WebViewWrapper create : this userAgentApplication : Args._UserAgentApplication.GetNSString() useTransparency : Args._UseTransparency  enableFloatingCloseButton : bEnableFloatingCloseButton];

		TSharedRef<SAppleWebBrowserWidget> SharedThis = 
			StaticCastSharedPtr<SAppleWebBrowserWidget>(AsShared().ToSharedPtr()).ToSharedRef();
		
#if !PLATFORM_TVOS
		OnSafeFrameChangedEventHandle = FCoreDelegates::OnSafeFrameChangedEvent.AddLambda( [this]()
										{
											if (WebViewWrapper != nil)
											{
												[WebViewWrapper didRotate];
											}
										});


		TextureSamplePool = new FWebBrowserTextureSamplePool();
		WebBrowserTextureSamplesQueue = MakeShared<FWebBrowserTextureSampleQueue, ESPMode::ThreadSafe>();
		WebBrowserTexture = nullptr;
		WebBrowserMaterial = nullptr;
		WebBrowserBrush = nullptr;
		
		[WebViewWrapper setUsingIndirectRendering: false];
		[WebViewWrapper setAllowingScreenshots: Args._AllowScreenshots];

		// create external texture
		WebBrowserTexture = NewObject<UWebBrowserTexture>((UObject*)GetTransientPackage(), NAME_None, RF_Transient | RF_Public);

		if (WebBrowserTexture.IsValid())
		{
			WebBrowserTexture->UpdateResource();
			WebBrowserTexture->AddToRoot();
		}

		// create wrapper material
		IWebBrowserSingleton* WebBrowserSingleton = IWebBrowserModule::Get().GetSingleton();

		UMaterialInterface* DefaultWBMaterial = Args._UseTransparency ? WebBrowserSingleton->GetDefaultTranslucentMaterial() : WebBrowserSingleton->GetDefaultMaterial();
		if (WebBrowserSingleton && DefaultWBMaterial)
		{
			// create wrapper material
			WebBrowserMaterial = UMaterialInstanceDynamic::Create(DefaultWBMaterial, nullptr);

			if (WebBrowserMaterial.IsValid())
			{
				WebBrowserMaterial->SetTextureParameterValue("SlateUI", WebBrowserTexture.Get());
				WebBrowserMaterial->AddToRoot();

				// create Slate brush
				WebBrowserBrush = MakeShareable(new FSlateBrush());
				{
					WebBrowserBrush->SetResourceObject(WebBrowserMaterial.Get());
				}
			}
		}
		
		if([WebViewWrapper isAllowingScreenshots] && !WebBrowserBrush.IsValid())
		{
			UE_LOGF(LogTemp, Warning, "SAppleWebBrowserWidget is allowing screenshots but the requisite materials were not found!");
		}
		else if([WebViewWrapper isAllowingScreenshots])
		{
			FSlateApplication::Get().OnPreScreenshot().AddSP(SharedThis, &SAppleWebBrowserWidget::OnTakingScreenshot);
		}
#endif
		
		FCoreDelegates::OnEnginePreExit.AddSP(SharedThis, &SAppleWebBrowserWidget::OnEnginePreExit);
		
		LoadURL(Args._InitialURL);
	}
	
	bool UsingIndirectRendering() const
	{
		return Is3DBrowser || (WebViewWrapper != nil && [WebViewWrapper isUsingIndirectRendering]);
	}
	
	void OnTakingScreenshot(TSharedRef<SWidget> Widget)
	{
		if(![WebViewWrapper isAllowingScreenshots] || (!CurrentWidgetPath.IsValid() && 
		   !FSlateApplication::Get().GeneratePathToWidgetUnchecked(AsShared(), CurrentWidgetPath)))
		{
			return;
		}
		
		bool bIsVisible = !WebBrowserWindowPtr.IsValid() || WebBrowserWindowPtr.Pin()->IsVisible();
	
		if(CurrentWidgetPath.ContainsWidget(&Widget.Get()) && WebViewWrapper != nil &&
		   !Is3DBrowser && bIsVisible)
		{
			// we are a descendent from this widget, therefore it is taking 
			// a screenshot of us and we must switch to a Slate-visible view
			[WebViewWrapper setUsingIndirectRendering: true];
		}
		
		bTakingScreenshot = true;
	}
	
	void OnEnginePreExit()
	{
		if(WebViewWrapper != nil)
		{
			[WebViewWrapper disableCallbacksOnShutdown];
		}
	}
	
	void ScreenshotFinished()
	{
		if(WebViewWrapper != nil && [WebViewWrapper isUsingIndirectRendering])
		{
			check([WebViewWrapper isAllowingScreenshots]);
			
			bForceRecomputeWindowOnPaint = true;
			bTakingScreenshot = false;
			
			if(!Is3DBrowser)
			{
				[WebViewWrapper setUsingIndirectRendering: false];
			}
		}
	}

	void UpdateFrameWithViewport(FIntPoint WindowSize, FIntPoint WindowPos) {
		CGRect NewFrame;
#if PLATFORM_MAC
		TSharedPtr<FWebBrowserWindow> BrowserWindow = WebBrowserWindowPtr.Pin();
		
		if(!BrowserWindow.IsValid() || !BrowserWindow->GetParentWindow().IsValid() ||
		   !BrowserWindow->GetParentWindow()->GetNativeWindow().IsValid())
		{
			return;
		}
		
		TSharedPtr<FGenericWindow> EnclosedWindow = WebBrowserWindowPtr.Pin()->GetParentWindow()->GetNativeWindow();
		
		if(!Is3DBrowser) 
		{
			NSWindow* ParentWindow = (NSWindow*)EnclosedWindow->GetOSWindowHandle();
			float ScaleFactor = [ParentWindow backingScaleFactor];
			const FVector2D CocoaPosition = FMacApplication::ConvertSlatePositionToCocoa(WindowPos.X, WindowPos.Y);
			
			WindowSize.X /= ScaleFactor;
			WindowSize.Y /= ScaleFactor;
			
			NSRect ParentFrame = [ParentWindow frame];
			NSRect Rect = NSMakeRect(CocoaPosition.X - ParentFrame.origin.x, (CocoaPosition.Y - ParentFrame.origin.y) - WindowSize.Y, 
									 FMath::Max(WindowSize.X, 1), FMath::Max(WindowSize.Y, 1));
			Rect = [ParentWindow frameRectForContentRect : Rect];
			
			// CGRect and NSRect have the same memory layout
			NewFrame = NSRectToCGRect(Rect);
			
			if(PastEnclosedWindow != EnclosedWindow) 
			{
				WindowFrame = NewFrame;
				PastEnclosedWindow = EnclosedWindow;
				LeaveDeadWindow();
				
				// Need to also replace the widget path as otherwise
				// screenshots will break when moving windows
				if(CurrentWidgetPath.IsValid())
				{
					CurrentWidgetPath = FWidgetPath{};
				}
			}
		}
#else
		{
			UIView* View = [IOSAppDelegate GetDelegate].IOSView;
			CGFloat ContentScaleFactor = View.contentScaleFactor;
			
			NewFrame.size.width = FMath::RoundToInt(WindowSize.X / ContentScaleFactor);
			NewFrame.size.height = FMath::RoundToInt(WindowSize.Y / ContentScaleFactor);
			
			NewFrame.origin.x = FMath::RoundToInt(WindowPos.X / ContentScaleFactor);
			NewFrame.origin.y = FMath::RoundToInt(WindowPos.Y / ContentScaleFactor);
		}
#endif

		if(!UsingIndirectRendering() && 
		   (bForceRecomputeWindowOnPaint || NewFrame.size.width != WindowFrame.size.width || 
			NewFrame.size.height != WindowFrame.size.height)) 
		{
			// WindowFrame is only used when we are a 2D web browser
			WindowFrame = NewFrame;
			bForceRecomputeWindowOnPaint = false;
		}
		
#if PLATFORM_IOS
		[WebViewWrapper updateframe : WindowFrame window : nil];
#else			
		[WebViewWrapper updateframe : WindowFrame window : EnclosedWindow.Get()];
#endif
	}
	
	static void CreateWebBrowserTextureIfNeeded(FRHICommandListImmediate& RHICmdList, 
												AppleWebViewWrapper* NativeWebBrowser, FIntPoint Size)
	{
		check(IsInRenderingThread());
		
		FTextureRHIRef VideoTexture = [NativeWebBrowser GetVideoTexture];
		bool bStaleVideoTexture = VideoTexture != nullptr && VideoTexture->GetSizeXY() != Size;
									
		if (VideoTexture == nullptr || bStaleVideoTexture)
		{
			// We need this texture to be accessible and writable from the CPU because we need to call
			// replaceRegion:mipmapLevel:withBytes:bytesPerRow: to copy over a screenshot captured in WKWebView
			// using takeSnapshotWithConfiguration:completionHandler: which gives us an UI/NSImage.
			const FRHITextureCreateDesc Desc =
				FRHITextureCreateDesc::Create2D(TEXT("SAppleWebBrowserWidget_VideoTexture"), Size, PF_R8G8B8A8)
				.SetFlags(ETextureCreateFlags::CPUWritable | ETextureCreateFlags::CPUReadback);

			VideoTexture = RHICmdList.CreateTexture(Desc);
			[NativeWebBrowser SetVideoTexture : VideoTexture];

			if (VideoTexture == nullptr)
			{
				UE_LOGF(LogAppleWebBrowser, Warning, "CreateTexture failed!");
				return;
			}

			[NativeWebBrowser SetVideoTextureValid : false];
		}
	}
	
	void WriteWebBrowserTexture(AppleWebViewWrapper* NativeWebBrowserPtr,
								FGuid PlayerGuid, FIntPoint Size)
	{
		// The FlushRenderingCommands for screenshot mode are needed to make
		// taking a screenshot a sync op so we don't need screenshot info from the previous frame.
		if(bTakingScreenshot)
		{
			ENQUEUE_RENDER_COMMAND(CreateWebBrowserTexture)([NativeWebBrowserPtr, PlayerGuid, Size](FRHICommandListImmediate& RHICmdList) 
															{
				CreateWebBrowserTextureIfNeeded(RHICmdList, NativeWebBrowserPtr, Size);
			});
			FlushRenderingCommands();
		}
		
		ENQUEUE_RENDER_COMMAND(WriteWebBrowser)(
			[NativeWebBrowserPtr, PlayerGuid, Size](FRHICommandListImmediate& RHICmdList)
			{
				AppleWebViewWrapper* NativeWebBrowser = NativeWebBrowserPtr;

				if (NativeWebBrowser == nil)
				{
					return;
				}

				CreateWebBrowserTextureIfNeeded(RHICmdList, NativeWebBrowserPtr, Size);
				FTextureRHIRef VideoTexture = [NativeWebBrowser GetVideoTexture];
				
				if(VideoTexture == nil)
				{
					return;
				}
				
				if ([NativeWebBrowser UpdateVideoFrame : VideoTexture->GetNativeResource()])
				{
					// if region changed, need to reregister UV scale/offset
					//UE_LOGF(LogAppleWebBrowser, Log, "UpdateVideoFrame RT: %ls", *Params.PlayerGuid.ToString());
				}

				if (![NativeWebBrowser IsVideoTextureValid])
				{
					FSamplerStateInitializerRHI SamplerStateInitializer(SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp);
					FSamplerStateRHIRef SamplerStateRHI = RHICreateSamplerState(SamplerStateInitializer);
					FExternalTextureRegistry::Get().RegisterExternalTexture(PlayerGuid, VideoTexture, SamplerStateRHI);

					[NativeWebBrowser SetVideoTextureValid : true];
				}
			});
		
		if(bTakingScreenshot)
		{
			FlushRenderingCommands();
		}
	}
	
	void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
	{
		if (WebViewWrapper != nil)
		{
			if (WebBrowserWindowPtr.IsValid())
			{
				TSharedPtr<FWebBrowserWindow> WebBrowserWindow = WebBrowserWindowPtr.Pin();
				WebBrowserWindow->SetTickLastFrame();
				if (TSharedPtr<SWindow> ParentWindow = WebBrowserWindow->GetParentWindow())
				{
					bool ShouldSet3DBrowser = ParentWindow->IsVirtualWindow();
					if (Is3DBrowser != ShouldSet3DBrowser)
					{
						Is3DBrowser = ShouldSet3DBrowser;
						[WebViewWrapper set3D : Is3DBrowser];
					}
				}
			}
			
			// If we are a 3D browser, we should let the Widget Interaction component fully control focus
			// Otherwise, to keep parity with CEF, we should initially gain focus on the WebBrowser widget.
			if(!bAcquiredInitialFocus && !Is3DBrowser && InKeyWindow()) 
			{
				FSlateApplication::Get().SetKeyboardFocus(SharedThis(this), EFocusCause::SetDirectly);
				FSlateApplication::Get().ReleaseAllPointerCapture();
				bAcquiredInitialFocus = true;
			} 
			
			FVector2D Position = AllottedGeometry.GetAccumulatedRenderTransform().GetTranslation();
			FVector2D Size = TransformVector(AllottedGeometry.GetAccumulatedRenderTransform(), AllottedGeometry.GetLocalSize());
			FSlateRect SlateRect = AllottedGeometry.GetRenderBoundingRect();
			
			UpdateFrameWithViewport(FIntPoint(static_cast<int32_t>(Size.X), static_cast<int32_t>(Size.Y)), 
									FIntPoint(static_cast<int32_t>(Position.X), static_cast<int32_t>(Position.Y)));
#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
			{
				TSharedPtr<FWebBrowserWindow> WebBrowserWindow = WebBrowserWindowPtr.Pin();
				if(WebBrowserWindow.IsValid())
				{
					TSharedPtr<SWindow> ParentWindow = WebBrowserWindow->GetParentWindow();
					
					if([WebViewWrapper isReadyToAddToNativeLayer] && LastSlateLayer.IsSet() && ParentWindow.IsValid())
					{
						int32 LayerId = LastSlateLayer.GetValue();
						TSharedPtr<SWindow> PastNativeLayerWindow = LastNativeLayerCreatedInWindow.Pin();
						bool bWasLayerAlreadyCreated = LastNativeLayerCreated.IsSet() && LastNativeLayerCreated.GetValue() == LayerId && 
							PastNativeLayerWindow.IsValid() && PastNativeLayerWindow == ParentWindow;
						
						if(!bWasLayerAlreadyCreated)
						{
							UE_LOGF(LogAppleWebBrowser, Display, "The native layer was not created for SAppleWebBrowserWidget %p on layer %d, creating...", this, LayerId);

							if(LastNativeLayerCreated.IsSet() && PastNativeLayerWindow.IsValid())
							{
								FSlateApplication::Get().GetRenderer()->DeleteNativeLayer(LastNativeLayerCreated.GetValue(), *PastNativeLayerWindow);
							}
							
							FSlateApplication::Get().GetRenderer()->CreateNativeLayer(LayerId, *ParentWindow, WebViewWrapper.WebViewContainer);
							LastNativeLayerCreated = LayerId;
							LastNativeLayerCreatedInWindow = ParentWindow;
						}
					}
				}
			}
#endif
			
#if !PLATFORM_TVOS
			if ([WebViewWrapper isAllowingScreenshots] || UsingIndirectRendering())
			{
				if(!bFirstScreenshotTaken && !Is3DBrowser)
				{
					// If we are allowing screenshots in 2D mode, we want to take one
					// screenshot at first to make sure that everything gets properly
					// initialized and the indirect texture set up properly.
					[WebViewWrapper setUsingIndirectRendering: true];
					bTakingScreenshot = true;
					bFirstScreenshotTaken = true;
				}
				else if(bFirstScreenshotTaken && !Is3DBrowser && !bTakingScreenshot)
				{
					// we shouldn't be switching to the expensive screenshot mode when we don't need to!
					return;
				}
				
				if (WebBrowserTexture.IsValid())
				{					
					TSharedPtr<FWebBrowserTextureSample, ESPMode::ThreadSafe> WebBrowserTextureSample;
					WebBrowserTextureSamplesQueue->Peek(WebBrowserTextureSample);

					WebBrowserTexture->TickResource(WebBrowserTextureSample);
				}

				if (WebBrowserTexture.IsValid())
				{
					TSharedPtr<FWebBrowserWindow> WebBrowserWindow = WebBrowserWindowPtr.Pin();
					bool TextureWritten = false;
					
					FIntPoint ViewportSize{WindowFrame.size.width, WindowFrame.size.height};
					
					if(Is3DBrowser)
					{
						// GetViewportSize not 100% accurate in 3D mode, we use it as a 
						// reasonable default if we need to only
						ViewportSize = WebBrowserWindow->GetViewportSize();
						
						if(TSharedPtr<SWindow> ParentWindow = WebBrowserWindow->GetParentWindow())
						{
							FVector2f WindowSize = ParentWindow->GetSizeInScreen();
							ViewportSize.X = static_cast<int32>(WindowSize.X);
							ViewportSize.Y = static_cast<int32>(WindowSize.Y);
#if !PLATFORM_MAC
							// needs to be on main thread for getScreenScale
							dispatch_async(dispatch_get_main_queue(), ^ 
							{
								FIntPoint ScaledViewportSize = ViewportSize;
								CGFloat ScreenScale = [WebViewWrapper getScreenScale];
								ScaledViewportSize.X *= ScreenScale;
								ScaledViewportSize.Y *= ScreenScale;
								
								WriteWebBrowserTexture(WebViewWrapper, WebBrowserTexture->GetExternalTextureGuid(), ScaledViewportSize);
							});
							TextureWritten = true;
#endif
						}
					}

					if(!TextureWritten) 
					{
						WriteWebBrowserTexture(WebViewWrapper, WebBrowserTexture->GetExternalTextureGuid(), ViewportSize);
					}
				}
			}
#endif
		}
	}

	int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
	{
#if !PLATFORM_TVOS
		bool bIsVisible = !WebBrowserWindowPtr.IsValid() || WebBrowserWindowPtr.Pin()->IsVisible();
		
		if (bIsVisible && UsingIndirectRendering() && WebBrowserBrush.IsValid())
		{
			FSlateDrawElement::MakeBox(OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(), WebBrowserBrush.Get(), ESlateDrawEffect::None);
		}
#endif

		LastSlateLayer = LayerId;

		return LayerId;
	}

	virtual FVector2D ComputeDesiredSize(float) const override
	{
		return FVector2D(640, 480);
	}

	void LoadURL(const FString& InNewURL)
	{
		if (WebViewWrapper != nil)
		{
			[WebViewWrapper loadurl : [NSURL URLWithString : [NSString stringWithUTF8String : TCHAR_TO_UTF8(*InNewURL)]]];
		}
	}

	void LoadString(const FString& InContents, const FString& InDummyURL)
	{
		if (WebViewWrapper != nil)
		{
			[WebViewWrapper loadstring : [NSString stringWithUTF8String : TCHAR_TO_UTF8(*InContents)] dummyurl : [NSURL URLWithString : [NSString stringWithUTF8String : TCHAR_TO_UTF8(*InDummyURL)]]];
		}
	}
	
	void StopLoad()
	{
		if (WebViewWrapper != nil)
		{
			[WebViewWrapper stopLoading];
		}
	}

	void Reload()
	{
		if (WebViewWrapper != nil)
		{
			[WebViewWrapper reload];
		}
	}

	void Close()
	{
		if(LastNativeLayerCreated.IsSet())
		{
			TSharedPtr<SWindow> NativeLayerWindow = LastNativeLayerCreatedInWindow.Pin();
			
			if(NativeLayerWindow.IsValid())
			{
				FSlateApplication::Get().GetRenderer()->DeleteNativeLayer(LastNativeLayerCreated.GetValue(), 
																		  *NativeLayerWindow);
			}
			
			LastNativeLayerCreated.Reset();
		}
		
#if !PLATFORM_TVOS
		FCoreDelegates::OnSafeFrameChangedEvent.Remove(OnSafeFrameChangedEventHandle);
#endif
		if (WebViewWrapper != nil)
		{
			[WebViewWrapper close];
			[WebViewWrapper release];
			WebViewWrapper = nil;
		}
		WebBrowserWindowPtr.Reset();

#if !PLATFORM_TVOS
		delete TextureSamplePool;
		TextureSamplePool = nullptr;
		
		WebBrowserTextureSamplesQueue->RequestFlush();
		if (WebBrowserBrush.IsValid())
		{
			WebBrowserBrush->SetResourceObject(nullptr);	
		}
		if (WebBrowserMaterial.IsValid())
		{
			WebBrowserMaterial->RemoveFromRoot();
			WebBrowserMaterial = nullptr;
		}
		if (WebBrowserTexture.IsValid())
		{
			WebBrowserTexture->RemoveFromRoot();
			WebBrowserTexture = nullptr;
		}
#endif
	}

	void ShowFloatingCloseButton(bool bShow, bool bDraggable)
	{
		if (WebViewWrapper != nil)
		{
			[WebViewWrapper showFloatingCloseButton: bShow setDraggable: bDraggable];
		}
	}
	
	void GoBack()
	{
		if (WebViewWrapper != nil)
		{
			[WebViewWrapper goBack];
		}
	}

	void GoForward()
	{
		if (WebViewWrapper != nil)
		{
			[WebViewWrapper goForward];
		}
	}


	bool CanGoBack()
	{
		if (WebViewWrapper != nil)
		{
			return [WebViewWrapper canGoBack];
		}
		return false;
	}

	bool CanGoForward()
	{
		if (WebViewWrapper != nil)
		{
			return [WebViewWrapper canGoForward];
		}
		return false;
	}

	void SetWebBrowserVisibility(bool InIsVisible)
	{
		if (WebViewWrapper != nil)
		{
			UE_LOGF(LogAppleWebBrowser, Warning, "SetWebBrowserVisibility %d!", InIsVisible);

			[WebViewWrapper setVisibility : InIsVisible];
		}
	}

	bool HandleOnBeforePopup(const FString& UrlStr, const FString& FrameName)
	{
		TSharedPtr<FWebBrowserWindow> BrowserWindow = WebBrowserWindowPtr.Pin();
		if (BrowserWindow.IsValid() && BrowserWindow->OnBeforePopup().IsBound())
		{
			return BrowserWindow->OnBeforePopup().Execute(UrlStr, FrameName);
		}
		return false;
	}

	bool HandleShouldOverrideUrlLoading(const FString& Url)
	{
		// UE-347936 Abnormal requests in EDA webpage, ignore these
		if (WebBrowserWindowPtr.IsValid() && !Url.Equals(TEXT("about:blank")) && !Url.Equals(TEXT("about:srcdoc")))
		{
			// Capture vars needed for AsyncTask
			NSString* UrlString = [NSString stringWithUTF8String : TCHAR_TO_UTF8(*Url)];
			TWeakPtr<FWebBrowserWindow> AsyncWebBrowserWindowPtr = WebBrowserWindowPtr;

			// Notify on the game thread
			[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
			{
				TSharedPtr<FWebBrowserWindow> BrowserWindow = AsyncWebBrowserWindowPtr.Pin();
				if (BrowserWindow.IsValid())
				{
					if (BrowserWindow->OnBeforeBrowse().IsBound())
					{
						FWebNavigationRequest RequestDetails;
						RequestDetails.bIsRedirect = false;
						RequestDetails.bIsMainFrame = true; // shouldOverrideUrlLoading is only called on the main frame

						BrowserWindow->OnBeforeBrowse().Execute(UrlString, RequestDetails);
						BrowserWindow->SetTitle("");
					}
				}
				return true;
			}];
		}
		return true;
	}

	void HandleReceivedTitle(const FString& Title)
	{
		if (WebBrowserWindowPtr.IsValid())
		{
			TSharedPtr<FWebBrowserWindow> BrowserWindow = WebBrowserWindowPtr.Pin();
			if (BrowserWindow.IsValid() && !BrowserWindow->GetTitle().Equals(Title))
			{
				BrowserWindow->SetTitle(Title);
			}
		}
	}

	void ProcessScriptMessage(const FString& InMessage)
	{
		if (WebBrowserWindowPtr.IsValid())
		{
			FString Message = InMessage;
			TWeakPtr<FWebBrowserWindow> AsyncWebBrowserWindowPtr = WebBrowserWindowPtr;

			[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
			{
				TSharedPtr<FWebBrowserWindow> BrowserWindow = AsyncWebBrowserWindowPtr.Pin();
				if (BrowserWindow.IsValid())
				{
					TArray<FString> Params;
					Message.ParseIntoArray(Params, TEXT("/"), false);
					if (Params.Num() > 0)
					{
						for (int I = 0; I < Params.Num(); I++)
						{
							Params[I] = FPlatformHttp::UrlDecode(Params[I]);
						}

						FString Command = Params[0];
						Params.RemoveAt(0, 1);
						BrowserWindow->OnJsMessageReceived(Command, Params, "");
					}
					else
					{
						GLog->Logf(ELogVerbosity::Error, TEXT("Invalid message from browser view: %s"), *Message);
					}
				}
				return true;
			}];
		}
	}

	void HandlePageLoad(const FString& InCurrentUrl, bool bIsLoading)
	{
		if (WebBrowserWindowPtr.IsValid())
		{
			TSharedPtr<FWebBrowserWindow> BrowserWindow = WebBrowserWindowPtr.Pin();
			if (BrowserWindow.IsValid())
			{
				BrowserWindow->NotifyDocumentLoadingStateChange(InCurrentUrl, bIsLoading);
			}
		}
	}

	void HandleReceivedError(int ErrorCode, const FString& InCurrentUrl)
	{
		if (WebBrowserWindowPtr.IsValid())
		{
			TSharedPtr<FWebBrowserWindow> BrowserWindow = WebBrowserWindowPtr.Pin();
			if (BrowserWindow.IsValid())
			{
				BrowserWindow->NotifyDocumentError(InCurrentUrl, ErrorCode);
			}
		}
	}

	void ExecuteJavascript(const FString& Script)
	{
		if (WebViewWrapper != nil)
		{
			[WebViewWrapper executejavascript : [NSString stringWithUTF8String : TCHAR_TO_UTF8(*Script)]];
		}
	}

	void FloatingCloseButtonPressed()
	{
		if (WebBrowserWindowPtr.IsValid())
		{
			TWeakPtr<FWebBrowserWindow> AsyncWebBrowserWindowPtr = WebBrowserWindowPtr;
			
#if !PLATFORM_MAC
			[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
			{
				if (TSharedPtr<FWebBrowserWindow> BrowserWindow = AsyncWebBrowserWindowPtr.Pin())
				{
					BrowserWindow->FloatingCloseButtonPressed();
				}
				return true;
			}];
#endif
		}
	}
	
	void LeaveDeadWindow() 
	{
		if(WebViewWrapper != nil) 
		{
			[WebViewWrapper leaveDeadWindow];
		}
	}
	
	void OnFocus(bool SetFocus)
	{
		if(SetFocus && !Is3D()) 
		{
			if(IsInGameThread())
			{
				FSlateApplication::Get().ReleaseAllPointerCapture();
			}
			else
			{
				FGraphEventRef PointerReleaseEvent = FFunctionGraphTask::CreateAndDispatchWhenReady([]() {
					FSlateApplication::Get().ReleaseAllPointerCapture();
				}, TStatId(), nullptr, ENamedThreads::GameThread);
				FTaskGraphInterface::Get().WaitUntilTaskCompletes(PointerReleaseEvent);
			}
		}
		else if(!FSlateApplication::Get().IsActive() && bFocused) 
		{
			// we changed windows but we still want to retain focus and prevent
			// mouse capture since we were the last widget focused.
			bAcquiredInitialFocus = false;
		}
		
		bFocused = SetFocus;
	}
	
	bool Is3D() const
	{
		return Is3DBrowser;
	}
	
	bool InKeyWindow()
	{
		return WebViewWrapper != nil && [WebViewWrapper inKeyWindow];
	}
	
	~SAppleWebBrowserWidget()
	{
		Close();
	}

protected:
	mutable __strong AppleWebViewWrapper* WebViewWrapper;
private:
	TWeakPtr<FGenericWindow> PastEnclosedWindow;
	TWeakPtr<FWebBrowserWindow> WebBrowserWindowPtr;

	/** Enable 3D appearance */
	bool Is3DBrowser;
	bool bAcquiredInitialFocus;
	bool bFocused;
	bool bForceRecomputeWindowOnPaint = false;
	bool bTakingScreenshot = false;
	bool bFirstScreenshotTaken = false;
	
	TOptional<int32> LastNativeLayerCreated;
	TWeakPtr<SWindow> LastNativeLayerCreatedInWindow;
	
	mutable TOptional<int32> LastSlateLayer;

#if !PLATFORM_TVOS
	/** The external texture to render the webbrowser output. */
	TWeakObjectPtr<UWebBrowserTexture> WebBrowserTexture;

	/** The material for the external texture. */
	TWeakObjectPtr<UMaterialInstanceDynamic> WebBrowserMaterial;

	/** The Slate brush that renders the material. */
	TSharedPtr<FSlateBrush> WebBrowserBrush;

	/** The sample queue. */
	TSharedPtr<FWebBrowserTextureSampleQueue, ESPMode::ThreadSafe> WebBrowserTextureSamplesQueue;

	/** Texture sample object pool. */
	FWebBrowserTextureSamplePool* TextureSamplePool;

	// Handle to detect changes in valid area to move the floating close button */
	FDelegateHandle OnSafeFrameChangedEventHandle;
#endif
	
	FWidgetPath CurrentWidgetPath{};
	
	CGRect WindowFrame;
};




@implementation AppleWebViewWrapper

#if !PLATFORM_TVOS
@synthesize WebView;
#if PLATFORM_IOS
@synthesize CloseButton;
#endif
@synthesize WebViewContainer;
@synthesize UserContentController;
@synthesize LastWebViewScreenshot;
#endif
@synthesize NextURL;
@synthesize NextContent;
#if PLATFORM_MAC
@synthesize GameWindowView;
#endif

-(void)create:(SAppleWebBrowserWidget*)InWebBrowserWidget userAgentApplication: (NSString*)UserAgentApplication useTransparency : (bool)InUseTransparency enableFloatingCloseButton : (bool)bEnableFloatingCloseButton;
{
	WebBrowserWidget = InWebBrowserWidget;
	NextURL = nil;
	NextContent = nil;
	VideoTexture = nil;
	bNeedsAddToView = true;
	Is3DBrowser = false;
	bVideoTextureValid = false;
	bShuttingDown = false;
	bReadyToAddToNativeLayer = false;
	
#if !UE_BUILD_SHIPPING
	BOOL Inspectable = CVarWebViewEnableInspectable.GetValueOnAnyThread()? YES : NO;
#endif
#if PLATFORM_MAC
	GameWindowView = nil;
#endif

#if !PLATFORM_TVOS
	dispatch_async(dispatch_get_main_queue(), ^
	{
		#if PLATFORM_IOS
		self.WebViewContainer = [[UIView alloc]initWithFrame:CGRectMake(1, 1, 100, 100)];
		[self.WebViewContainer setOpaque : NO];
		[self.WebViewContainer setBackgroundColor : [UIColor clearColor]];
		#else
		self.WebViewContainer = [[NSView alloc]initWithFrame:CGRectMake(1, 1, 100, 100)];
		#endif

		WKWebViewConfiguration* theConfiguration = [[WKWebViewConfiguration alloc] init];

		self.UserContentController = [[WKUserContentController alloc] init];
		
		theConfiguration.websiteDataStore = [WKWebsiteDataStore dataStoreForIdentifier: FAppleCookieManager::CookieManagerIdentifier];
		
		[theConfiguration.preferences setValue:@YES forKey:@"allowFileAccessFromFileURLs"];
		[theConfiguration.preferences setValue:@YES forKey:@"developerExtrasEnabled"];
	
		NSString* MessageHandlerName = [NSString stringWithFString : FMobileJSScripting::JSMessageHandler];
		[self.UserContentController addScriptMessageHandler:self name: MessageHandlerName];
		
		theConfiguration.applicationNameForUserAgent = UserAgentApplication;
		theConfiguration.userContentController = self.UserContentController;

		self.WebView = [[WKWebView alloc]initWithFrame:CGRectMake(1, 1, 100, 100)configuration : theConfiguration];
#if !UE_BUILD_SHIPPING
		self.WebView.inspectable = Inspectable;
#endif
		[self.WebViewContainer addSubview : WebView];
		self.WebView.navigationDelegate = self;
		self.WebView.UIDelegate = self;

#if PLATFORM_IOS
		self.WebView.scrollView.bounces = NO;
		
		if (InUseTransparency)
		{
			[self.WebView setOpaque : NO];
			[self.WebView setBackgroundColor : [UIColor clearColor]];
		}
		else
		{
			[self.WebView setOpaque : YES];
		}
		
		if (bEnableFloatingCloseButton)
		{
			CloseButton = MakeCloseButton();
			[self.WebView addSubview : CloseButton];

			CloseButton.TapHandler = [^
				{
				// WebBrowserWidget should have the same lifecycle as this
				// CloseButton (see [AppleWebViewWrapper cleanup], 
				// which is only called by SAppleWebBrowserWidget::Close()), 
				// so using this raw ptr should be safe. Previously, a
				// shared ptr was used, but the shared ptr was created in the
				// SAppleWebBrowserWidget's Construct function, 
				// which caused strange behavior with the TSharedRef SNew creates,
				// potentially leading to over-releasing.
				if(WebBrowserWidget != nil)
				{
					WebBrowserWidget->FloatingCloseButtonPressed();
				}
			} copy];
		}
#endif

		[theConfiguration release];
		[self setDefaultVisibility];
	});
#endif
}

-(void)didRotate;
{
#if !PLATFORM_TVOS && !PLATFORM_MAC
	dispatch_async(dispatch_get_main_queue(), ^ {
		if (CloseButton)
		{
			[CloseButton setupLayout];
		}
	});
#endif
}

-(void)cleanup;
{
#if !PLATFORM_TVOS
	if(self.UserContentController != nil) 
	{
		[self.UserContentController removeAllScriptMessageHandlers];
		[self.UserContentController removeAllUserScripts];
		
		[self.UserContentController release];
 
		self.UserContentController = nil;
	}
	
	if (self.WebView != nil)
	{		
#if PLATFORM_MAC
		if(self.WebView.window.firstResponder == self.WebView) 
		{
			[self.WebView.window makeFirstResponder: nil];
		}
#endif
		
		self.WebView.navigationDelegate = nil;
		self.WebView.UIDelegate = nil;
		
		[self.WebView removeFromSuperview];
		[self.WebViewContainer removeFromSuperview];
		
		[self.WebView release];
		[self.WebViewContainer release];
		
		self.WebView = nil;
		self.WebViewContainer = nil;
		WebBrowserWidget = nil;
	}

	if(LastWebViewScreenshot != nil)
	{
		[LastWebViewScreenshot release];
		LastWebViewScreenshot = nil;
	}
	
#if !PLATFORM_MAC
	if (CloseButton != nil)
	{
		[CloseButton release];
		CloseButton = nil;
	}
#endif
#endif
}

-(void)close;
{
	dispatch_async(dispatch_get_main_queue(), ^
	{
		[self cleanup];
	});
}

-(void)showFloatingCloseButton:(BOOL)bShow setDraggable:(BOOL)bDraggable;
{
#if !PLATFORM_TVOS && !PLATFORM_MAC
	dispatch_async(dispatch_get_main_queue(), ^
	{
		if (CloseButton)
		{
			[CloseButton showButton:bShow setDraggable: bDraggable];
		}
		else
		{
			UE_LOGF(LogAppleWebBrowser, Warning, "[PlatformWebBrowser]: Close button not enabled in config. Add config for Engine:[Browser]bEnableFloatingCloseButton=true");
		}
	});
#endif
}

-(void)dealloc;
{
	[self cleanup];
	[NextContent release];
	NextContent = nil;
	[NextURL release];
	NextURL = nil;

	[super dealloc];
}

-(void)leaveDeadWindow;
{
	bNeedsAddToView = true;
	bReadyToAddToNativeLayer = false;
}

-(void)updateframe:(CGRect)InFrame window:(FGenericWindow*)EnclosedWindow;
{
	self.DesiredFrame = InFrame;
	
#if !UE_BUILD_SHIPPING
	BOOL Inspectable = CVarWebViewEnableInspectable.GetValueOnAnyThread()? YES : NO;
#endif

#if !PLATFORM_TVOS
	dispatch_async(dispatch_get_main_queue(), ^
	{
		if (self.WebView != nil)
		{
			
#if !UE_BUILD_SHIPPING
			self.WebView.inspectable = Inspectable;
#endif
			
#if PLATFORM_MAC
			if(Is3DBrowser) 
			{
				if(bNeedsAddToView || !Current3DWindow.IsValid())
				{
					// we handle display of 3D WebViews but choose to still add it to the view hierarchy
					// as a completely transparent view for better integration 
					// (allows audio playback, some tick-like JS events dispatched for us, probably some other stuff)
					self.WebViewContainer.alphaValue = 0.0;
					bNeedsAddToView = false;
					
					AsyncTask(ENamedThreads::Type::GameThread, [self]() 
				    {
						if (const TSharedPtr<SWindow> ActiveWindow = FSlateApplication::Get().GetActiveTopLevelWindow())
						{
							FMacWindow* MacGameWindow = static_cast<FMacWindow*>(ActiveWindow->GetNativeWindow().Get());
							
							if(MacGameWindow != nullptr)
							{
								FCocoaWindow* GameCocoaWindow = MacGameWindow->GetWindowHandle();
								
								Current3DWindow = ActiveWindow;
								self.GameWindowView = GameCocoaWindow.openGLView;
							}
						}
						else
						{
							bNeedsAddToView = true;
						}
					});
				}
				else if(GameWindowView != nil)
				{
					[self.GameWindowView addSubview : WebViewContainer];
					self.GameWindowView = nil;
				}
			}
			else
			{
				self.WebViewContainer.frame = self.DesiredFrame;
				self.WebView.frame = WebViewContainer.bounds;
			}
#else
			if(![self isUsingIndirectRendering])
			{
				self.WebViewContainer.frame = self.DesiredFrame;
				self.WebView.frame = WebViewContainer.bounds;
			}
#endif
			if (bNeedsAddToView)
			{
#if PLATFORM_IOS 
				[self.WebViewContainer setOpaque : NO];
				[self.WebViewContainer setBackgroundColor : [UIColor clearColor]];
				[[IOSAppDelegate GetDelegate].IOSView addSubview : WebViewContainer];
				
				if (CloseButton != nil)
				{
					[CloseButton setupLayout];
				}
#elif !WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT				
				FCocoaWindow *CocoaWindow = ((FMacWindow*)EnclosedWindow)->GetWindowHandle();
				NSView *CocoaWindowView = [CocoaWindow contentView];
				
				[CocoaWindowView addSubview : self.WebViewContainer];
#endif
				bNeedsAddToView = false;
				bReadyToAddToNativeLayer = true;
			}
			else
			{
				if (NextContent != nil)
				{
					// Load web content from string
					[self.WebView loadHTMLString : NextContent baseURL : NextURL];
					NextContent = nil;
					NextURL = nil;
				}
				else if (NextURL != nil)
				{
					// Load web content from URL
					NSURLRequest *nsrequest = [NSURLRequest requestWithURL : NextURL];
					[self.WebView loadRequest : nsrequest];
					NextURL = nil;
				}
			}
		}
	});
#endif
}

-(NSString *)UrlDecode:(NSString *)stringToDecode
{
	NSString *result = [stringToDecode stringByReplacingOccurrencesOfString : @"+" withString:@" "];
	result = [result stringByRemovingPercentEncoding];
	return result;
}

#if !PLATFORM_TVOS
-(void)userContentController:(WKUserContentController *)userContentController didReceiveScriptMessage : (WKScriptMessage *)message
{
	if ([message.body isKindOfClass : [NSString class]])
	{
		NSString *Message = message.body;
		if (WebBrowserWidget != nil && Message != nil)
		{
			//NSLog(@"Received message %@", Message);
			WebBrowserWidget->ProcessScriptMessage(Message);
		}

	}
}
#endif

-(void)executejavascript:(NSString*)InJavaScript
{
#if !PLATFORM_TVOS
	dispatch_async(dispatch_get_main_queue(), ^
	{
		const char* InJavaScriptCString = [InJavaScript UTF8String];
		FString InJavaScriptFString = FString(InJavaScriptCString);
		
		//NSLog(@"executejavascript %@", InJavaScript);
		
		[self.WebView evaluateJavaScript : InJavaScript completionHandler : ^(id Object, NSError* Error) 
		 {
			if(Error != nil) 
			{
				const char *domainString = [Error.domain UTF8String];
				/*UE_LOGF(LogAppleWebBrowser, Error, "executeJavaScript(\"%ls\") errored with NSError %d in domain %s!", 
					   *InJavaScriptFString, Error.code, domainString);*/
			}
		}];
	});
#endif
}

-(void)loadurl:(NSURL*)InURL;
{
	dispatch_async(dispatch_get_main_queue(), ^
	{
		self.NextURL = InURL;
	});
}

-(void)loadstring:(NSString*)InString dummyurl : (NSURL*)InURL;
{
	dispatch_async(dispatch_get_main_queue(), ^
	{
		self.NextContent = InString;
		self.NextURL = InURL;
	});
}

-(bool)isUsingIndirectRendering;
{
	return Is3DBrowser || bUsingIndirectRendering;
}

-(bool)isAllowingScreenshots;
{
	return bAllowScreenshots;
}

-(void)set3D:(bool)InIs3DBrowser;
{
	dispatch_async(dispatch_get_main_queue(), ^
	{
		if (Is3DBrowser != InIs3DBrowser)
		{
			//default is 2D
			Is3DBrowser = InIs3DBrowser;
			[self setDefaultVisibility];
		}
	});
}

-(void)setUsingIndirectRendering:(bool)UsingIndirectRendering;
{
	bUsingIndirectRendering = UsingIndirectRendering;
	[self setDefaultVisibility];
}

-(void)setAllowingScreenshots:(bool)AllowScreenshots;
{
	bAllowScreenshots = AllowScreenshots;
}

-(void)setDefaultVisibility;
{
#if !PLATFORM_TVOS
	dispatch_async(dispatch_get_main_queue(), ^
	{
		if (Is3DBrowser)
		{
			[self.WebViewContainer setHidden : YES];
		}
		else
		{
			[self.WebViewContainer setHidden : NO];
		}
	});
#endif
}

-(void)setVisibility:(bool)InIsVisible;
{
#if !PLATFORM_TVOS
	dispatch_async(dispatch_get_main_queue(), ^
	{
		if (InIsVisible)
		{
			[self setDefaultVisibility];
		}
		else
		{
			[self.WebViewContainer setHidden : YES];
		}
	});
#endif
}

-(void)stopLoading;
{
#if !PLATFORM_TVOS
	dispatch_async(dispatch_get_main_queue(), ^
	{
		[self.WebView stopLoading];
	});
#endif
}

-(void)disableCallbacksOnShutdown;
{
	bShuttingDown = true;
}

-(void)reload;
{
#if !PLATFORM_TVOS
	dispatch_async(dispatch_get_main_queue(), ^
	{
		[self.WebView reload];
	});
#endif
}

-(void)goBack;
{
#if !PLATFORM_TVOS
	dispatch_async(dispatch_get_main_queue(), ^
	{
		[self.WebView goBack];
	});
#endif
}

-(void)goForward;
{
#if !PLATFORM_TVOS
	dispatch_async(dispatch_get_main_queue(), ^
	{
		[self.WebView goForward];
	});
#endif
}

-(bool)canGoBack;
{
#if PLATFORM_TVOS
	return false;
#else
	return [self.WebView canGoBack];
#endif
}

-(bool)canGoForward;
{
#if PLATFORM_TVOS
	return false;
#else
	return [self.WebView canGoForward];
#endif
}

-(FTextureRHIRef)GetVideoTexture;
{
	return VideoTexture;
}

-(void)SetVideoTexture:(FTextureRHIRef)Texture;
{
	VideoTexture = Texture;
}

-(void)SetVideoTextureValid:(bool)Condition;
{
	bVideoTextureValid = Condition;
}

-(bool)IsVideoTextureValid;
{
	return bVideoTextureValid;
}

-(bool)UpdateVideoFrame:(void*)ptr;
{
#if !PLATFORM_TVOS
	@synchronized(self) // Briefly block render thread
	{
		id<MTLTexture> ptrToMetalTexture = (id<MTLTexture>)ptr;

		[self updateWebViewMetalTexture : ptrToMetalTexture];
	}
#endif
	return true;
}

#if !PLATFORM_MAC
-(CGFloat)getScreenScale
{	
#if !PLATFORM_VISIONOS
	for(UIScene* scene in UIApplication.sharedApplication.connectedScenes) 
	{
		if([scene isMemberOfClass:[UIWindowScene class]])
		{
			for(UIWindow* window in ((UIWindowScene*)scene).windows) 
			{
				if(window.isKeyWindow) 
				{ 
					return ((UIWindowScene*)scene).screen.scale;
				}
			}
		}
	}
#endif
	
	return 1;
}
#endif

#if PLATFORM_MAC
-(void)replaceMetalTexture:(id<MTLTexture>)Texture fromImage:(NSImage*)Image;
#else
-(void)replaceMetalTexture:(id<MTLTexture>)Texture fromImage:(UIImage*)Image;
#endif
{
#if PLATFORM_MAC
	if(Image == nil)
#else
	if(Image == nil || Image.CGImage == nil)
#endif
	{
		return;
	}
	
	NSUInteger TextureWidth = [Texture width];
	NSUInteger TextureHeight = [Texture height];
	
	CGColorSpaceRef ColorSpace = CGColorSpaceCreateDeviceRGB();
	CGContextRef WebViewContext = CGBitmapContextCreate(NULL, TextureWidth, TextureHeight, 8, 4 * TextureWidth, ColorSpace, (CGBitmapInfo)kCGImageAlphaPremultipliedLast);
	
#if PLATFORM_MAC
	NSGraphicsContext *NewGfxContext = [NSGraphicsContext graphicsContextWithCGContext : WebViewContext
														  flipped : NO];
	NSGraphicsContext *OldGfxContext = NSGraphicsContext.currentContext;
	NSGraphicsContext.currentContext = NewGfxContext;
	
	[Image drawInRect: WebView.frame];
 
	NSGraphicsContext.currentContext = OldGfxContext;
#else
	CGContextDrawImage(WebViewContext, CGRectMake(0, 0, TextureWidth, TextureHeight), Image.CGImage);
#endif
	[Texture replaceRegion : MTLRegionMake2D(0, 0, TextureWidth, TextureHeight)
		 mipmapLevel : 0
		 withBytes : CGBitmapContextGetData(WebViewContext)
		 bytesPerRow : 4 * TextureWidth];
 
	CGColorSpaceRelease(ColorSpace);
	CGContextRelease(WebViewContext);
}

-(void)updateWebViewMetalTexture:(id<MTLTexture>)texture
{
#if !PLATFORM_TVOS
	dispatch_semaphore_t semaphore = dispatch_semaphore_create(0);
	
	dispatch_sync(dispatch_get_main_queue(), ^ 
	{
		@autoreleasepool 
		{
			WKSnapshotConfiguration *SnapshotConfig = [[WKSnapshotConfiguration alloc] init];
			
			NSUInteger TextureWidth = [texture width];
			NSUInteger TextureHeight = [texture height];
#if PLATFORM_MAC
			NSUInteger Width = TextureWidth;
			NSUInteger Height = TextureHeight;
			if(Is3DBrowser)
			{
				// This should only be the authoritative frame if we are a 3D browser
				// and we have no other UI constraints forced on us to worry about
				WebView.frame = CGRectMake(0, 0, Width, Height);
			}
#else
			float ScreenScale = [self getScreenScale];
			NSUInteger Width = TextureWidth / ScreenScale;
			NSUInteger Height = TextureHeight / ScreenScale;
			
			UIView* View = [IOSAppDelegate GetDelegate].IOSView;
			CGFloat ContentScaleFactor = View.contentScaleFactor;
			
			// snapshotWidth needs to be set to the pre-dpi scaling size of the WebView
			// to avoid an internal WKWebView memory leak
			WebView.frame = CGRectMake(0, 0, ContentScaleFactor * Width, ContentScaleFactor * Height);
			SnapshotConfig.snapshotWidth = [NSNumber numberWithFloat: WebView.bounds.size.width / ContentScaleFactor];
#endif
			SnapshotConfig.rect = CGRectMake(0, 0, WebView.bounds.size.width, WebView.bounds.size.height);
			
			[WebView takeSnapshotWithConfiguration : SnapshotConfig
								 completionHandler : 
#if PLATFORM_MAC
			 ^ void(NSImage *Image, NSError *Error)
#else
			 ^ void(UIImage *Image, NSError *Error)
#endif
			{
				if(Error != nil) 
				{
					UE_LOGF(LogAppleWebBrowser, Error, "takeSnapshotWithConfiguration errored with NSError %lld!", Error.code);
				}
				else if(Is3DBrowser)
				{
					[self replaceMetalTexture: texture fromImage: Image];
				}
				else if(bAllowScreenshots)
				{
					if(LastWebViewScreenshot != nil)
					{
						[LastWebViewScreenshot release];
					}
					LastWebViewScreenshot = Image;
					[LastWebViewScreenshot retain];
				}
				
				Image = nil;
				[SnapshotConfig release];
				
				if(!Is3DBrowser && [self isUsingIndirectRendering])
				{
					// This semaphore only matters if we are trying to do a synchronous copy of 
					// WKWebView's content, which is only necessary when we are trying to take a
					// screenshot.
					dispatch_semaphore_signal(semaphore);
				}
			}];
		}
	});
	
	if(!Is3DBrowser && [self isUsingIndirectRendering])
	{
		// Wait on the render thread for the screenshot to complete on the main thread.
		// Afterwards, we wait to copy that screenshot over to the metal texture to
		// make this one synchronous operation. This makes Slate screenshotting work as
		// expected without needing a WebKit screenshot from a previous frame.
		dispatch_semaphore_wait(semaphore, DISPATCH_TIME_FOREVER);
			
		dispatch_sync(dispatch_get_main_queue(), ^ 
		{
			check(bAllowScreenshots);
			[self replaceMetalTexture: texture fromImage: LastWebViewScreenshot];
		});
	}
#endif
}

-(bool)inKeyWindow;
{
#if PLATFORM_TVOS
	return YES;
#else
	if(self.WebViewContainer != nil && [self.WebViewContainer window] != nil)
	{
		return [[self.WebViewContainer window] isKeyWindow];
	}
	else 
	{
		return NO;
	}
#endif
}

#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
-(bool)isReadyToAddToNativeLayer;
{
	return bReadyToAddToNativeLayer;
}
#endif

#if !PLATFORM_TVOS

- (nullable WKWebView *)webView:(WKWebView *)InWebView createWebViewWithConfiguration:(WKWebViewConfiguration *)InConfiguration forNavigationAction:(WKNavigationAction *)InNavigationAction windowFeatures:(WKWindowFeatures *)InWindowFeatures
{
	NSURLRequest *request = InNavigationAction.request;
	FString UrlStr([[request URL] absoluteString]);

	if (InNavigationAction.targetFrame == nil && !UrlStr.IsEmpty() && FPlatformProcess::CanLaunchURL(*UrlStr))
	{
		if (WebBrowserWidget->HandleOnBeforePopup(UrlStr, TEXT("_blank")))
		{
			// Launched the URL in external browser, don't create a new webview
			return nil;
		}
	}
	return nil;
}

- (void)webView:(WKWebView*)InWebView decidePolicyForNavigationAction : (WKNavigationAction*)InNavigationAction decisionHandler : (void(^)(WKNavigationActionPolicy))InDecisionHandler
{
	if(IWebBrowserModule::IsAvailable() && !bShuttingDown)
	{
		NSURLRequest *request = InNavigationAction.request;
		FString UrlStr([[request URL] absoluteString]);

		if (InNavigationAction.targetFrame == nil && !UrlStr.IsEmpty() && FPlatformProcess::CanLaunchURL(*UrlStr))
		{
			if (WebBrowserWidget->HandleOnBeforePopup(UrlStr, TEXT("_blank")))
			{
				// Launched the URL in external browser, don't open the link here too
				InDecisionHandler(WKNavigationActionPolicyCancel);
				return;
			}
		}
		
		WebBrowserWidget->HandleShouldOverrideUrlLoading(UrlStr);
		InDecisionHandler(WKNavigationActionPolicyAllow);
	}
	else
	{
		UE_LOGF(LogAppleWebBrowser, Warning, 
			   "ApplePlatformWebBrowser received decidePolicyForNavigationAction callback during or after module shutdown, preventing navigation");
		InDecisionHandler(WKNavigationActionPolicyCancel);
	}
}

-(void)webView:(WKWebView *)InWebView didCommitNavigation : (WKNavigation *)InNavigation
{
	if(IWebBrowserModule::IsAvailable() && !bShuttingDown)
	{
		NSString* CurrentUrl = [self.WebView URL].absoluteString;
		NSString* Title = [self.WebView title];
		
	//	NSLog(@"didCommitNavigation: %@", CurrentUrl);
		WebBrowserWidget->HandleReceivedTitle(Title);
		WebBrowserWidget->HandlePageLoad(CurrentUrl, true);
	}
	else
	{
		UE_LOGF(LogAppleWebBrowser, Warning, 
			   "ApplePlatformWebBrowser received didCommitNavigation callback during or after module shutdown, ignoring");
	}
}

-(void)webView:(WKWebView *)InWebView didFinishNavigation : (WKNavigation *)InNavigation
{
	if(IWebBrowserModule::IsAvailable() && !bShuttingDown)
	{
		NSString* CurrentUrl = [self.WebView URL].absoluteString;
		NSString* Title = [self.WebView title];
		// NSLog(@"didFinishNavigation: %@", CurrentUrl);
		WebBrowserWidget->HandleReceivedTitle(Title);
		WebBrowserWidget->HandlePageLoad(CurrentUrl, false);
	}
	else
	{
		UE_LOGF(LogAppleWebBrowser, Warning, 
			   "ApplePlatformWebBrowser received didFinishNavigation callback during or after module shutdown, ignoring");
	}
}
-(void)webView:(WKWebView *)InWebView didFailNavigation : (WKNavigation *)InNavigation withError : (NSError*)InError
{
	if(IWebBrowserModule::IsAvailable() && !bShuttingDown)
	{
		if (InError.domain == NSURLErrorDomain && InError.code == NSURLErrorCancelled)
		{
			//ignore this one, interrupted load
			return;
		}
		NSString* CurrentUrl = [InError.userInfo objectForKey : @"NSErrorFailingURLStringKey"];
	//	NSLog(@"didFailNavigation: %@, error %@", CurrentUrl, InError);
		WebBrowserWidget->HandleReceivedError(InError.code, CurrentUrl);
	}
	else
	{
		UE_LOGF(LogAppleWebBrowser, Warning, 
			   "ApplePlatformWebBrowser received didFailNavigation callback during or after module shutdown, ignoring");
	}
}
-(void)webView:(WKWebView *)InWebView didFailProvisionalNavigation : (WKNavigation *)InNavigation withError : (NSError*)InError
{
	if(IWebBrowserModule::IsAvailable() && !bShuttingDown)
	{
		NSString* CurrentUrl = [InError.userInfo objectForKey : @"NSErrorFailingURLStringKey"];
		// NSLog(@"didFailProvisionalNavigation: %@, error %@", CurrentUrl, InError);
		WebBrowserWidget->HandleReceivedError(InError.code, CurrentUrl);
	}
	else
	{
		UE_LOGF(LogAppleWebBrowser, Warning, 
			   "ApplePlatformWebBrowser received didFailProvisionalNavigation callback during or after module shutdown, ignoring");
	}
}
#endif
@end

namespace {
	static const FString JSGetSourceCommand = "GetSource";
	static const FString JSMessageGetSourceScript =
		TEXT("	window.webkit.messageHandlers.") + FMobileJSScripting::JSMessageHandler + TEXT(".postMessage('")+ JSGetSourceCommand +
		TEXT("/' + encodeURIComponent(document.documentElement.innerHTML));");

}

FWebBrowserWindow::FWebBrowserWindow(FString InUrl, TOptional<FString> InContentsToLoad, bool InShowErrorMessage, bool InThumbMouseButtonNavigation, bool InUseTransparency, bool bInJSBindingToLoweringEnabled, FString InUserAgentApplication, bool bMobileJSReturnInDict)
	: CurrentUrl(MoveTemp(InUrl))
	, UserAgentApplication(MoveTemp(InUserAgentApplication))
	, ContentsToLoad(MoveTemp(InContentsToLoad))
	, bUseTransparency(InUseTransparency)
	, DocumentState(EWebBrowserDocumentState::NoDocument)
	, ErrorCode(0)
	, WindowSize(FIntPoint(500, 500))
	, bIsDisabled(false)
	, bIsVisible(true)
	, bTickedLastFrame(true)
{
	bool bInjectJSOnPageStarted = false;
	GConfig->GetBool(TEXT("Browser"), TEXT("bInjectJSOnPageStarted"), bInjectJSOnPageStarted, GEngineIni);
	
	Scripting = MakeShared<FMobileJSScripting>(bInJSBindingToLoweringEnabled, bInjectJSOnPageStarted, bMobileJSReturnInDict);
	WebView3DInput = MakeShared<FWebView3DInputHandler>();
}

FWebBrowserWindow::~FWebBrowserWindow()
{
	CloseBrowser(true, false);
}

void FWebBrowserWindow::LoadURL(FString NewURL)
{
	BrowserWidget->LoadURL(NewURL);
}

void FWebBrowserWindow::LoadString(FString Contents, FString DummyURL)
{
	BrowserWidget->LoadString(Contents, DummyURL);
}

TOptional<FSlateRenderTransform> FWebBrowserWindow::GetWebBrowserRenderTransform() const
{
	return FSlateRenderTransform();
}

TSharedRef<SViewport> FWebBrowserWindow::CreateWidget()
{
	TSharedRef<SAppleWebBrowserWidget> BrowserWidgetRef =
		SNew(SAppleWebBrowserWidget)
		.UseTransparency(bUseTransparency)
		.InitialURL(CurrentUrl)
		.UserAgentApplication(UserAgentApplication)
		.WebBrowserWindow(SharedThis(this))
		.RenderTransform(this, &FWebBrowserWindow::GetWebBrowserRenderTransform);

	BrowserWidget = BrowserWidgetRef;
	
	Scripting->SetWindow(SharedThis(this));

	return BrowserWidgetRef;
}

void FWebBrowserWindow::SetViewportSize(FIntPoint ViewportSize, FIntPoint ViewportPos)
{
	WindowSize = ViewportSize;
	if(BrowserWidget != nullptr) 
	{
		BrowserWidget->UpdateFrameWithViewport(WindowSize, ViewportPos);
	}
}

FIntPoint FWebBrowserWindow::GetViewportSize() const
{
	return WindowSize;
}

FSlateShaderResource* FWebBrowserWindow::GetTexture(bool bIsPopup /*= false*/)
{
	return nullptr;
}

bool FWebBrowserWindow::IsValid() const
{
	return true;
}

bool FWebBrowserWindow::IsInitialized() const
{
	return true;
}

bool FWebBrowserWindow::IsClosing() const
{
	return false;
}

bool FWebBrowserWindow::Is3D() 
{
	return BrowserWidget != nil ? BrowserWidget->Is3D() : false;
}

EWebBrowserDocumentState FWebBrowserWindow::GetDocumentLoadingState() const
{
	return DocumentState;
}

FString FWebBrowserWindow::GetTitle() const
{
	return Title;
}

FString FWebBrowserWindow::GetUrl() const
{
	return CurrentUrl;
}

bool FWebBrowserWindow::OnKeyDown(const FKeyEvent& InKeyEvent)
{
	if(Is3D())
	{
		WebView3DInput->SendKeyEventToJS(*this, "keydown", InKeyEvent);
	}
	else
	{
		RedirectNSEvent();
	}
	return true;
}

bool FWebBrowserWindow::OnKeyUp(const FKeyEvent& InKeyEvent)
{
	if(Is3D())
	{
		WebView3DInput->SendKeyEventToJS(*this, "keyup", InKeyEvent);
	}
	else
	{
		RedirectNSEvent();
	}
	return true;
}

bool FWebBrowserWindow::OnKeyChar(const FCharacterEvent& InCharacterEvent)
{
	if(Is3D()) 
	{
		WebView3DInput->SendCharEventToJS(*this, InCharacterEvent);
	}
	return true;
}

void FWebBrowserWindow::RedirectNSEvent() 
{
	// by default, some events (mainly keystrokes) don't get redirected to NSApp when consumed in Slate
#if PLATFORM_MAC
	if (const TSharedPtr<GenericApplication> Application = FSlateApplication::Get().GetPlatformApplication())
	{
		if(BrowserWidget != nullptr && BrowserWidget->InKeyWindow())
		{
			static_cast<FMacApplication*>(Application.Get())->ResendLastEvent();
		}
	}
#endif
}

FReply FWebBrowserWindow::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup)
{
	if(Is3D()) 
	{		
		WebView3DInput->SendMouseEventToJS(*this, "down", MouseEvent);
	}
	
	return FReply::Handled().ReleaseMouseCapture().ReleaseMouseLock();
}

FReply FWebBrowserWindow::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup)
{
	if(Is3D()) 
	{
		WebView3DInput->SendMouseEventToJS(*this, "up", MouseEvent);
		WebView3DInput->SendMouseEventToJS(*this, "click", MouseEvent);
	}

	return FReply::Handled().ReleaseMouseCapture().ReleaseMouseLock();
}

FReply FWebBrowserWindow::OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup)
{
	if(Is3D())
	{
		WebView3DInput->SendMouseEventToJS(*this, "dblclick", MouseEvent);
	}
	
	return FReply::Handled().ReleaseMouseCapture().ReleaseMouseLock();
}

FReply FWebBrowserWindow::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup)
{
	if(Is3D()) 
	{
		WebView3DInput->SendMouseEventToJS(*this, "move", MouseEvent);
	}
	
	return FReply::Handled().ReleaseMouseCapture().ReleaseMouseLock();
}

void FWebBrowserWindow::OnMouseLeave(const FPointerEvent& MouseEvent)
{
}

void FWebBrowserWindow::SetSupportsMouseWheel(bool bValue)
{

}

bool FWebBrowserWindow::GetSupportsMouseWheel() const
{
	return false;
}

FReply FWebBrowserWindow::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup)
{
	if(Is3D())
	{
		WebView3DInput->SendMouseEventToJS(*this, "wheel", MouseEvent);
	}
	
	return FReply::Handled();
}

FReply FWebBrowserWindow::OnTouchGesture(const FGeometry& MyGeometry, const FPointerEvent& GestureEvent, bool bIsPopup)
{
	return FReply::Unhandled();
}


void FWebBrowserWindow::OnFocus(bool SetFocus, bool bIsPopup)
{
	if(BrowserWidget != nullptr)
	{		
		BrowserWidget->OnFocus(SetFocus);
	}
}

void FWebBrowserWindow::OnCaptureLost()
{
}

bool FWebBrowserWindow::CanGoBack() const
{
	return BrowserWidget->CanGoBack();
}

void FWebBrowserWindow::GoBack()
{
	BrowserWidget->GoBack();
}

bool FWebBrowserWindow::CanGoForward() const
{
	return BrowserWidget->CanGoForward();
}

void FWebBrowserWindow::GoForward()
{
	BrowserWidget->GoForward();
}

bool FWebBrowserWindow::IsLoading() const
{
	return DocumentState != EWebBrowserDocumentState::Loading;
}

void FWebBrowserWindow::Reload()
{
	BrowserWidget->Reload();
}

void FWebBrowserWindow::StopLoad()
{
	BrowserWidget->StopLoad();
}

void FWebBrowserWindow::GetSource(TFunction<void(const FString&)> Callback) const
{
	//@todo: decide what to do about multiple pending requests
	GetPageSourceCallback.Emplace(Callback);

	// Ugly hack: Work around the fact that ExecuteJavascript is non-const.
	const_cast<FWebBrowserWindow*>(this)->ExecuteJavascript(JSMessageGetSourceScript);
}

int FWebBrowserWindow::GetLoadError()
{
	return ErrorCode;
}

void FWebBrowserWindow::NotifyDocumentError(const FString& InCurrentUrl, int InErrorCode)
{
	if (!CurrentUrl.Equals(InCurrentUrl, ESearchCase::CaseSensitive))
	{
		CurrentUrl = InCurrentUrl;
		UrlChangedEvent.Broadcast(CurrentUrl);
	}

	ErrorCode = InErrorCode;
	DocumentState = EWebBrowserDocumentState::Error;
	DocumentStateChangedEvent.Broadcast(DocumentState);
}

void FWebBrowserWindow::NotifyDocumentLoadingStateChange(const FString& InCurrentUrl, bool IsLoading)
{
	// Ignore a load completed notification if there was an error.
	// For load started, reset any errors from previous page load.
	bool bIsNotError = DocumentState != EWebBrowserDocumentState::Error;
	if (IsLoading || bIsNotError)
	{
		if (!CurrentUrl.Equals(InCurrentUrl, ESearchCase::CaseSensitive))
		{
			CurrentUrl = InCurrentUrl;
			UrlChangedEvent.Broadcast(CurrentUrl);
		}

		if (bIsNotError && !InCurrentUrl.StartsWith("javascript:"))
		{
			if (IsLoading)
			{
				Scripting->PageStarted(SharedThis(this));
			}
			else
			{
				Scripting->PageLoaded(SharedThis(this));
			}
		}
		ErrorCode = 0;
		DocumentState = IsLoading
			? EWebBrowserDocumentState::Loading
			: EWebBrowserDocumentState::Completed;
		DocumentStateChangedEvent.Broadcast(DocumentState);
	}

}

void FWebBrowserWindow::SetIsDisabled(bool bValue)
{
	bIsDisabled = bValue;
}

TSharedPtr<SWindow> FWebBrowserWindow::GetParentWindow() const
{
	if (ParentWindow.IsValid())
	{
		return ParentWindow.Pin();
	}
	return nullptr;
}

void FWebBrowserWindow::SetParentWindow(TSharedPtr<SWindow> Window)
{
	ParentWindow = Window;
}

void FWebBrowserWindow::ShowFloatingCloseButton(bool bShow, bool bDraggable)
{
	if (BrowserWidget)
	{
		BrowserWidget->ShowFloatingCloseButton(bShow, bDraggable);
	}
}

void FWebBrowserWindow::ExecuteJavascript(const FString& Script)
{
	BrowserWidget->ExecuteJavascript(Script);
}

void FWebBrowserWindow::CloseBrowser(bool bForce, bool bBlockTillClosed /* ignored */)
{
	BrowserWidget->Close();
}

bool FWebBrowserWindow::OnJsMessageReceived(const FString& Command, const TArray<FString>& Params, const FString& Origin)
{
	if (Command.Equals(JSGetSourceCommand, ESearchCase::CaseSensitive) && GetPageSourceCallback.IsSet() && Params.Num() == 1)
	{
		GetPageSourceCallback.GetValue()(Params[0]);
		GetPageSourceCallback.Reset();
		return true;
	}
	return Scripting->OnJsMessageReceived(Command, Params, Origin);
}

void FWebBrowserWindow::BindUObject(const FString& Name, UObject* Object, bool bIsPermanent /*= true*/)
{
	Scripting->BindUObject(Name, Object, bIsPermanent);
}

void FWebBrowserWindow::UnbindUObject(const FString& Name, UObject* Object /*= nullptr*/, bool bIsPermanent /*= true*/)
{
	Scripting->UnbindUObject(Name, Object, bIsPermanent);
}

void FWebBrowserWindow::CheckTickActivity()
{
	if (bIsVisible != bTickedLastFrame)
	{
		bIsVisible = bTickedLastFrame;
		BrowserWidget->SetWebBrowserVisibility(bIsVisible);
	}

	bTickedLastFrame = false;
	
	BrowserWidget->ScreenshotFinished();
}

void FWebBrowserWindow::SetTickLastFrame()
{
	bTickedLastFrame = !bIsDisabled;
}

bool FWebBrowserWindow::IsVisible()
{
	return bIsVisible;
}

void FWebBrowserWindow::FloatingCloseButtonPressed()
{
	if (OnFloatingCloseButtonPressed().IsBound())
	{
		OnFloatingCloseButtonPressed().Execute();
	}
}

void FWebBrowserWindow::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	// Not supported on this platform
}

FReply FWebBrowserWindow::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	// Not supported on this platform
	return FReply::Unhandled();
}

void FWebBrowserWindow::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	// Not supported on this platform
}

FReply FWebBrowserWindow::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	// Not supported on this platform
	return FReply::Unhandled();
}
#endif
