// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebView3DInputHandler.h"

#include "IWebBrowserWindow.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#include "HAL/PlatformInput.h"
#endif

#import <Foundation/Foundation.h>

DEFINE_LOG_CATEGORY(LogWebView3DInputHandler);

FWebView3DInputHandler::FWebView3DInputHandler()
	: bCtrlKeyDown(false) 
	, bAltKeyDown(false)
	, bCmdKeyDown(false) 
	, bShiftKeyDown(false)
	, bCapsLockOn(false)
{
}

FString FWebView3DInputHandler::BoolToString(bool InBool)
{
	return InBool ? "true" : "false";
}

FString FWebView3DInputHandler::GetModifierKeyStatus()
{
	return FString::Printf(TEXT("altKey: %s, ctrlKey: %s, metaKey: %s, shiftKey: %s"), 
						   *BoolToString(bAltKeyDown), *BoolToString(bCtrlKeyDown), 
						   *BoolToString(bCmdKeyDown), *BoolToString(bShiftKeyDown));
}

bool FWebView3DInputHandler::CheckIfCharacterIsInputable(TCHAR CharToInput)
{
	// is there a more UE way to do this?
	return ![NSCharacterSet.illegalCharacterSet characterIsMember: CharToInput] && 
		   ![NSCharacterSet.controlCharacterSet characterIsMember: CharToInput];
}

FString FWebView3DInputHandler::GetCurrentJSElement()
{
	return FString::Printf(TEXT("document.elementFromPoint(%f, %f)"), MousePos.X, MousePos.Y); 
}

void FWebView3DInputHandler::SendMouseEventToJS(IWebBrowserWindow& InWindow, const FString& EventName, const FPointerEvent& MouseEvent) 
{
	MousePos = MouseEvent.GetScreenSpacePosition();
	TSet<FKey> PressedButtons = MouseEvent.GetPressedButtons();
	FKey EffectingButton = MouseEvent.GetEffectingButton();
	int32_t PressedButtonSum = 0;
	FString DispatchEventJS;
	
	if(EffectingButton != EKeys::Invalid) 
	{
		PressedButtons.Add(EffectingButton);
	}
	
	for(FKey Button : PressedButtons) 
	{
		if(Button == EKeys::LeftMouseButton) 
		{
			PressedButtonSum += 1;
		} 
		else if(Button == EKeys::RightMouseButton) 
		{
			PressedButtonSum += 2;
		} 
		else if(Button == EKeys::MiddleMouseButton) 
		{
			PressedButtonSum += 4;
		}
	}
	
	FString CurrentJSElement = FWebView3DInputHandler::GetCurrentJSElement();
	FString BaseDispatch = FString::Printf(TEXT("%s.dispatchEvent"), *CurrentJSElement);
	FString BaseMouseEventData = FString::Printf(TEXT("view: window, clientX: %f, clientY: %f, buttons: %d, bubbles: true, %s"), 
												 MousePos.X, MousePos.Y, PressedButtonSum, *GetModifierKeyStatus());
	
	FString MouseEventName = EventName;
	FString PointerEventName;
	
	if(EventName == "wheel")
	{
		// we need to update scrollLeft and scrollTop of the target, dispatch the wheel event and the scroll event
		float WheelDelta = MouseEvent.GetWheelDelta();
		float ScrollDelta = InWindow.GetViewportSize().Y * WheelDelta;
		
		FString UpdateScrollJS = FString::Printf(TEXT("window.scrollBy(0.0, %f)"), ScrollDelta);
		InWindow.ExecuteJavascript(UpdateScrollJS);
		
		DispatchEventJS = FString::Printf(TEXT("%s(new WheelEvent(\"%s\", {%s, deltaY: %f}))"), *BaseDispatch, *EventName, *BaseMouseEventData, ScrollDelta);
	}
	else 
	{
		// Most sites should still work with just mouse events, but we dispatch
		// pointer events as well for newer sites which may take them for granted.
		if(EventName != "click" && EventName != "dblclick")
		{
			MouseEventName = FString::Printf(TEXT("mouse%s"), *EventName);
			PointerEventName = FString::Printf(TEXT("pointer%s"), *EventName);
		}
		
		if(EffectingButton != EKeys::Invalid) 
		{
			int32_t PressedButton = 0;
			
			if(EffectingButton == EKeys::LeftMouseButton) 
			{
				PressedButton = 0;
			} 
			else if(EffectingButton == EKeys::RightMouseButton) 
			{
				PressedButton = 2;
			} 
			else if(EffectingButton == EKeys::MiddleMouseButton) 
			{
				PressedButton = 1;
			}
			
			// follows JS convention for mouse buttons: 0 (left), 1 (middle), 2 (right)
			DispatchEventJS = FString::Printf(TEXT("%s(new MouseEvent(\"%s\", {%s, button: %d}))"), *BaseDispatch, *MouseEventName, 
											  *BaseMouseEventData, PressedButton);
			
			if(PointerEventName.Len() > 0)
			{
				InWindow.ExecuteJavascript(DispatchEventJS);
				
				DispatchEventJS = FString::Printf(TEXT("%s(new PointerEvent(\"%s\", {%s, button: %d}))"), *BaseDispatch, *PointerEventName, 
												  *BaseMouseEventData, PressedButton);
			}
		}
		else
		{
			DispatchEventJS = FString::Printf(TEXT("%s(new MouseEvent(\"%s\", {%s}))"), *BaseDispatch, *MouseEventName, *BaseMouseEventData);
			
			if(PointerEventName.Len() > 0)
			{
				InWindow.ExecuteJavascript(DispatchEventJS);
				
				DispatchEventJS = FString::Printf(TEXT("%s(new PointerEvent(\"%s\", {%s}))"), *BaseDispatch, *PointerEventName, *BaseMouseEventData);
			}
		}
	}
	
	InWindow.ExecuteJavascript(DispatchEventJS);
}

void FWebView3DInputHandler::DispatchInputEvent(IWebBrowserWindow& InWindow, const FString& InputType, TCHAR* InputData)
{
	FString CurrentJSElement = GetCurrentJSElement();
	FString InputDataString;
	FString BaseInputEvent;
	
	if(InputData != nullptr)
	{
		InputDataString = FString(1, InputData).ReplaceCharWithEscapedChar();
		
		if(bCapsLockOn || bShiftKeyDown)
		{
			InputDataString = InputDataString.ToUpper();
		}
		
		BaseInputEvent = FString::Printf(TEXT("data: \"%s\", inputType: \"%s\", isComposing: false"), *InputDataString, *InputType);
	}
	else
	{
		BaseInputEvent = FString::Printf(TEXT("data: null, inputType: \"%s\", isComposing: false"), *InputType);
	}
	
	FString BeforeInputJS = FString::Printf(TEXT("%s.dispatchEvent(new InputEvent(\"beforeinput\", {%s}))"), *CurrentJSElement, *BaseInputEvent);
	
	InWindow.ExecuteJavascript(BeforeInputJS);
	
	if(InputType == "insertText" && InputData != nullptr)
	{
		FString ContentJS = FString::Printf(TEXT("%s.value += \"%s\";"), *CurrentJSElement, *InputDataString);
		
		InWindow.ExecuteJavascript(ContentJS);
	}
	else if(InputType == "deleteContentBackward")
	{
		FString ContentJS = FString::Printf(TEXT("%s.value = %s.value.slice(0, -1);"), *CurrentJSElement, *CurrentJSElement);
		
		InWindow.ExecuteJavascript(ContentJS);
	}
	
	FString InputJS = FString::Printf(TEXT("%s.dispatchEvent(new InputEvent(\"input\", {%s}))"), *CurrentJSElement, *BaseInputEvent);
	
	InWindow.ExecuteJavascript(InputJS);
}

void FWebView3DInputHandler::SendCharEventToJS(IWebBrowserWindow& InWindow,
											   const FCharacterEvent& CharEvent) 
{
	TCHAR InputChar = CharEvent.GetCharacter();
	
	if(CheckIfCharacterIsInputable(InputChar))
	{
		DispatchInputEvent(InWindow, "insertText", &InputChar);
	}
	else
	{
		UE_LOGF(LogWebView3DInputHandler, Verbose, "Uninputable character received, ignoring");
	}
}

void FWebView3DInputHandler::UpdateModifierKeys(const FKey& Key, bool KeyDown)
{
	if (Key == EKeys::LeftControl || Key == EKeys::RightControl)
	{
		bCtrlKeyDown = KeyDown;
	}
	else if (Key == EKeys::LeftAlt || Key == EKeys::RightAlt)
	{
		bAltKeyDown = KeyDown;
	}
	else if (Key == EKeys::LeftCommand || Key == EKeys::RightCommand)
	{
		bCmdKeyDown = KeyDown;
	}
	else if (Key == EKeys::LeftShift || Key == EKeys::RightShift)
	{
		bShiftKeyDown = KeyDown;
	}
	else if (Key == EKeys::CapsLock)
	{
		bCapsLockOn = KeyDown;
	}
}

void FWebView3DInputHandler::SendKeyEventToJS(IWebBrowserWindow& InWindow, const FString& EventName, const FKeyEvent& KeyEvent) 
{
	// check for supported key event type
	check(EventName == "keyup" || EventName == "keydown");
	
	FString CurrentJSElement = GetCurrentJSElement();
	FString DispatchEventJS;
	FKey Key = KeyEvent.GetKey();
	bool KeyDown = EventName == "keydown";
	uint32 CharacterKeyU32 = KeyEvent.GetCharacter();
	uint32 KeyEventCode = KeyEvent.GetKeyCode();
	
	// modifier keys should be updated before we construct the event to dispatch
	UpdateModifierKeys(Key, KeyDown);
	
	if(CharacterKeyU32 == 0)
	{
#if PLATFORM_MAC
		if(Key.IsModifierKey())
		{
			KeyEventCode = FMacPlatformInput::GetMacNativeModifierKeyCode(KeyEventCode);
		}
#endif
		
		DispatchEventJS = FString::Printf(TEXT("%s.dispatchEvent(new KeyboardEvent(\"%s\", {code: %d, %s}));"), 
										  *CurrentJSElement, *EventName, KeyEventCode, *GetModifierKeyStatus());
	}
	else
	{
		FString KeyEventCharacter = FString(reinterpret_cast<TCHAR*>(&CharacterKeyU32), 1).ReplaceCharWithEscapedChar();
		
		DispatchEventJS = FString::Printf(TEXT("%s.dispatchEvent(new KeyboardEvent(\"%s\", {code: %d, key: \"%s\", %s}));"), 
										  *CurrentJSElement, *EventName, KeyEventCode, *KeyEventCharacter, *GetModifierKeyStatus());
	}

	InWindow.ExecuteJavascript(DispatchEventJS);
	
	if (KeyDown && (Key == EKeys::BackSpace || Key == EKeys::Delete))  
	{
		DispatchInputEvent(InWindow, "deleteContentBackward", nullptr);
	}
}
