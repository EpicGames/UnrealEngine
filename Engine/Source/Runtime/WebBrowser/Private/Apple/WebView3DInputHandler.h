// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Input/Events.h"

DECLARE_LOG_CATEGORY_EXTERN(LogWebView3DInputHandler, Log, All);

class IWebBrowserWindow;

/**
* Helper class for ApplePlatformWebBrowser's FWebBrowserWindow to handle input with 3D WKWebView
* Web Browser widgets.
 * @warning Due to WebKit limitations, all input events dispatched in 3D mode will be untrusted (i.e. Event.isTrusted will be false). 
*/
class FWebView3DInputHandler 
{
public:
	
	FWebView3DInputHandler();
	
	void SendMouseEventToJS(IWebBrowserWindow& InWindow,
							const FString& EventName, const FPointerEvent& MouseEvent);
	void SendCharEventToJS(IWebBrowserWindow& InWindow, const FCharacterEvent& CharEvent);
	void SendKeyEventToJS(IWebBrowserWindow& InWindow, const FString& EventName, const FKeyEvent& KeyEvent);
	
private:
	
	FString BoolToString(bool InBool);
	FString GetCurrentJSElement();
	
	void DispatchInputEvent(IWebBrowserWindow& InWindow, const FString& InputType, TCHAR* InputData);
	
	FString GetModifierKeyStatus();
	bool CheckIfCharacterIsInputable(TCHAR CharToInput);
	
	void UpdateModifierKeys(const FKey& Key, bool KeyDown);
	
	bool bCtrlKeyDown;
	bool bAltKeyDown;
	bool bCmdKeyDown;
	bool bShiftKeyDown;
	bool bCapsLockOn;
	
	FVector2f MousePos;
};
