// Copyright Epic Games, Inc. All Rights Reserved.
#include "WebViewCloseButton.h"

#if PLATFORM_IOS
#if !PLATFORM_TVOS

static constexpr float GFullscreenCloseButtonSize = 40.0f;
static constexpr float GFullscreenCloseButtonInnerXInnerSize = 32.0f;
static constexpr float GFullscreenCloseButtonInnerXOuterSize = 34.0f;
static constexpr float GFullscreenCloseButtonOffsetFromEdge = 4.0f;
static constexpr float GCloseButtonClickDragTolerance = 10.0f;

static const uint8_t XTextureData[] =
{
		0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00,
		0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00,
		0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00,
		0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00,
		0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00,
		0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00,
		0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x71, 0x93, 0x3d, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00,
		0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x50, 0xe9, 0xfa, 0xe3, 0x58, 0x00, 0x00, 0x00, 0x00, 0x00,
		0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0xbc, 0xf8, 0xfa, 0xe3, 0x58, 0x00, 0x00, 0x00, 0x00,
		0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x1e, 0xc2, 0xf8, 0xfa, 0xe3, 0x58, 0x00, 0x00, 0x00,
		0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x1e, 0xc2, 0xf8, 0xfa, 0xe3, 0x58, 0x00, 0x00,
		0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x1e, 0xc2, 0xf8, 0xfa, 0xe3, 0x58, 0x00,
		0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x1e, 0xc2, 0xf8, 0xfa, 0xe3, 0x58,
		0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x1e, 0xc2, 0xf8, 0xfa, 0xe5,
		0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x1e, 0xc2, 0xf8, 0xfa,
		0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x34, 0xdc, 0xfa,
};

struct FDefaultSymmetricTextureColorFn
{
	uint32_t operator()(uint32_t In) { return In; }
};

static UIImage* CreateSymmetricTexture(const uint8_t Data[16 * 16], TFunction<uint32_t(uint32_t)>&& ColorFn = FDefaultSymmetricTextureColorFn())
{
	uint32_t TextureData[32 * 32];

	for (int Row = 0; Row < 16; ++Row)
	{
		for (int Col = 0; Col < 16; ++Col)
		{
			const uint32_t Gray = Data[Row * 16 + Col];
			const uint32_t Color = ColorFn(0xFFFFFF | (Gray << 24));

#define Q1 Row*32+Col
#define Q2 Row*32+(32 - Col - 1)
#define Q3 (32 - Row - 1)*32+Col
#define Q4 (32 - Row - 1)*32+(32 - Col - 1)
			TextureData[Q1] = Color;
			TextureData[Q2] = Color;
			TextureData[Q3] = Color;
			TextureData[Q4] = Color;
#undef Q1
#undef Q2
#undef Q3
#undef Q4
		}
	}

	UIGraphicsBeginImageContext(CGSizeMake(32, 32));
	CGColorSpaceRef ColorSpace = CGColorSpaceCreateDeviceRGB();
	CGBitmapInfo BitmapInfo = kCGImageAlphaFirst | kCGBitmapByteOrder32Little;
	CGDataProviderRef DataProvider = CGDataProviderCreateWithData(NULL, TextureData, sizeof(TextureData), NULL);
	CGImageRef ImageRef = CGImageCreate(32, 32, 8, 32, 32 * 4, ColorSpace, BitmapInfo, DataProvider, NULL, NO, kCGRenderingIntentDefault);
	CGColorSpaceRelease(ColorSpace);
	CGDataProviderRelease(DataProvider);

	CGContextRef G = UIGraphicsGetCurrentContext();
	CGContextClearRect(G, CGRectMake(0, 0, 32, 32));
	CGContextDrawImage(G, CGRectMake(0, 0, 32, 32), ImageRef);
	UIImage* Result = UIGraphicsGetImageFromCurrentImageContext();
	UIGraphicsEndImageContext();

	return Result;
}

static UIImage* CreateTintedImage(UIImage* SourceImage, UIColor* TintColor)
{
	bool bNeedsTint = false;
	CGFloat Red, Green, Blue, Alpha;
	if ([TintColor getRed:&Red green:&Green blue:&Blue alpha:&Alpha])
	{
		bNeedsTint = Red < 0.99 || Green < 0.99 || Blue < 0.99 || Alpha < 0.99;
	}
	else
	{
		bNeedsTint = !CGColorEqualToColor(UIColor.whiteColor.CGColor, TintColor.CGColor);
	}

	if (!bNeedsTint)
	{
		return SourceImage;
	}

	size_t ImageWidth = (size_t)(SourceImage.size.width * SourceImage.scale);
	size_t ImageHeight = (size_t)(SourceImage.size.height * SourceImage.scale);
	if (!ImageHeight || !ImageWidth)
	{
		return SourceImage;
	}

	uint32_t* PixelData = (uint32_t*)calloc(ImageWidth*ImageHeight, 4);
	CGColorSpaceRef ColorSpace = CGColorSpaceCreateDeviceRGB();

	CGRect Rect = CGRectMake(0, 0, ImageWidth, ImageHeight);
	CGRect FirstPixelDataEntryRect = CGRectMake(0, ImageHeight-1, 1, 1); // bottom-most scanline is the first row in the array
	CGContextRef BitmapContext = CGBitmapContextCreate(PixelData, ImageWidth, ImageHeight, 8, ImageWidth*4, ColorSpace, kCGImageAlphaPremultipliedLast | kCGImageByteOrder32Big);
	CGContextClearRect(BitmapContext, Rect);
	CGContextSetBlendMode(BitmapContext, kCGBlendModeNormal);
	CGContextSetFillColorSpace(BitmapContext, CGColorGetColorSpace(TintColor.CGColor));
	CGContextSetFillColor(BitmapContext, CGColorGetComponents(TintColor.CGColor));
	CGContextFillRect(BitmapContext, FirstPixelDataEntryRect); // write a single pixel in the tinting color so we can read it out as RGBA
	CGContextFlush(BitmapContext);

	// PixelData now contains a single entry with our TintColor -- grab it.
	const uint32_t PremultipliedTintColor = *PixelData;
	uint32_t TA = (PremultipliedTintColor >> 24) & 0xFF;
	uint32_t TB = (PremultipliedTintColor >> 16) & 0xFF;
	uint32_t TG = (PremultipliedTintColor >> 8) & 0xFF;
	uint32_t TR = PremultipliedTintColor & 0xFF;
	if (TA)
	{
		TB = TB * 255 / TA + 1;
		TG = TG * 255 / TA + 1;
		TR = TR * 255 / TA + 1;
	}

	CGContextClearRect(BitmapContext, FirstPixelDataEntryRect);
	CGContextDrawImage(BitmapContext, Rect, SourceImage.CGImage);
	CGContextFlush(BitmapContext);

	// PixelData now contains the SourceImage pixel data with premultiplied alpha.
	for (size_t Y = 0; Y < ImageHeight; ++Y)
	{
		uint32_t* PixelRow = &PixelData[Y*ImageWidth];
		for (size_t X = 0; X < ImageWidth; ++X)
		{
			uint32_t& Pixel = PixelRow[X];
			uint32_t SA = (Pixel >> 24) & 0xFF, SB = (Pixel >> 16) & 0xFF, SG = (Pixel >> 8) & 0xFF, SR = Pixel & 0xFF;
			if (SA)
			{
				SB = SB * 255 / SA;
				SG = SG * 255 / SA;
				SR = SR * 255 / SA;

				uint32_t NewAlpha = SA*TA/255;
				NewAlpha = NewAlpha + ((255 - NewAlpha)*TA / 255) + 1;

				SB = (SB*TB*NewAlpha + 0x80) >> 16;
				SG = (SG*TG*NewAlpha + 0x80) >> 16;
				SR = (SR*TR*NewAlpha + 0x80) >> 16;
				SA = NewAlpha - 1;
				Pixel = (SA << 24) | (SB << 16) | (SG << 8) | SR;
			}
		}
	}

	// Get an image of the pixel data
	CGImageRef TempImage = CGBitmapContextCreateImage(BitmapContext);
	UIGraphicsBeginImageContext(CGSizeMake(ImageWidth, ImageHeight));
	CGContextRef CopyImageContext = UIGraphicsGetCurrentContext();
	CGContextDrawImage(CopyImageContext, Rect, TempImage);
	UIImage* OutImage = UIGraphicsGetImageFromCurrentImageContext();
	UIGraphicsEndImageContext();

	// Release resources
	CGImageRelease(TempImage);
	CGContextRelease(BitmapContext);
	CGColorSpaceRelease(ColorSpace);

	return OutImage;
}

@interface WebViewCloseButtonDrawingLayer : NSObject
@property (nonatomic, retain) UIImage* Texture;
@property (nonatomic) CGFloat TextureInset;
@property (nonatomic) BOOL bDrawWhileTouchDown;
@property (nonatomic) CGBlendMode BlendMode;
@end

@implementation WebViewCloseButtonDrawingLayer

+ (instancetype)drawingLayerWithTexture:(UIImage*)Texture blendMode:(CGBlendMode)BlendMode inset:(CGFloat)TextureInset forTouchDown:(BOOL)bDrawWhileTouchDown
{
	WebViewCloseButtonDrawingLayer* Layer = [self new];
	Layer.Texture = Texture;
	Layer.TextureInset = TextureInset;
	Layer.bDrawWhileTouchDown = bDrawWhileTouchDown;
	Layer.BlendMode = BlendMode;
	return Layer;
}

@end

@implementation WebViewCloseButton
{
	NSMutableArray* DrawingLayers;
	BOOL bIsDraggable;
	BOOL bIsTouchDown;
	BOOL bMayBeDragged;
	CGPoint TouchDownPoint;
	CGPoint DownDisplacement;
	CGPoint Displacement;
	CGPoint LayoutPosition;
	CGRect SafeBounds;
}

- (void)dealloc
{
	[DrawingLayers release];
	
	[super dealloc];
}

- (instancetype)initWithFrame:(CGRect)frame
{
	if (self = [super initWithFrame:frame])
	{
		DrawingLayers = [[NSMutableArray arrayWithCapacity:4] retain];
		self.hidden = YES;
		bIsTouchDown = NO;
		bMayBeDragged = NO;
		bIsDraggable = NO;
	}
	return self;
}

-(void)showButton:(BOOL)bShow setDraggable:(BOOL)bDraggable
{
	self.hidden = !bShow;
	if (bIsDraggable != bDraggable)
	{
		bIsDraggable = bDraggable;
		[self setupLayout];
	}
}

- (void)handleTap
{
	if (self.TapHandler)
	{
		self.TapHandler();
	}
}

- (void)addDrawingLayerTexture:(UIImage *)Texture textureMultiply:(nonnull UIColor *)TextureMultiply blendMode:(CGBlendMode)BlendMode inset:(CGFloat)TextureInset forTouchDown:(BOOL)bInIsTouchDown
{
	if (Texture)
	{
		Texture = CreateTintedImage(Texture, TextureMultiply);
		[DrawingLayers addObject:[WebViewCloseButtonDrawingLayer drawingLayerWithTexture:Texture blendMode:BlendMode inset:TextureInset forTouchDown:bInIsTouchDown]];
		[self setNeedsDisplay];
	}
}

- (void)setupLayout
{
	bMayBeDragged = NO;
	Displacement = CGPointMake(0,0);
	self.bounds = CGRectMake(0,0, GFullscreenCloseButtonSize, GFullscreenCloseButtonSize);
	
	if (!self.superview)
	{
		return;
	}

	UIEdgeInsets SuperInsets = [self.superview safeAreaInsets];
	
	// Substract insets from parent bounds
	SafeBounds = self.superview.bounds;
	SafeBounds.origin.x += SuperInsets.left;
	SafeBounds.origin.y += SuperInsets.top;
	SafeBounds.size.width -= SuperInsets.left + SuperInsets.right;
	SafeBounds.size.height -= SuperInsets.top + SuperInsets.bottom;
	
	// Substract space for some margins and size of the brutton from its center
	SafeBounds = CGRectInset(SafeBounds, GFullscreenCloseButtonSize/2 + GFullscreenCloseButtonOffsetFromEdge, GFullscreenCloseButtonSize/2 + GFullscreenCloseButtonOffsetFromEdge);

	// Position the button in the top right corner fo the safe bounds
	LayoutPosition = CGPointMake(SafeBounds.origin.x + SafeBounds.size.width, SafeBounds.origin.y);
	self.center = LayoutPosition;
}

- (void)drawRect:(CGRect)rect
{
	for (WebViewCloseButtonDrawingLayer* Layer in DrawingLayers)
	{
		if (Layer.bDrawWhileTouchDown != bIsTouchDown)
		{
			continue;
		}

		CGRect RectCopy = CGRectInset(rect, Layer.TextureInset, Layer.TextureInset);
		[Layer.Texture drawInRect:RectCopy blendMode:Layer.BlendMode alpha:1.0];
	}
}

- (void)touchesBegan:(NSSet<UITouch *> *)touches withEvent:(UIEvent *)event
{
	if (touches.count > 1)
	{
		return;
	}

	bIsTouchDown = [self pointInside:[touches.anyObject locationInView:self] withEvent:event];
	if (bIsTouchDown)
	{
		bMayBeDragged = bIsDraggable;
		UITouch* Touch = [[touches allObjects] objectAtIndex: 0];
		TouchDownPoint = [Touch locationInView: self.superview];
		DownDisplacement = CGPointMake(TouchDownPoint.x - Displacement.x, TouchDownPoint.y - Displacement.y);
	}
	[self setNeedsDisplay];
}

- (void)touchesMoved:(NSSet<UITouch *> *)touches withEvent:(UIEvent *)event
{
	if (touches.count > 1)
	{
		return;
	}

	if (bMayBeDragged)
	{
		UITouch* Touch = [[touches allObjects] objectAtIndex: 0];
		CGPoint Point = [Touch locationInView: self.superview];
		CGPoint AttemptedDisplacement = CGPointMake(Point.x - DownDisplacement.x, Point.y - DownDisplacement.y);

		CGPoint UpdatedCenter = CGPointMake( FMath::Clamp(LayoutPosition.x + AttemptedDisplacement.x, CGRectGetMinX(SafeBounds), CGRectGetMaxX(SafeBounds)),
											 FMath::Clamp(LayoutPosition.y + AttemptedDisplacement.y, CGRectGetMinY(SafeBounds), CGRectGetMaxY(SafeBounds)));
		
		Displacement = CGPointMake(UpdatedCenter.x - LayoutPosition.x, UpdatedCenter.y - LayoutPosition.y);
		
		self.center = UpdatedCenter;
	}
	bIsTouchDown = [self pointInside:[touches.anyObject locationInView:self] withEvent:event];
	[self setNeedsDisplay];
}

- (void)touchesEnded:(NSSet<UITouch *> *)touches withEvent:(UIEvent *)event
{
	if (touches.count > 1)
	{
		return;
	}

	bIsTouchDown = NO;
	[self setNeedsDisplay];

	UITouch* Touch = [[touches allObjects] objectAtIndex: 0];
	CGPoint Point = [Touch locationInView: self.superview];
	CGPoint AbsDisplacement = CGPointMake(Point.x - TouchDownPoint.x, Point.y - TouchDownPoint.y);
	
	if (FMath::Abs(AbsDisplacement.x) < GCloseButtonClickDragTolerance && FMath::Abs(AbsDisplacement.y) < GCloseButtonClickDragTolerance )
	{
		if ([self pointInside:[touches.anyObject locationInView:self] withEvent:event])
		{
			[self handleTap];
		}
	}
}

- (void)touchesCancelled:(NSSet<UITouch *> *)touches withEvent:(UIEvent *)event
{
	bIsTouchDown = NO;
	bMayBeDragged = NO;
	[self setNeedsDisplay];
}

- (void)touchesEstimatedPropertiesUpdated:(NSSet<UITouch *> *)touches
{
}

@end

WebViewCloseButton* MakeCloseButton()
{
	WebViewCloseButton* Button = [[WebViewCloseButton alloc] initWithFrame:CGRectZero];
	Button.translatesAutoresizingMaskIntoConstraints = NO;
	Button.opaque = NO;

	UIImage* XImage = CreateSymmetricTexture(XTextureData);

	// X states
	CGFloat InsetWhite = (GFullscreenCloseButtonSize - GFullscreenCloseButtonInnerXInnerSize) / 2.0;
	CGFloat InsetDark = (GFullscreenCloseButtonSize - GFullscreenCloseButtonInnerXOuterSize) / 2.0;
	[Button addDrawingLayerTexture:XImage textureMultiply:UIColor.blackColor blendMode:kCGBlendModeNormal inset:InsetDark forTouchDown:NO];
	[Button addDrawingLayerTexture:XImage textureMultiply:UIColor.whiteColor blendMode:kCGBlendModeNormal inset:InsetWhite forTouchDown:NO];
	[Button addDrawingLayerTexture:XImage textureMultiply:UIColor.whiteColor blendMode:kCGBlendModeNormal inset:InsetDark forTouchDown:YES];
	[Button addDrawingLayerTexture:XImage textureMultiply:UIColor.blackColor blendMode:kCGBlendModeNormal inset:InsetWhite forTouchDown:YES];

	return Button;
}

#endif // !PLATFORM_TVOS
#endif // PLATFORM_IOS
