// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if PLATFORM_IOS

#if !PLATFORM_TVOS
#import <UIKit/UIKit.h>

@interface WebViewCloseButton : UIView
@property (nonatomic, copy) void (^TapHandler)(void);

- (void)setupLayout;
- (void)showButton:(BOOL)bShow setDraggable:(BOOL)bDraggable;

@end

WebViewCloseButton* MakeCloseButton();
#endif

#endif
