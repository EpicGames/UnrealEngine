// Copyright Epic Games, Inc. All Rights Reserved.

#include "CEF/CEFBrowserApp.h"
#include "HAL/IConsoleManager.h"

#if WITH_CEF3
#include "WebBrowserLog.h"
#include "CEF/CEFWebBrowserWindowRHIHelper.h"

//#define DEBUG_CEFMESSAGELOOP_FRAMERATE 1 // uncomment this to have debug spew about the FPS we call the CefDoMessageLoopWork function

DEFINE_LOG_CATEGORY(LogCEFBrowser);

static bool bCEFGPUAcceleration = true;
static FAutoConsoleVariableRef CVarCEFGPUAcceleration(
	TEXT("r.CEFGPUAcceleration"),
	bCEFGPUAcceleration,
	TEXT("Enables GPU acceleration in CEF\n"),
	ECVF_Default);

FCEFBrowserApp::FCEFBrowserApp()
	: MessagePumpCountdown(0)
{
}

void FCEFBrowserApp::OnBeforeChildProcessLaunch(CefRefPtr<CefCommandLine> CommandLine)
{
}

void FCEFBrowserApp::OnBeforeCommandLineProcessing(const CefString& ProcessType, CefRefPtr< CefCommandLine > CommandLine)
{
	if (bCEFGPUAcceleration)
	{
		UE_LOGF(LogCEFBrowser, Log, "CEF GPU acceleration enabled");
		CommandLine->AppendSwitch("enable-gpu");
		CommandLine->AppendSwitch("enable-gpu-compositing");

#if PLATFORM_WINDOWS
		// Make sure CEF uses the same adapter as UE for shared textures to work,
		// which requires using ANGLE's D3D11 backend and having a valid (non-zero) LUID
		const uint64_t AdapterLuid = FCEFWebBrowserWindowRHIHelper::GetRHIAdapterLuid();
		if (AdapterLuid)
		{
			CommandLine->AppendSwitchWithValue("use-angle", "d3d11");
			CommandLine->AppendSwitchWithValue("use-adapter-luid",
				*FString::Printf(TEXT("%u,%u"), static_cast<uint32_t>(AdapterLuid >> 32), static_cast<uint32_t>(AdapterLuid & 0xFFFFFFFF)));
		}
#endif // PLATFORM_WINDOWS

	}
	else
	{
		UE_LOGF(LogCEFBrowser, Log, "CEF GPU acceleration disabled");
		CommandLine->AppendSwitch("disable-gpu");
		CommandLine->AppendSwitch("disable-gpu-compositing");
	}

#if PLATFORM_LINUX
	CommandLine->AppendSwitchWithValue("ozone-platform", "headless");
	// Stick to the GL backend for Angle as the Vulkan backend is failing to initialize,
	// likely due to the removal of the conflicting libvulkan.so.1 loader at CL43202912
#if 0
	CommandLine->AppendSwitchWithValue("use-gl", "angle");
	CommandLine->AppendSwitchWithValue("use-angle", "vulkan");
	CommandLine->AppendSwitchWithValue("use-vulkan", "native");
	CommandLine->AppendSwitchWithValue("enable-features", "Vulkan");
#else
	CommandLine->AppendSwitchWithValue("use-gl", "angle");
	CommandLine->AppendSwitchWithValue("use-angle", "gl");
#endif
#endif

	CommandLine->AppendSwitch("enable-begin-frame-scheduling");
	CommandLine->AppendSwitch("disable-pinch"); // the web pages we have don't expect zoom to work right now so disable touchpad pinch zoom

	// These settings reduce the amount of (unused) data downloaded by Chromium under cache_path
	CommandLine->AppendSwitch("disable-background-networking");
	CommandLine->AppendSwitch("disable-component-update");
	CommandLine->AppendSwitch("disable-domain-reliability");
	CommandLine->AppendSwitch("disable-extensions");
	CommandLine->AppendSwitch("disable-gpu-shader-disk-cache");
	CommandLine->AppendSwitch("disable-sync");

#if PLATFORM_MAC
	CommandLine->AppendSwitch("use-mock-keychain"); // Disable the toolchain prompt on macOS.
#endif

	CommandLine->AppendSwitchWithValue("ipc-connection-timeout", "120"); // Increase the timeout from default 15s to 120s as some plugins can delay things on startup
	CommandLine->AppendSwitchWithValue("test-memory-log-delay-in-minutes", "10000"); // Skip memory metrics reporting, it causes occasional crashes inside the memory-mapped allocator

	// Uncomment these to lines to create a FULL network log from chrome, which can then be inspected using https://netlog-viewer.appspot.com/
	//CommandLine->AppendSwitchWithValue("log-net-log", "c:\\temp\\cef_net_log.json");
	//CommandLine->AppendSwitchWithValue("net-log-capture-mode", "IncludeCookiesAndCredentials");
}

void FCEFBrowserApp::OnScheduleMessagePumpWork(int64_t delay_ms)
{
	FScopeLock Lock(&MessagePumpCountdownCS);

	// As per CEF documentation, if delay_ms is <= 0, then the call to CefDoMessageLoopWork should happen reasonably soon.  If delay_ms is > 0, then the call
	//  to CefDoMessageLoopWork should be scheduled to happen after the specified delay and any currently pending scheduled call should be canceled.
	if (delay_ms < 0)
	{
		delay_ms = 0;
	}
	MessagePumpCountdown = delay_ms;
}

bool FCEFBrowserApp::TickMessagePump(float DeltaTime, bool bForce)
{
	bool bPump = false;
	{
		FScopeLock Lock(&MessagePumpCountdownCS);
		
		// count down in order to call message pump
		if (MessagePumpCountdown >= 0)
		{
			MessagePumpCountdown -= (DeltaTime * 1000);
			if (MessagePumpCountdown <= 0)
			{
				bPump = true;
			}
			
			if (bPump || bForce)
			{
				// -1 indicates that no countdown is currently happening
				MessagePumpCountdown = -1;
			}
		}
	}
	
#ifdef  DEBUG_CEFMESSAGELOOP_FRAMERATE
	static float SecondsFrameRate = 0;
	static int NumFrames = 0;
	SecondsFrameRate += DeltaTime;
#endif
	if (bPump || bForce)
	{
#ifdef DEBUG_CEFMESSAGELOOP_FRAMERATE
		++NumFrames;
		if (NumFrames % 100 == 0 || SecondsFrameRate > 5.0f)
		{
			UE_LOGF(LogCEFBrowser, Log, "CefDoMessageLoopWork call Frame Rate %0.2f", NumFrames / std::max(SecondsFrameRate, 0.01f));
			SecondsFrameRate = 0;
			NumFrames = 0;
		}

		static double LastTickTime = FPlatformTime::Seconds();
		double BeforeSec = FPlatformTime::Seconds();
#endif
		CefDoMessageLoopWork();
#ifdef DEBUG_CEFMESSAGELOOP_FRAMERATE
		double AfterSec = FPlatformTime::Seconds();
		UE_LOGF(LogCEFBrowser, Verbose, "FCEFBrowserApp::TickMessagePump %u ms %u ms %ls", static_cast<uint32_t>((BeforeSec - LastTickTime) * 1000), static_cast<uint32_t>((AfterSec - BeforeSec) * 1000), bForce ? TEXT("FORCED") : TEXT(""));
		LastTickTime = BeforeSec;
#endif
		return true;
	}
	return false;
}

bool FCEFBrowserApp::OnAlreadyRunningAppRelaunch(CefRefPtr<CefCommandLine> CommandLine, const CefString& CurrentDirectory)
{
	// nothing to do, except report the relaunch as handled to prevent the default handler from opening a new browser window
	return true;
}

#endif
