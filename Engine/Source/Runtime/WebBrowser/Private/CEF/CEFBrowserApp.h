// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/ScopeLock.h"

#if WITH_CEF3

#include "CEFLibCefIncludes.h"

DECLARE_LOG_CATEGORY_EXTERN(LogCEFBrowser, Log, All);

/**
 * Implements CEF App and other Process level interfaces
 */
class FCEFBrowserApp : public CefApp,
	public CefBrowserProcessHandler
{
public:

	/**
	 * Default Constructor
	 */
	FCEFBrowserApp();

	/** Used to pump the CEF message loop whenever OnScheduleMessagePumpWork is triggered */
	bool TickMessagePump(float DeltaTime, bool bForce);

private:
	// CefApp methods.
	virtual CefRefPtr<CefBrowserProcessHandler> GetBrowserProcessHandler() override { return this; }
	virtual void OnBeforeCommandLineProcessing(const CefString& ProcessType, CefRefPtr< CefCommandLine > CommandLine) override;
	// CefBrowserProcessHandler methods:
	virtual void OnBeforeChildProcessLaunch(CefRefPtr<CefCommandLine> CommandLine) override;
	virtual void OnScheduleMessagePumpWork(int64_t delay_ms) override;
	virtual bool OnAlreadyRunningAppRelaunch(CefRefPtr<CefCommandLine> CommandLine, const CefString& CurrentDirectory) override;

	// Include the default reference counting implementation.
	IMPLEMENT_REFCOUNTING(FCEFBrowserApp);

	// Lock for access MessagePumpCountdown
	FCriticalSection MessagePumpCountdownCS;
	// Countdown in milliseconds until CefDoMessageLoopWork is called.  Updated by OnScheduleMessagePumpWork
	int64 MessagePumpCountdown;
};
#endif
