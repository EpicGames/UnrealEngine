// Copyright Epic Games, Inc. All Rights Reserved.

#include "CEF/CEFBrowserByteResource.h"

#if WITH_CEF3

FCEFBrowserByteResource::FCEFBrowserByteResource(const CefRefPtr<CefPostDataElement>& PostData, const FString& InMimeType)
	: Position(0)
	, Buffer(nullptr)
	, MimeType(InMimeType)
{
	Size = PostData->GetBytesCount();
	if (Size > 0)
	{
		Buffer = new unsigned char[Size];
		PostData->GetBytes(Size, Buffer);
	}
}

FCEFBrowserByteResource::~FCEFBrowserByteResource()
{
	delete[] Buffer;
}

void FCEFBrowserByteResource::Cancel()
{

}

void FCEFBrowserByteResource::GetResponseHeaders(CefRefPtr<CefResponse> Response, int64_t& ResponseLength, CefString& RedirectUrl)
{
	Response->SetMimeType(TCHAR_TO_WCHAR(*MimeType));
	Response->SetStatus(200);
	Response->SetStatusText("OK");
	ResponseLength = Size;
}

bool FCEFBrowserByteResource::ProcessRequest(CefRefPtr<CefRequest> Request, CefRefPtr<CefCallback> Callback)
{
	Callback->Continue();
	return true;
}

bool FCEFBrowserByteResource::ReadResponse(void* DataOut, int BytesToRead, int& BytesRead, CefRefPtr<CefCallback> Callback)
{
	int32 BytesLeft = Size - Position;
	BytesRead = BytesLeft >= BytesToRead ? BytesToRead : BytesLeft;
	if (BytesRead > 0)
	{
		FMemory::Memcpy(DataOut, Buffer + Position, BytesRead);
		Position += BytesRead;
		return true;
	}
	return false;
}
#endif
