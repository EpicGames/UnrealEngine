// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_CEF3


#include "CEFLibCefIncludes.h"



/**
 * Implements a resource handler that will return the contents of a string as the result.
 */
class FCEFBrowserByteResource
	: public CefResourceHandler
{
public:
	/**
	 */
	FCEFBrowserByteResource(const CefRefPtr<CefPostDataElement>& PostData, const FString& InMimeType);
	~FCEFBrowserByteResource();
	
	// CefResourceHandler interface
	virtual void Cancel() override;
	virtual void GetResponseHeaders(CefRefPtr<CefResponse> Response, int64_t& ResponseLength, CefString& RedirectUrl) override;
	virtual bool ProcessRequest(CefRefPtr<CefRequest> Request, CefRefPtr<CefCallback> Callback) override;
	virtual bool ReadResponse(void* DataOut, int BytesToRead, int& BytesRead, CefRefPtr<CefCallback> Callback) override;
	
private:
	int32 Position;
	int32 Size;
	unsigned char* Buffer;
	FString MimeType;
	
	// Include the default reference counting implementation.
	IMPLEMENT_REFCOUNTING(FCEFBrowserByteResource);
};


#endif
