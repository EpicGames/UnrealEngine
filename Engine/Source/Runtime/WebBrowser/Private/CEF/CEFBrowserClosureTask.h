// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_CEF3
#include "CEFLibCefIncludes.h"

// Helper for posting a closure as a task
class FCEFBrowserClosureTask
	: public CefTask
{
public:
	FCEFBrowserClosureTask(CefRefPtr<CefBaseRefCounted> InHandle, TFunction<void ()> InClosure)
		: Handle(InHandle)
		, Closure(InClosure)
	{ }

	virtual void Execute() override
	{
		Closure();
	}

private:
	CefRefPtr<CefBaseRefCounted> Handle; // Used so the handler will not go out of scope before the closure is executed.
	TFunction<void ()> Closure;
	IMPLEMENT_REFCOUNTING(FCEFBrowserClosureTask);
};


#endif /* WITH_CEF3 */
