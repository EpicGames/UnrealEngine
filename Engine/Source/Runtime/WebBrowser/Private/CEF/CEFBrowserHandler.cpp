// Copyright Epic Games, Inc. All Rights Reserved.

#include "CEF/CEFBrowserHandler.h"
#include "HAL/PlatformApplicationMisc.h"
#include "WebBrowserLog.h"

#if WITH_CEF3

//#define DEBUG_ONBEFORELOAD // Debug print beforebrowse steps

#include "WebBrowserModule.h"
#include "CEFBrowserClosureTask.h"
#include "IWebBrowserSingleton.h"
#include "WebBrowserSingleton.h"
#include "CEFBrowserPopupFeatures.h"
#include "CEFWebBrowserWindow.h"
#include "CEFBrowserByteResource.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/ThreadingBase.h"
#include "PlatformHttp.h"
#include "Misc/CommandLine.h"

#define LOCTEXT_NAMESPACE "WebBrowserHandler"

#ifdef DEBUG_ONBEFORELOAD
// Debug helper function to track URL loads
void LogCEFLoad(const FString &Msg, CefRefPtr<CefRequest> Request)
{
	auto url = Request->GetURL();
	auto type = Request->GetResourceType();
	if (type == CefRequest::ResourceType::RT_MAIN_FRAME || type == CefRequest::ResourceType::RT_XHR || type == CefRequest::ResourceType::RT_SUB_RESOURCE|| type == CefRequest::ResourceType::RT_SUB_FRAME)
	{
		GLog->Logf(ELogVerbosity::Display, TEXT("%s :%s type:%s"), *Msg, url.c_str(), *ResourceTypeToString(type));
	}
}

#define LOG_CEF_LOAD(MSG) LogCEFLoad(#MSG, Request)
#else
#define LOG_CEF_LOAD(MSG)
#endif

// Used to force returning custom content instead of performing a request.
const FString CustomContentMethod("X-GET-CUSTOM-CONTENT");

FCEFBrowserHandler::FCEFBrowserHandler(bool InUseTransparency, bool InInterceptLoadRequests, const TArray<FString>& InAltRetryDomains, const TArray<FString>& InAuthorizationHeaderAllowListURLS)
	: bUseTransparency(InUseTransparency), 
	bAllowAllCookies(false),
	bInterceptLoadRequests(InInterceptLoadRequests),
	AltRetryDomains(InAltRetryDomains),
	AuthorizationHeaderAllowListURLS(InAuthorizationHeaderAllowListURLS)
{
	// should we forcefully allow all cookies to be set rather than filtering a couple store side ones
	bAllowAllCookies = FParse::Param(FCommandLine::Get(), TEXT("CefAllowAllCookies"));
}

void FCEFBrowserHandler::OnTitleChange(CefRefPtr<CefBrowser> Browser, const CefString& Title)
{
	TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();

	if (BrowserWindow.IsValid())
	{
		BrowserWindow->SetTitle(Title);
	}
}

void FCEFBrowserHandler::OnAddressChange(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, const CefString& Url)
{
	if (Frame->IsMain())
	{
		TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();

		if (BrowserWindow.IsValid())
		{
			BrowserWindow->SetUrl(Url);
		}
	}
}

bool FCEFBrowserHandler::OnTooltip(CefRefPtr<CefBrowser> Browser, CefString& Text)
{
	TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();
	if (BrowserWindow.IsValid())
	{
		BrowserWindow->SetToolTip(Text);
	}

	return false;
}

bool FCEFBrowserHandler::OnConsoleMessage(CefRefPtr<CefBrowser> Browser, cef_log_severity_t level, const CefString& Message, const CefString& Source, int Line)
{
	ConsoleMessageDelegate.ExecuteIfBound(Browser, level, Message, Source, Line);
	// Return false to let it output to console.
	return false;
}

void FCEFBrowserHandler::OnAfterCreated(CefRefPtr<CefBrowser> Browser)
{
	if (Browser->IsPopup())
	{
		TSharedPtr<FCEFWebBrowserWindow> BrowserWindowParent = ParentHandler.get() ? ParentHandler->BrowserWindowPtr.Pin() : nullptr;
		if (BrowserWindowParent.IsValid() && ParentHandler->OnCreateWindow().IsBound())
		{
			TSharedPtr<FWebBrowserWindowInfo> NewBrowserWindowInfo = MakeShareable(new FWebBrowserWindowInfo(Browser, this));
			TSharedPtr<IWebBrowserWindow> NewBrowserWindow = IWebBrowserModule::Get().GetSingleton()->CreateBrowserWindow(
				BrowserWindowParent,
				NewBrowserWindowInfo
				);

			{
				// @todo: At the moment we need to downcast since the handler does not support using the interface.
				TSharedPtr<FCEFWebBrowserWindow> HandlerSpecificBrowserWindow = StaticCastSharedPtr<FCEFWebBrowserWindow>(NewBrowserWindow);
				BrowserWindowPtr = HandlerSpecificBrowserWindow;
			}

			// Request a UI window for the browser.  If it is not created we do some cleanup.
			bool bUIWindowCreated = ParentHandler->OnCreateWindow().Execute(TWeakPtr<IWebBrowserWindow>(NewBrowserWindow), TWeakPtr<IWebBrowserPopupFeatures>(BrowserPopupFeatures));
			if (!bUIWindowCreated)
			{
				NewBrowserWindow->CloseBrowser(true);
			}
			else
			{
				checkf(!NewBrowserWindow.IsUnique(), TEXT("Handler indicated that new window UI was created, but failed to save the new WebBrowserWindow instance."));
			}
		}
		else
		{
			Browser->GetHost()->CloseBrowser(true);
		}
	}
}

bool FCEFBrowserHandler::DoClose(CefRefPtr<CefBrowser> Browser)
{
	TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();
	if (BrowserWindow.IsValid())
	{
		BrowserWindow->OnBrowserClosing();
	}
#if PLATFORM_WINDOWS
	// If we have a window handle, we're rendering directly to the screen and not off-screen
	HWND NativeWindowHandle = Browser->GetHost()->GetWindowHandle();
	if (NativeWindowHandle != nullptr)
	{
		HWND ParentWindow = ::GetParent(NativeWindowHandle);

		if (ParentWindow)
		{
			HWND FocusHandle = ::GetFocus();
			if (FocusHandle && (FocusHandle == NativeWindowHandle || ::IsChild(NativeWindowHandle, FocusHandle)))
			{
				// Set focus to the parent window, otherwise keyboard and mouse wheel input will become wonky
				::SetFocus(ParentWindow);
			}
			// CEF will send a WM_CLOSE to the parent window and potentially exit the application if we don't do this
			::SetParent(NativeWindowHandle, nullptr);
		}
	}
#endif
	return false;
}

void FCEFBrowserHandler::OnBeforeClose(CefRefPtr<CefBrowser> Browser)
{
	TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();
	if (BrowserWindow.IsValid())
	{
		BrowserWindow->OnBrowserClosed();
	}

}

bool FCEFBrowserHandler::OnBeforePopup(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, const CefString& TargetUrl, const CefString& TargetFrameName, const CefPopupFeatures& PopupFeatures, CefWindowInfo& OutWindowInfo, CefRefPtr<CefClient>& OutClient, CefBrowserSettings& OutSettings, bool* OutNoJavascriptAccess)
{
	FString URL = WCHAR_TO_TCHAR(TargetUrl.ToWString().c_str());
	FString FrameName = WCHAR_TO_TCHAR(TargetFrameName.ToWString().c_str());

	/* If OnBeforePopup() is not bound, we allow creating new windows as long as OnCreateWindow() is bound.
	   The BeforePopup delegate is always executed even if OnCreateWindow is not bound to anything .
	  */
	if ((OnBeforePopup().IsBound() && OnBeforePopup().Execute(URL, FrameName)) || !OnCreateWindow().IsBound())
	{
		return true;
	}
	else
	{
		TSharedPtr<FCEFBrowserPopupFeatures> NewBrowserPopupFeatures = MakeShareable(new FCEFBrowserPopupFeatures(PopupFeatures));
		bool bIsDevtools = URL.Contains(TEXT("chrome-devtools"));
		bool shouldUseTransparency = bIsDevtools ? false : bUseTransparency;
		NewBrowserPopupFeatures->SetResizable(bIsDevtools); // only have the window for DevTools have resize options

		cef_color_t Alpha = shouldUseTransparency ? 0 : CefColorGetA(OutSettings.background_color);
		cef_color_t R = CefColorGetR(OutSettings.background_color);
		cef_color_t G = CefColorGetG(OutSettings.background_color);
		cef_color_t B = CefColorGetB(OutSettings.background_color);
		OutSettings.background_color = CefColorSetARGB(Alpha, R, G, B);

		CefRefPtr<FCEFBrowserHandler> NewHandler(new FCEFBrowserHandler(shouldUseTransparency, true /*InterceptLoadRequests*/));
		NewHandler->ParentHandler = this;
		NewHandler->SetPopupFeatures(NewBrowserPopupFeatures);
		OutClient = NewHandler;

		// Always use off screen rendering so we can integrate with our windows
#if PLATFORM_LINUX
		OutWindowInfo.SetAsWindowless(kNullWindowHandle);
#elif PLATFORM_WINDOWS
		OutWindowInfo.SetAsWindowless(kNullWindowHandle);
		OutWindowInfo.shared_texture_enabled = 0; // always render popups with the simple OSR renderer
#elif PLATFORM_MAC
		OutWindowInfo.SetAsWindowless(kNullWindowHandle);
		TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();
		if (BrowserWindow.IsValid())
		{
			OutWindowInfo.shared_texture_enabled = BrowserWindow->UsingAcceleratedPaint() ? 1 : 0; // match what other windows do
		}
		else
		{
			OutWindowInfo.shared_texture_enabled = 0;
		}
#else
		OutWindowInfo.SetAsWindowless(kNullWindowHandle);
#endif

		// We need to rely on CEF to create our window so we set the WindowInfo, BrowserSettings, Client, and then return false
		return false;
	}
}

bool FCEFBrowserHandler::OnCertificateError(CefRefPtr<CefBrowser> Browser, cef_errorcode_t CertError, const CefString &RequestUrl, CefRefPtr<CefSSLInfo> SslInfo, CefRefPtr<CefCallback> Callback)
{
	// Forward the cert error to the normal load error handler
	CefString ErrorText = "Certificate error";
	OnLoadError(Browser, Browser->GetMainFrame(), CertError, ErrorText, RequestUrl);
	return false;
}

void FCEFBrowserHandler::OnLoadError(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefLoadHandler::ErrorCode InErrorCode, const CefString& ErrorText, const CefString& FailedUrl)
{
	// notify browser window
	if (Frame->IsMain())
	{
		TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();

		if (BrowserWindow.IsValid())
		{
			if (AltRetryDomains.Num() > 0 && AltRetryDomainIdx < (uint32)AltRetryDomains.Num())
			{
				FString Url = WCHAR_TO_TCHAR(FailedUrl.ToWString().c_str());
				FString OriginalUrlDomain = FPlatformHttp::GetUrlDomain(Url);
				if (!OriginalUrlDomain.IsEmpty())
				{
					const FString NewUrl(Url.Replace(*OriginalUrlDomain, *AltRetryDomains[AltRetryDomainIdx++]));
					BrowserWindow->LoadURL(NewUrl);
					return;
				}

			}
			BrowserWindow->NotifyDocumentError(InErrorCode, ErrorText, FailedUrl);
		}
	}
}

void FCEFBrowserHandler::OnLoadStart(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, TransitionType CefTransitionType)
{
}

void FCEFBrowserHandler::OnLoadingStateChange(CefRefPtr<CefBrowser> Browser, bool bIsLoading, bool bCanGoBack, bool bCanGoForward)
{
	TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();

	if (BrowserWindow.IsValid())
	{
		BrowserWindow->NotifyDocumentLoadingStateChange(bIsLoading);
	}
}

bool FCEFBrowserHandler::GetRootScreenRect(CefRefPtr<CefBrowser> Browser, CefRect& Rect)
{
	if (CefCurrentlyOn(TID_UI) && FSlateApplication::IsInitialized())
	{
		// CEF may call this off the main gamethread which slate requires, so double check here
		FDisplayMetrics DisplayMetrics;
		FSlateApplication::Get().GetDisplayMetrics(DisplayMetrics);
		Rect.width = DisplayMetrics.PrimaryDisplayWidth;
		Rect.height = DisplayMetrics.PrimaryDisplayHeight;
		return true;
	}
	
	return false;
}

void FCEFBrowserHandler::GetViewRect(CefRefPtr<CefBrowser> Browser, CefRect& Rect)
{
	TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();

	if (BrowserWindow.IsValid())
	{
		BrowserWindow->GetViewRect(Rect);
	}
	else
	{
		// CEF requires at least a 1x1 area for painting
		Rect.x = Rect.y = 0;
		Rect.width = Rect.height = 1;
	}
}

void FCEFBrowserHandler::OnPaint(CefRefPtr<CefBrowser> Browser, PaintElementType Type, const RectList& DirtyRects, const void* Buffer, int Width, int Height)
{
	TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();

	if (BrowserWindow.IsValid())
	{
		BrowserWindow->OnPaint(Type, DirtyRects, Buffer, Width, Height);
	}
}

void FCEFBrowserHandler::OnAcceleratedPaint(CefRefPtr<CefBrowser> Browser, PaintElementType Type, const RectList& DirtyRects, const CefAcceleratedPaintInfo& SharedHandle)
{
	TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();

	if (BrowserWindow.IsValid())
	{
		BrowserWindow->OnAcceleratedPaint(Type, DirtyRects, SharedHandle);
	}
}

bool FCEFBrowserHandler::OnCursorChange(CefRefPtr<CefBrowser> Browser, CefCursorHandle Cursor, cef_cursor_type_t Type, const CefCursorInfo& CustomCursorInfo)
{
	TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();

	if (BrowserWindow.IsValid())
	{
		return BrowserWindow->OnCursorChange(Cursor, Type, CustomCursorInfo);
	}
	return false;
}

void FCEFBrowserHandler::OnPopupShow(CefRefPtr<CefBrowser> Browser, bool bShow)
{
	TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();

	if (BrowserWindow.IsValid())
	{
		BrowserWindow->ShowPopupMenu(bShow);
	}

}

void FCEFBrowserHandler::OnPopupSize(CefRefPtr<CefBrowser> Browser, const CefRect& Rect)
{
	TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();

	if (BrowserWindow.IsValid())
	{
		BrowserWindow->SetPopupMenuPosition(Rect);
	}
}

bool FCEFBrowserHandler::GetScreenInfo(CefRefPtr<CefBrowser> Browser, CefScreenInfo& ScreenInfo)
{
	TSharedPtr<FWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();
	ScreenInfo.depth = 24;

	if (BrowserWindow.IsValid() && BrowserWindow->GetParentWindow().IsValid())
	{
		ScreenInfo.device_scale_factor = BrowserWindow->GetParentWindow()->GetNativeWindow()->GetDPIScaleFactor();
	}
	else
	{
		FDisplayMetrics DisplayMetrics;
		FDisplayMetrics::RebuildDisplayMetrics(DisplayMetrics);
		ScreenInfo.device_scale_factor = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(DisplayMetrics.PrimaryDisplayWorkAreaRect.Left, DisplayMetrics.PrimaryDisplayWorkAreaRect.Top);
	}
	return true;
}


#if !PLATFORM_LINUX
void FCEFBrowserHandler::OnImeCompositionRangeChanged(CefRefPtr<CefBrowser> Browser, const CefRange& SelectionRange, const CefRenderHandler::RectList& CharacterBounds)
{
	TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();
	if (BrowserWindow.IsValid())
	{
		BrowserWindow->OnImeCompositionRangeChanged(Browser, SelectionRange, CharacterBounds);
	}
}

void FCEFBrowserHandler::OnTextSelectionChanged(CefRefPtr<CefBrowser> Browser, const CefString& SelectedText, const CefRange& SelectionRange)
{
	TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();
	if (BrowserWindow.IsValid())
	{
		BrowserWindow->OnTextSelectionChanged(Browser, SelectedText, SelectionRange);
	}
}
#endif


CefResourceRequestHandler::ReturnValue FCEFBrowserHandler::OnBeforeResourceLoad(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefRequest> Request, CefRefPtr<CefCallback> Callback)
{
	if (Request->IsReadOnly())
	{
		LOG_CEF_LOAD("FCEFBrowserHandler::OnBeforeResourceLoad - readonly");

		// we can't alter this request so just allow it through
		return RV_CONTINUE;
	}

	// Current thread is IO thread. We need to invoke BrowserWindow->GetResourceContent on the UI (aka Game) thread:
	CefPostTask(TID_UI, new FCEFBrowserClosureTask(this, [=, this]()
	{
		const FString LanguageHeaderText("Accept-Language");
		const FString LocaleCode = FWebBrowserSingleton::GetCurrentLocaleCode();
		CefRequest::HeaderMap HeaderMap;
		Request->GetHeaderMap(HeaderMap);
		auto LanguageHeader = HeaderMap.find(TCHAR_TO_WCHAR(*LanguageHeaderText));
		if (LanguageHeader != HeaderMap.end())
		{
			(*LanguageHeader).second = TCHAR_TO_WCHAR(*LocaleCode);
		}
		else
		{
			HeaderMap.insert(std::pair<CefString, CefString>(TCHAR_TO_WCHAR(*LanguageHeaderText), TCHAR_TO_WCHAR(*LocaleCode)));
		}
		
		LOG_CEF_LOAD("FCEFBrowserHandler::OnBeforeResourceLoad");

		if (BeforeResourceLoadDelegate.IsBound())
		{
			// Allow appending the Authorization header if this was NOT  a RT_XHR type of page load
			bool bAllowCredentials = URLRequestAllowsCredentials(WCHAR_TO_TCHAR(Request->GetURL().ToWString().c_str()));
			FRequestHeaders AdditionalHeaders;
			BeforeResourceLoadDelegate.Execute(Request->GetURL(), Request->GetResourceType(), AdditionalHeaders, bAllowCredentials);

			for (auto Iter = AdditionalHeaders.CreateConstIterator(); Iter; ++Iter)
			{
				const FString& Header = Iter.Key();
				const FString& Value = Iter.Value();

				HeaderMap.insert(std::pair<CefString, CefString>(TCHAR_TO_WCHAR(*Header), TCHAR_TO_WCHAR(*Value)));
			}
		}

		TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();

		if (BrowserWindow.IsValid())
		{
			TOptional<FString> Contents = BrowserWindow->GetResourceContent(Frame, Request);
			if (Contents.IsSet())
			{
				// pass the text we'd like to come back as a response to the request post data
				CefRefPtr<CefPostData> PostData = CefPostData::Create();
				CefRefPtr<CefPostDataElement> Element = CefPostDataElement::Create();
				FTCHARToUTF8 UTF8String(*Contents.GetValue());
				Element->SetToBytes(UTF8String.Length(), UTF8String.Get());
				PostData->AddElement(Element);
				Request->SetPostData(PostData);

				// Set a custom request header, so we know the mime type if it was specified as a hash on the dummy URL
				std::string Url = Request->GetURL().ToString();
				std::string::size_type HashPos = Url.find_last_of('#');
				if (HashPos != std::string::npos)
				{
					std::string MimeType = Url.substr(HashPos + 1);
					HeaderMap.insert(std::pair<CefString, CefString>(TCHAR_TO_WCHAR(TEXT("Content-Type")), MimeType));
				}

				// Change http method to tell GetResourceHandler to return the content
				Request->SetMethod(TCHAR_TO_WCHAR(*CustomContentMethod));
			}
		}

		if (Request->IsReadOnly())
		{
			LOG_CEF_LOAD("FCEFBrowserHandler::OnBeforeResourceLoad - readonly");
		}
		else
		{
			Request->SetHeaderMap(HeaderMap);
		}
		Callback->Continue();
	}));

	// Tell CEF that we're handling this asynchronously.
	return RV_CONTINUE_ASYNC;
}

void FCEFBrowserHandler::OnResourceLoadComplete(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefRequest> Request, CefRefPtr<CefResponse> Response, URLRequestStatus Status, int64_t Received_content_length)
{
	LOG_CEF_LOAD("FCEFBrowserHandler::OnResourceLoadComplete");

	// Current thread is IO thread. We need to invoke our delegates on the UI (aka Game) thread:
	CefPostTask(TID_UI, new FCEFBrowserClosureTask(this, [=, this]()
	{
		auto resType = Request->GetResourceType();
		const FString URL = WCHAR_TO_TCHAR(Request->GetURL().ToWString().c_str());
		if (MainFrameLoadTypes.Contains(URL))
		{
			// CEF has a bug where it confuses a MAIN_FRAME load for a XHR one, so fix it up here if we detect it.
			resType = CefRequest::ResourceType::RT_MAIN_FRAME;
		}
		ResourceLoadCompleteDelegate.ExecuteIfBound(Request->GetURL(), resType, Status, Received_content_length);

		// this load is done, clear the request from our map
		MainFrameLoadTypes.Remove(URL);
	}));
}

void FCEFBrowserHandler::OnResourceRedirect(CefRefPtr<CefBrowser> browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefRequest> Request, CefRefPtr<CefResponse> Response, CefString& new_url)
{
	LOG_CEF_LOAD("FCEFBrowserHandler::OnResourceRedirect");
	// Current thread is IO thread. We need to invoke our delegates on the UI (aka Game) thread:
	CefPostTask(TID_UI, new FCEFBrowserClosureTask(this, [=, this]()
	{
		// this load is effectively done, clear the request from our map
		MainFrameLoadTypes.Remove(WCHAR_TO_TCHAR(Request->GetURL().ToWString().c_str()));
	}));
}


void FCEFBrowserHandler::OnRenderProcessTerminated(CefRefPtr<CefBrowser> Browser, TerminationStatus Status, int error_code, const CefString& error_string)
{
	TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();
	if (BrowserWindow.IsValid())
	{
		BrowserWindow->OnRenderProcessTerminated(Status);
	}
}

bool FCEFBrowserHandler::OnBeforeBrowse(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefRequest> Request, bool user_gesture, bool IsRedirect)
{
	CefRequest::ResourceType RequestType = Request->GetResourceType();
	// We only want to append Authorization headers to main frame and similar requests
	// BUGBUG - in theory we want to support XHR requests that have the access-control-allow-credentials header but CEF doesn't give us preflight details here
	if (RequestType == CefRequest::ResourceType::RT_MAIN_FRAME || RequestType == CefRequest::ResourceType::RT_SUB_FRAME || RequestType == CefRequest::ResourceType::RT_SUB_RESOURCE)
	{
		// record that we saw this URL request as a main frame load
		MainFrameLoadTypes.Add(WCHAR_TO_TCHAR(Request->GetURL().ToWString().c_str()), RequestType);
	}

	// Current thread: UI thread
	TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();
	if (BrowserWindow.IsValid())
	{
		LOG_CEF_LOAD("FCEFBrowserHandler::OnBeforeBrowse");
		if (BrowserWindow->OnBeforeBrowse(Browser, Frame, Request, user_gesture, IsRedirect))
		{
			return true;
		}
	}

	return false;
}

CefRefPtr<CefResourceHandler> FCEFBrowserHandler::GetResourceHandler(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefRequest> Request)
{

	if (Request->GetMethod() == TCHAR_TO_WCHAR(*CustomContentMethod))
	{
		// Content override header will be set by OnBeforeResourceLoad before passing the request on to this.
		if (Request->GetPostData() && Request->GetPostData()->GetElementCount() > 0)
		{
			// get the mime type from Content-Type header (default to text/html to support old behavior)
			FString MimeType = "text/html"; // default if not specified
			CefRequest::HeaderMap HeaderMap;
			Request->GetHeaderMap(HeaderMap);
			auto ContentOverride = HeaderMap.find(TCHAR_TO_WCHAR(TEXT("Content-Type")));
			if (ContentOverride != HeaderMap.end())
			{
				MimeType = WCHAR_TO_TCHAR(ContentOverride->second.ToWString().c_str());
			}

			// reply with the post data
			CefPostData::ElementVector Elements;
			Request->GetPostData()->GetElements(Elements);
			return new FCEFBrowserByteResource(Elements[0], MimeType);
		}
	}
	return nullptr;
}

CefRefPtr<CefResourceRequestHandler> FCEFBrowserHandler::GetResourceRequestHandler(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefRequest> Request, bool is_navigation, bool is_download, const CefString& request_initiator, bool& disable_default_handling)
{
	LOG_CEF_LOAD("FCEFBrowserHandler::GetResourceRequestHandler");
	if (bInterceptLoadRequests)
	{
		return this;
	}
	return nullptr;
}

void FCEFBrowserHandler::SetBrowserWindow(TSharedPtr<FCEFWebBrowserWindow> InBrowserWindow)
{
	BrowserWindowPtr = InBrowserWindow;

	if (InBrowserWindow.IsValid())
	{
		// Register any JS bindings that are setup in the new browser. In theory there should be 0 here as we are still being created.
		CefRefPtr<CefProcessMessage> SetValueMessage = CefProcessMessage::Create(TCHAR_TO_WCHAR(TEXT("CEF::STARTUP")));
		CefRefPtr<CefListValue> MessageArguments = SetValueMessage->GetArgumentList();
		CefRefPtr<CefDictionaryValue> Bindings = InBrowserWindow->GetProcessInfo();
		if (Bindings.get())
		{
			MessageArguments->SetDictionary(0, Bindings);
		}
		InBrowserWindow->GetCefBrowser()->GetMainFrame()->SendProcessMessage(PID_RENDERER, SetValueMessage);
	}
}

bool FCEFBrowserHandler::OnProcessMessageReceived(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefProcessId SourceProcess, CefRefPtr<CefProcessMessage> Message)
{
	if (IWebBrowserModule::Get().GetSingleton()->IsShuttingDown())
	{
		UE_LOGF(LogWebBrowser, Warning, "Received a process message while the module is shutting down, ignoring.");
		return false;
	}

	bool Retval = false;
	FString MessageName = WCHAR_TO_TCHAR(Message->GetName().ToWString().c_str());
	TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();
	if (BrowserWindow.IsValid())
	{
		if (MessageName.StartsWith(TEXT("CEF::BROWSERCREATED")))
		{
			// Register any JS bindings that are setup in the new browser. In theory there should be 0 here as we are still being created.
			CefRefPtr<CefProcessMessage> SetValueMessage = CefProcessMessage::Create(TCHAR_TO_WCHAR(TEXT("CEF::STARTUP")));
			CefRefPtr<CefListValue> MessageArguments = SetValueMessage->GetArgumentList();
			CefRefPtr<CefDictionaryValue> Bindings = BrowserWindow->GetProcessInfo();
			if (Bindings.get())
			{
				MessageArguments->SetDictionary(0, Bindings);
			}
			// CEF has a race condition for newly constructed browser objects, we may route this to the wrong renderer if we send right away
			// so just PostTake to send this message next frame
			CefPostTask(TID_UI, new FCEFBrowserClosureTask(this, [=]()
				{
					Frame->SendProcessMessage(PID_RENDERER, SetValueMessage);
				}));

		}
		else
		{
			Retval = BrowserWindow->OnProcessMessageReceived(Browser, Frame, SourceProcess, Message);
		}
	}
	return Retval;
}

bool FCEFBrowserHandler::ShowDevTools(const CefRefPtr<CefBrowser>& Browser)
{
	CefPoint Point;
	CefString TargetUrl = "chrome-devtools://devtools/devtools.html";
	CefString TargetFrameName = "devtools";
	CefPopupFeatures PopupFeatures;
	CefWindowInfo WindowInfo;
	CefRefPtr<CefClient> NewClient;
	CefBrowserSettings BrowserSettings;
	bool NoJavascriptAccess = false;

	PopupFeatures.xSet = false;
	PopupFeatures.ySet = false;
	PopupFeatures.heightSet = false;
	PopupFeatures.widthSet = false;
	PopupFeatures.isPopup = true;

	// Set max framerate to maximum supported.
	BrowserSettings.windowless_frame_rate = 60;
	// Dev Tools look best with a white background color
	BrowserSettings.background_color = CefColorSetARGB(255, 255, 255, 255);

	// OnBeforePopup already takes care of all the details required to ask the host application to create a new browser window.
	bool bSuppressWindowCreation = OnBeforePopup(Browser, Browser->GetFocusedFrame(), TargetUrl, TargetFrameName, PopupFeatures, WindowInfo, NewClient, BrowserSettings, &NoJavascriptAccess);

	if (!bSuppressWindowCreation)
	{
		Browser->GetHost()->ShowDevTools(WindowInfo, NewClient, BrowserSettings, Point);
	}
	return !bSuppressWindowCreation;
}

bool FCEFBrowserHandler::OnKeyEvent(CefRefPtr<CefBrowser> Browser, const CefKeyEvent& Event, CefEventHandle OsEvent)
{
	// Show dev tools on CMD/CTRL+SHIFT+I
	if ( (Event.type == KEYEVENT_RAWKEYDOWN || Event.type == KEYEVENT_KEYDOWN || Event.type == KEYEVENT_CHAR) &&
#if PLATFORM_MAC
		(Event.modifiers == (EVENTFLAG_COMMAND_DOWN | EVENTFLAG_SHIFT_DOWN)) &&
#else
		(Event.modifiers == (EVENTFLAG_CONTROL_DOWN | EVENTFLAG_SHIFT_DOWN)) &&
#endif
		(Event.windows_key_code == 'I' ||
		Event.unmodified_character == 'i' || Event.unmodified_character == 'I') &&
		IWebBrowserModule::Get().GetSingleton()->IsDevToolsShortcutEnabled()
	  )
	{
		return ShowDevTools(Browser);
	}

#if PLATFORM_MAC
	// We need to handle standard Copy/Paste/etc... shortcuts on OS X
	if ( (Event.type == KEYEVENT_RAWKEYDOWN || Event.type == KEYEVENT_KEYDOWN) &&
		(Event.modifiers & EVENTFLAG_COMMAND_DOWN) != 0 &&
		(Event.modifiers & EVENTFLAG_CONTROL_DOWN) == 0 &&
		(Event.modifiers & EVENTFLAG_ALT_DOWN) == 0 &&
		( (Event.modifiers & EVENTFLAG_SHIFT_DOWN) == 0 || Event.unmodified_character == 'z' )
	  )
	{
		CefRefPtr<CefFrame> Frame = Browser->GetFocusedFrame();
		if (Frame)
		{
			switch (Event.unmodified_character)
			{
				case 'a':
					Frame->SelectAll();
					return true;
				case 'c':
					Frame->Copy();
					return true;
				case 'v':
					Frame->Paste();
					return true;
				case 'x':
					Frame->Cut();
					return true;
				case 'z':
					if ( (Event.modifiers & EVENTFLAG_SHIFT_DOWN) == 0 )
					{
						Frame->Undo();
					}
					else
					{
						Frame->Redo();
					}
					return true;
			}
		}
	}
#endif
	TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();
	if (BrowserWindow.IsValid())
	{
		return BrowserWindow->OnUnhandledKeyEvent(Event);
	}

	return false;
}

bool FCEFBrowserHandler::OnJSDialog(CefRefPtr<CefBrowser> Browser, const CefString& OriginUrl, JSDialogType DialogType, const CefString& MessageText, const CefString& DefaultPromptText, CefRefPtr<CefJSDialogCallback> Callback, bool& OutSuppressMessage)
{
	bool Retval = false;
	TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();
	if (BrowserWindow.IsValid())
	{
		Retval = BrowserWindow->OnJSDialog(DialogType, MessageText, DefaultPromptText, Callback, OutSuppressMessage);
	}
	return Retval;
}

bool FCEFBrowserHandler::OnBeforeUnloadDialog(CefRefPtr<CefBrowser> Browser, const CefString& MessageText, bool IsReload, CefRefPtr<CefJSDialogCallback> Callback)
{
	bool Retval = false;
	TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();
	if (BrowserWindow.IsValid())
	{
		Retval = BrowserWindow->OnBeforeUnloadDialog(MessageText, IsReload, Callback);
	}
	return Retval;
}

void FCEFBrowserHandler::OnResetDialogState(CefRefPtr<CefBrowser> Browser)
{
	TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();
	if (BrowserWindow.IsValid())
	{
		BrowserWindow->OnResetDialogState();
	}
}

void FCEFBrowserHandler::OnBeforeContextMenu(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefContextMenuParams> Params, CefRefPtr<CefMenuModel> Model)
{
	TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();
	if (BrowserWindow.IsValid() && BrowserWindow->OnSuppressContextMenu().IsBound() && BrowserWindow->OnSuppressContextMenu().Execute())
	{
		Model->Clear();
	}
}

void FCEFBrowserHandler::OnDraggableRegionsChanged(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> frame, const std::vector<CefDraggableRegion>& Regions)
{
	TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();
	if (BrowserWindow.IsValid())
	{
		TArray<FWebBrowserDragRegion> DragRegions;
		for (uint32 Idx = 0; Idx < Regions.size(); Idx++)
		{
			DragRegions.Add(FWebBrowserDragRegion(
				FIntRect(Regions[Idx].bounds.x, Regions[Idx].bounds.y, Regions[Idx].bounds.x + Regions[Idx].bounds.width, Regions[Idx].bounds.y + Regions[Idx].bounds.height),
				Regions[Idx].draggable ? true : false));
		}
		BrowserWindow->UpdateDragRegions(DragRegions);
	}
}

bool FCEFBrowserHandler::OnFileDialog(CefRefPtr<CefBrowser> Browser, FileDialogMode Mode, const CefString& Title, const CefString& DefaultFilePath, const std::vector<CefString>& AcceptFilters, const std::vector<CefString>& AcceptExtensions, const std::vector<CefString>& AcceptDescriptions, CefRefPtr<CefFileDialogCallback> Callback)
{
	bool Retval = false;
	TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();
	if (BrowserWindow.IsValid())
	{
		Retval = BrowserWindow->OnFileDialog(Mode, Title, DefaultFilePath, AcceptFilters,
			AcceptExtensions,
			AcceptDescriptions,
			Callback);
	}

	return Retval;
}	

CefRefPtr<CefCookieAccessFilter> FCEFBrowserHandler::GetCookieAccessFilter(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefRequest> Request)
{
	FString Url = WCHAR_TO_TCHAR(Request->GetURL().ToWString().c_str());
	TArray<FString> UrlParts;
	if (Url.ParseIntoArray(UrlParts, TEXT("/"), true) >= 2)
	{
		if (UrlParts[1].Contains(TEXT(".epicgames.com")) || UrlParts[1].Contains(TEXT(".epicgames.net")))
		{
			// We only support custom cookie alteration for the epicgames domains right now. 
			// There are limitations/bugs in CEF when the cookie filtering it on making it fail to pass cookies for some requests, so 
			// we want to limit the scope of the filtering. See https://jira.it.epicgames.com/browse/DISTRO-1847 as an example of a bug
			// caused by filtering
			return this;
		}
	}

	return nullptr;
}

bool FCEFBrowserHandler::CanSaveCookie(CefRefPtr<CefBrowser> browser, CefRefPtr<CefFrame> frame, CefRefPtr<CefRequest> request, CefRefPtr<CefResponse> response, const CefCookie& cookie)
{
	if (bAllowAllCookies)
	{
		return true;
	}

	// these two cookies shouldn't be saved by the client. While we are debugging why the backend is causing them to be set filter them out
	if (CefString(&cookie.name).ToString() == "store-token" || CefString(&cookie.name) == "EPIC_SESSION_DIESEL")
	{
		return false;
	}

	return true;
}

bool FCEFBrowserHandler::CanSendCookie(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefRequest> Request, const CefCookie& Cookie)
{
	if (bAllowAllCookies)
	{
		return true;
	}

	FString RequestURL(WCHAR_TO_TCHAR(Request->GetURL().ToWString().c_str()));
	FString ReffererURL(WCHAR_TO_TCHAR(Request->GetReferrerURL().ToWString().c_str()));
	if (ReffererURL.Contains("marketplace-website-node-launcher-") && RequestURL.Contains("graphql.epicgames.com"))
	{
		// requests from the marketplace UE4 page to graphql can exceed the header size limits so manually prune this large cookie here
		if (CefString(&Cookie.name).ToString() == "ecma")
		{
			return false;
		}
	}

	return true;
}


bool FCEFBrowserHandler::URLRequestAllowsCredentials(const FString& URL) const
{
	// if we inserted this URL into our map then we want to allow credentials for it
	if (MainFrameLoadTypes.Find(URL) != nullptr)
	{
		return true;
	}

	// check the explicit allowlist also
	for (const FString& AuthorizationHeaderAllowListURL : AuthorizationHeaderAllowListURLS)
	{
		if (URL.Contains(AuthorizationHeaderAllowListURL))
		{
			return true;
		}
	}
	return false;
}


#undef LOCTEXT_NAMESPACE

#endif // WITH_CEF
