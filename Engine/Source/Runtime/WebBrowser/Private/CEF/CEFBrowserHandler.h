// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_CEF3


#include "CEFLibCefIncludes.h"


#include "IWebBrowserWindow.h"

#endif

class IWebBrowserWindow;
struct Rect;
class FCEFWebBrowserWindow;
class FCEFBrowserPopupFeatures;

#if WITH_CEF3

/**
 * Implements CEF Client and other Browser level interfaces.
 */
class FCEFBrowserHandler
	: public CefClient
	, public CefDisplayHandler
	, public CefLifeSpanHandler
	, public CefLoadHandler
	, public CefRenderHandler
	, public CefRequestHandler
	, public CefKeyboardHandler
	, public CefJSDialogHandler
	, public CefContextMenuHandler
	, public CefDragHandler
	, public CefResourceRequestHandler
	, public CefRequestContextHandler
	, public CefCookieAccessFilter
	, public CefDialogHandler
{
public:

	/** Default constructor. */
	FCEFBrowserHandler(bool InUseTransparency, bool InInterceptLoadRequests, const TArray<FString>& AltRetryDomains = TArray<FString>(), const TArray<FString>& AuthorizationHeaderAllowListURLS = TArray<FString>());

	/**
	 * Pass in a pointer to our Browser Window so that events can be passed on.
	 *
	 * @param InBrowserWindow The browser window this will be handling.
	 */
	void SetBrowserWindow(TSharedPtr<FCEFWebBrowserWindow> InBrowserWindow);
	
	/**
	 * Sets the browser window features and settings for popups which will be passed along when creating the new window.
	 *
	 * @param InPopupFeatures The popup features and settings for the window.
	 */
	void SetPopupFeatures(const TSharedPtr<FCEFBrowserPopupFeatures>& InPopupFeatures)
	{
		BrowserPopupFeatures = InPopupFeatures;
	}

	// CefClient Interface
	virtual CefRefPtr<CefDisplayHandler> GetDisplayHandler() override { return this; }
	virtual CefRefPtr<CefLifeSpanHandler> GetLifeSpanHandler() override { return this; }
	virtual CefRefPtr<CefLoadHandler> GetLoadHandler() override { return this; }
	virtual CefRefPtr<CefRenderHandler> GetRenderHandler() override { return this; }
	virtual CefRefPtr<CefRequestHandler> GetRequestHandler() override { return this; }
	virtual CefRefPtr<CefKeyboardHandler> GetKeyboardHandler() override { return this; }
	virtual CefRefPtr<CefJSDialogHandler> GetJSDialogHandler() override { return this; }
	virtual CefRefPtr<CefContextMenuHandler> GetContextMenuHandler() override { return this; }
	virtual CefRefPtr<CefDragHandler> GetDragHandler() override { return this; }
	virtual CefRefPtr<CefDialogHandler> GetDialogHandler() override { return this; }
	virtual CefRefPtr<CefCookieAccessFilter> GetCookieAccessFilter(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefRequest> Request) override;
	virtual bool OnProcessMessageReceived(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> frame, CefProcessId SourceProcess, CefRefPtr<CefProcessMessage> Message) override;

	// CefDisplayHandler Interface
	virtual void OnTitleChange(CefRefPtr<CefBrowser> Browser, const CefString& Title) override;
	virtual void OnAddressChange(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, const CefString& Url) override;
	virtual bool OnTooltip(CefRefPtr<CefBrowser> Browser, CefString& Text) override;
	virtual bool OnConsoleMessage(CefRefPtr<CefBrowser> Browser, cef_log_severity_t level, const CefString& Message, const CefString& Source, int Line) override;

	// CefLifeSpanHandler Interface
	virtual void OnAfterCreated(CefRefPtr<CefBrowser> Browser) override;
	virtual bool DoClose(CefRefPtr<CefBrowser> Browser) override;
	virtual void OnBeforeClose(CefRefPtr<CefBrowser> Browser) override;
	virtual bool OnBeforePopup(CefRefPtr<CefBrowser> Browser,
		CefRefPtr<CefFrame> Frame,
#if CEF_VERSION_MAJOR >= 147
		int /*Popup_Id*/,
#endif
		const CefString& Target_Url,
		const CefString& Target_Frame_Name,
		CefLifeSpanHandler::WindowOpenDisposition /* Target_Disposition */,
		bool /* User_Gesture */,
		const CefPopupFeatures& PopupFeatures,
		CefWindowInfo& WindowInfo,
		CefRefPtr<CefClient>& Client,
		CefBrowserSettings& Settings,
		CefRefPtr<CefDictionaryValue>& extra_info,
		bool* no_javascript_access) override
	{
		return OnBeforePopup(Browser, Frame, Target_Url, Target_Frame_Name, PopupFeatures, WindowInfo, Client, Settings, no_javascript_access);
	}
	virtual bool OnBeforePopup(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, const CefString& Target_Url, const CefString& Target_Frame_Name, const CefPopupFeatures& PopupFeatures, CefWindowInfo& WindowInfo, CefRefPtr<CefClient>& Client, CefBrowserSettings& Settings, bool* no_javascript_access);

	// CefLoadHandler Interface
	virtual void OnLoadError(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefLoadHandler::ErrorCode InErrorCode, const CefString& ErrorText, const CefString& FailedUrl) override;
	virtual void OnLoadingStateChange(CefRefPtr<CefBrowser> browser, bool isLoading, bool canGoBack, bool canGoForward) override;
	virtual void OnLoadStart(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, TransitionType CefTransitionType) override;

	// CefRenderHandler Interface
	virtual bool GetRootScreenRect(CefRefPtr<CefBrowser> Browser, CefRect& Rect) override;
	virtual void GetViewRect(CefRefPtr<CefBrowser> Browser, CefRect& Rect) override;
	virtual void OnPaint(CefRefPtr<CefBrowser> Browser, PaintElementType Type, const RectList& DirtyRects, const void* Buffer, int Width, int Height) override;
	virtual void OnAcceleratedPaint(CefRefPtr<CefBrowser> Browser, PaintElementType Type, const RectList& DirtyRects, const CefAcceleratedPaintInfo& SharedHandle) override;
	virtual void OnPopupShow(CefRefPtr<CefBrowser> Browser, bool bShow) override;
	virtual void OnPopupSize(CefRefPtr<CefBrowser> Browser, const CefRect& Rect) override;
	virtual bool GetScreenInfo(CefRefPtr<CefBrowser> Browser, CefScreenInfo& ScreenInfo) override;

	// CefDisplayHandler interface
	virtual bool OnCursorChange(CefRefPtr<CefBrowser> browser, CefCursorHandle cursor, cef_cursor_type_t type, const CefCursorInfo& custom_cursor_info) override;
#if !PLATFORM_LINUX
	virtual void OnImeCompositionRangeChanged(CefRefPtr<CefBrowser> Browser, const CefRange& SelectionRange, const CefRenderHandler::RectList& CharacterBounds) override;
	virtual void OnTextSelectionChanged(CefRefPtr<CefBrowser> Browser, const CefString& SelectedText, const CefRange& SelectionRange) override;
#endif

	// CefRequestHandler Interface
	virtual CefResourceRequestHandler::ReturnValue OnBeforeResourceLoad(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefRequest> Request, CefRefPtr<CefCallback> Callback) override;
	virtual void OnResourceLoadComplete(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefRequest> Request, CefRefPtr<CefResponse> Response, URLRequestStatus Status, int64_t Received_content_length) override;
	virtual void OnResourceRedirect(CefRefPtr<CefBrowser> browser, CefRefPtr<CefFrame> frame, CefRefPtr<CefRequest> request, CefRefPtr<CefResponse> response, CefString& new_url) override;
	virtual void OnRenderProcessTerminated(CefRefPtr<CefBrowser> Browser, TerminationStatus Status, int error_code, const CefString& error_string) override;
	virtual bool OnBeforeBrowse(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefRequest> Request, bool user_gesture, bool IsRedirect) override;
	virtual CefRefPtr<CefResourceHandler> GetResourceHandler(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefRequest> Request) override;
	virtual bool OnCertificateError(CefRefPtr<CefBrowser> Browser, cef_errorcode_t CertError, const CefString& RequestUrl, CefRefPtr<CefSSLInfo> SslInfo, CefRefPtr<CefCallback> Callback) override;
	virtual CefRefPtr<CefResourceRequestHandler> GetResourceRequestHandler(CefRefPtr<CefBrowser> browser, CefRefPtr<CefFrame> frame, CefRefPtr<CefRequest> request, bool is_navigation, bool is_download, const CefString& request_initiator, bool& disable_default_handling) override;

	// CefKeyboardHandler interface
	virtual bool OnKeyEvent(CefRefPtr<CefBrowser> Browser, const CefKeyEvent& Event, CefEventHandle OsEvent) override;

	// CefCookieAccessFilter interface
	virtual bool CanSaveCookie(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefRequest> Request, CefRefPtr<CefResponse> Response, const CefCookie& Cookie) override;
	virtual bool CanSendCookie(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefRequest> Request, const CefCookie& Cookie) override;

	// CefJSDialogHandler interface
	virtual bool OnJSDialog(CefRefPtr<CefBrowser> Browser, const CefString& OriginUrl, JSDialogType DialogType, const CefString& MessageText, const CefString& DefaultPromptText, CefRefPtr<CefJSDialogCallback> Callback, bool& OutSuppressMessage) override;
	virtual bool OnBeforeUnloadDialog(CefRefPtr<CefBrowser> Browser, const CefString& MessageText, bool IsReload, CefRefPtr<CefJSDialogCallback> Callback) override;
	virtual void OnResetDialogState(CefRefPtr<CefBrowser> Browser) override;

	// CefContextMenuHandler interface
	virtual void OnBeforeContextMenu(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefContextMenuParams> Params, CefRefPtr<CefMenuModel> Model) override;

	// CefDragHandler interface
	virtual void OnDraggableRegionsChanged(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> frame, const std::vector<CefDraggableRegion>& Regions) override;

	// CefDialogHandler interface
	virtual bool OnFileDialog(CefRefPtr<CefBrowser> Browser, FileDialogMode Mode, const CefString& Title, const CefString& DefaultFilePath, const std::vector<CefString>& AcceptFilters, const std::vector<CefString>& AcceptExtensions, const std::vector<CefString>& AcceptDescriptions, CefRefPtr<CefFileDialogCallback> Callback) override;


	IWebBrowserWindow::FOnBeforePopupDelegate& OnBeforePopup()
	{
		return BeforePopupDelegate;
	}

	IWebBrowserWindow::FOnCreateWindow& OnCreateWindow()
	{
		return CreateWindowDelegate;
	}

	typedef TMap<FString, FString> FRequestHeaders;
	DECLARE_DELEGATE_FourParams(FOnBeforeResourceLoadDelegate, const CefString& /*URL*/, CefRequest::ResourceType /*Type*/, FRequestHeaders& /*AdditionalHeaders*/, const bool /*AllowUserCredentials*/);
	FOnBeforeResourceLoadDelegate& OnBeforeResourceLoad()
	{
		return BeforeResourceLoadDelegate;
	}

	DECLARE_DELEGATE_FourParams(FOnResourceLoadCompleteDelegate, const CefString& /*URL*/, CefRequest::ResourceType /*Type*/, CefResourceRequestHandler::URLRequestStatus /*Status*/, int64 /*ContentLength*/);
	FOnResourceLoadCompleteDelegate& OnResourceLoadComplete()
	{
		return ResourceLoadCompleteDelegate;
	}

	DECLARE_DELEGATE_FiveParams(FOnConsoleMessageDelegate, CefRefPtr<CefBrowser> /*Browser*/, cef_log_severity_t /*level*/, const CefString& /*Message*/, const CefString& /*Source*/, int32 /*Line*/);
	FOnConsoleMessageDelegate& OnConsoleMessage()
	{
		return ConsoleMessageDelegate;
	}

	bool URLRequestAllowsCredentials(const FString& URL) const;

private:

	bool ShowDevTools(const CefRefPtr<CefBrowser>& Browser);

	bool bUseTransparency;
	bool bAllowAllCookies;
	bool bInterceptLoadRequests;

	TArray<FString> AltRetryDomains;
	uint32 AltRetryDomainIdx = 0;

	/** Domains we allow sending an authorization header too even if the request doesn't otherwise indicate support */
	TArray<FString> AuthorizationHeaderAllowListURLS;

	/** Keep track of URLs we see being loaded and the type of load it is*/
	TMap<FString, CefRequest::ResourceType> MainFrameLoadTypes;

	/** Delegate for notifying that a popup window is attempting to open. */
	IWebBrowserWindow::FOnBeforePopupDelegate BeforePopupDelegate;
	
	/** Delegate for handling requests to create new windows. */
	IWebBrowserWindow::FOnCreateWindow CreateWindowDelegate;

	/** Delegate for handling adding additional headers to requests */
	FOnBeforeResourceLoadDelegate BeforeResourceLoadDelegate;

	/** Delegate that allows response to the status of resource loads */
	FOnResourceLoadCompleteDelegate ResourceLoadCompleteDelegate;

	/** Delegate that allows for response to console logs.  Typically used to capture and mirror web logs in client application logs. */
	FOnConsoleMessageDelegate ConsoleMessageDelegate;

	/** Weak Pointer to our Web Browser window so that events can be passed on while it's valid.*/
	TWeakPtr<FCEFWebBrowserWindow> BrowserWindowPtr;
	
	/** Pointer to the parent web browser handler */
	CefRefPtr<FCEFBrowserHandler> ParentHandler;

	/** Stores popup window features and settings */
	TSharedPtr<FCEFBrowserPopupFeatures> BrowserPopupFeatures;

	// Include the default reference counting implementation.
	IMPLEMENT_REFCOUNTING(FCEFBrowserHandler);
};

#endif
