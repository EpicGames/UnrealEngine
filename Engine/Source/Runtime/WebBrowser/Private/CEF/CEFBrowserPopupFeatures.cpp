// Copyright Epic Games, Inc. All Rights Reserved.

#include "CEF/CEFBrowserPopupFeatures.h"

#if WITH_CEF3

FCEFBrowserPopupFeatures::FCEFBrowserPopupFeatures()
	: X(0)
	, bXSet(false)
	, Y(0)
	, bYSet(false)
	, Width(0)
	, bWidthSet(false)
	, Height(0)
	, bHeightSet(false)
	, bMenuBarVisible(true)
	, bStatusBarVisible(false)
	, bToolBarVisible(true)
	, bLocationBarVisible(true)
	, bScrollbarsVisible(true)
	, bResizable(true)
	, bIsFullscreen(false)
	, bIsDialog(false)
{
}


FCEFBrowserPopupFeatures::FCEFBrowserPopupFeatures(const CefPopupFeatures& PopupFeatures)
{
	X = PopupFeatures.x;
	bXSet = PopupFeatures.xSet ? true : false;
	Y = PopupFeatures.y;
	bYSet = PopupFeatures.ySet ? true : false;
	Width = PopupFeatures.width;
	bWidthSet = PopupFeatures.widthSet ? true : false;
	Height = PopupFeatures.height;
	bHeightSet = PopupFeatures.heightSet ? true : false;
	bMenuBarVisible = PopupFeatures.isPopup ? false : true;
	bStatusBarVisible = PopupFeatures.isPopup ? false : true;
	bToolBarVisible = PopupFeatures.isPopup ? false : true;
	bScrollbarsVisible = PopupFeatures.isPopup ? false : true;
	
	// no longer set by the CEF API so default them here to their historic value
	bLocationBarVisible = false;
	bResizable = false;
	bIsFullscreen = false;
	bIsDialog = false;
}

FCEFBrowserPopupFeatures::~FCEFBrowserPopupFeatures()
{
}

void FCEFBrowserPopupFeatures::SetResizable(const bool bResize)
{
	bResizable = bResize;
}

int FCEFBrowserPopupFeatures::GetX() const
{
	return X;
}

bool FCEFBrowserPopupFeatures::IsXSet() const
{
	return bXSet;
}

int FCEFBrowserPopupFeatures::GetY() const
{
	return Y;
}

bool FCEFBrowserPopupFeatures::IsYSet() const
{
	return bYSet;
}

int FCEFBrowserPopupFeatures::GetWidth() const
{
	return Width;
}

bool FCEFBrowserPopupFeatures::IsWidthSet() const
{
	return bWidthSet;
}

int FCEFBrowserPopupFeatures::GetHeight() const
{
	return Height;
}

bool FCEFBrowserPopupFeatures::IsHeightSet() const
{
	return bHeightSet;
}

bool FCEFBrowserPopupFeatures::IsMenuBarVisible() const
{
	return bMenuBarVisible;
}

bool FCEFBrowserPopupFeatures::IsStatusBarVisible() const
{
	return bStatusBarVisible;
}

bool FCEFBrowserPopupFeatures::IsToolBarVisible() const
{
	return bToolBarVisible;
}

bool FCEFBrowserPopupFeatures::IsLocationBarVisible() const
{
	return bLocationBarVisible;
}

bool FCEFBrowserPopupFeatures::IsScrollbarsVisible() const
{
	return bScrollbarsVisible;
}

bool FCEFBrowserPopupFeatures::IsResizable() const
{
	return bResizable;
}

bool FCEFBrowserPopupFeatures::IsFullscreen() const
{
	return bIsFullscreen;
}

bool FCEFBrowserPopupFeatures::IsDialog() const
{
	return bIsDialog;
}

TArray<FString> FCEFBrowserPopupFeatures::GetAdditionalFeatures() const
{
	TArray<FString> Empty;
	return Empty;
}

#endif
