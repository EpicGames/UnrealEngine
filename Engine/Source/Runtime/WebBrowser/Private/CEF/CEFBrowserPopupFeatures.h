// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_CEF3

#include "IWebBrowserPopupFeatures.h"

#include "CEFLibCefIncludes.h"

#endif

class IWebBrowserPopupFeatures;

#if WITH_CEF3

class FCEFBrowserPopupFeatures
	: public IWebBrowserPopupFeatures
{
public:
	FCEFBrowserPopupFeatures();
	FCEFBrowserPopupFeatures(const CefPopupFeatures& PopupFeatures);
	virtual ~FCEFBrowserPopupFeatures();
	void SetResizable(const bool bResize);

	// IWebBrowserPopupFeatures Interface

	virtual int GetX() const override;
	virtual bool IsXSet() const override;
	virtual int GetY() const override;
	virtual bool IsYSet() const override;
	virtual int GetWidth() const override;
	virtual bool IsWidthSet() const override;
	virtual int GetHeight() const override;
	virtual bool IsHeightSet() const override;
	virtual bool IsMenuBarVisible() const override;
	virtual bool IsStatusBarVisible() const override;
	virtual bool IsToolBarVisible() const override;
	virtual bool IsLocationBarVisible() const override;
	virtual bool IsScrollbarsVisible() const override;
	virtual bool IsResizable() const override;
	virtual bool IsFullscreen() const override;
	virtual bool IsDialog() const override;
	virtual TArray<FString> GetAdditionalFeatures() const override;

private:

	int X;
	bool bXSet;
	int Y;
	bool bYSet;
	int Width;
	bool bWidthSet;
	int Height;
	bool bHeightSet;

	bool bMenuBarVisible;
	bool bStatusBarVisible;
	bool bToolBarVisible;
	bool bLocationBarVisible;
	bool bScrollbarsVisible;
	bool bResizable;

	bool bIsFullscreen;
	bool bIsDialog;
};

#endif
