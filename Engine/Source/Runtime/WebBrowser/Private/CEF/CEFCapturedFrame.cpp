// Copyright Epic Games, Inc. All Rights Reserved.


#include "CEFCapturedFrame.h"

#if WITH_CEF3

#include "Textures/SlateTextureData.h"

static constexpr uint32 BUFFER_ELEMENT_SIZE = 4; /*PF_B8G8R8A8*/

bool FCEFCapturedFrame::SetBufferAsB8G8R8A8(const void* InBufferB8G8R8A8, const int32 InBufferWidth, const int32 InBufferHeight, const bool bSkipTransparencyTest)
{
	if (!InBufferB8G8R8A8 || InBufferWidth <= 0 || InBufferHeight <= 0)
	{
		ClearBuffer();
		return false;
	}

	// When resizing, CEF can sometimes send us buffers with the contents stretched/compressed inside the frame and surrounded by transparent bars.
	// We detect this by sampling the alpha channel of the first pixel, in the top-left corner. If it holds a 0x0 then the frame is bad and should be skipped.
	if (!bSkipTransparencyTest && (static_cast<const uint8*>(InBufferB8G8R8A8)[3] == 0x0))
	{
		return false;
	}

	// Don't capture anymore if we already have a buffer of the same size
	if (InBufferWidth != BufferDimensions.X || InBufferHeight != BufferDimensions.Y)
	{
		BufferDimensions = FUintPoint(InBufferWidth, InBufferHeight);

		const TArray<uint8>::SizeType NumBytes = BufferDimensions.X * BufferDimensions.Y * BUFFER_ELEMENT_SIZE;
		if (BufferData.Num() != NumBytes)
		{
			BufferData.SetNum(NumBytes);
		}
		FMemory::Memcpy(BufferData.GetData(), InBufferB8G8R8A8, NumBytes);
	}

	return true;
}

bool FCEFCapturedFrame::ClearBuffer()
{
	BufferData.Empty();
	BufferDimensions = FUintPoint::ZeroValue;
	return true;
}

bool FCEFCapturedFrame::IsEmpty() const
{
	return BufferData.IsEmpty();
}

FSlateTextureData* FCEFCapturedFrame::CreateSlateTextureData(const FIntPoint& TextureSize) const
{
	if (BufferData.IsEmpty() || TextureSize.X <= 0 || TextureSize.Y <= 0)
	{
		return new FSlateTextureData(nullptr, 0, 0, BUFFER_ELEMENT_SIZE);
	}

	const FUintPoint TextureSizeChecked = (FUintPoint)TextureSize;
	if (BufferDimensions == TextureSizeChecked)
	{
		return new FSlateTextureData(BufferData.GetData(), BufferDimensions.X, BufferDimensions.Y, BUFFER_ELEMENT_SIZE);
	}

	TArray<uint8> TextureDataBuffer;
	TextureDataBuffer.SetNum(TextureSizeChecked.X * TextureSizeChecked.Y * BUFFER_ELEMENT_SIZE);
	for (uint32 Row = 0, MaxRow = std::min(BufferDimensions.Y, TextureSizeChecked.Y); Row < MaxRow; ++Row)
	{
		const size_t SourceOffset = Row * BufferDimensions.X * BUFFER_ELEMENT_SIZE;
		const size_t TargetOffset = Row * TextureSizeChecked.X * BUFFER_ELEMENT_SIZE;
		FMemory::Memcpy(TextureDataBuffer.GetData() + TargetOffset, BufferData.GetData() + SourceOffset, std::min(BufferDimensions.X, TextureSizeChecked.X) * BUFFER_ELEMENT_SIZE);
	}

	return new FSlateTextureData(TextureSizeChecked.X, TextureSizeChecked.Y, BUFFER_ELEMENT_SIZE, std::move(TextureDataBuffer));
}

#endif // WITH_CEF3
