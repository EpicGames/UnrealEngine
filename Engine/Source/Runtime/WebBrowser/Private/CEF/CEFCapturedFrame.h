// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "CEFCapturedFrame.generated.h"

struct FSlateTextureData;

USTRUCT()
struct FCEFCapturedFrame
{
	GENERATED_BODY()

#if WITH_CEF3
public:

	bool SetBufferAsB8G8R8A8(const void* InBufferB8G8R8A8, const int32 InBufferWidth, const int32 InBufferHeight, const bool bSkipTransparencyTest);
	bool ClearBuffer();
	
	bool IsEmpty() const;

	FSlateTextureData* CreateSlateTextureData(const FIntPoint& TextureSize) const;

private:

	TArray<uint8> BufferData;
	FUintPoint BufferDimensions = FUintPoint::ZeroValue;
#endif // WITH_CEF3
};
