// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Containers/ContainersFwd.h"
#if WITH_CEF3
#include "IWebBrowserCookieManager.h"
#include "WebBrowserSingleton.h"

#include "CEFLibCefIncludes.h"

class FCefCookieManager
	: public IWebBrowserCookieManager
{
public:

	virtual ~FCefCookieManager()
	{ }

	virtual void SetCookie(const FString& URL, const FCookie& Cookie, TFunction<void(bool)> Completed) override
	{
		CefRefPtr<FSetCookieFunctionCallback> Callback = Completed ? new FSetCookieFunctionCallback(Completed) : nullptr;

		CefCookie CefCookie;
		CefString(&CefCookie.name) = TCHAR_TO_WCHAR(*Cookie.Name);
		CefString(&CefCookie.value) = TCHAR_TO_WCHAR(*Cookie.Value);
		CefString(&CefCookie.domain) = TCHAR_TO_WCHAR(*Cookie.Domain);
		CefString(&CefCookie.path) = TCHAR_TO_WCHAR(*Cookie.Path);
		CefCookie.secure = Cookie.bSecure;
		CefCookie.httponly = Cookie.bHttpOnly;
		CefCookie.has_expires = Cookie.bHasExpires;

		cef_time_t CefTime;
		CefTime.year = Cookie.Expires.GetYear();
		CefTime.month = Cookie.Expires.GetMonth();
		CefTime.day_of_month = Cookie.Expires.GetDay();
		CefTime.hour = Cookie.Expires.GetHour();
		CefTime.minute = Cookie.Expires.GetMinute();
		CefTime.second = Cookie.Expires.GetSecond();
		CefTime.millisecond = Cookie.Expires.GetMillisecond();

		const EDayOfWeek DayOfWeek = Cookie.Expires.GetDayOfWeek();

		if (DayOfWeek == EDayOfWeek::Sunday) // some crazy reason our date/time class think Monday is the beginning of the week
		{
			CefTime.day_of_week = 0;
		}
		else
		{
			CefTime.day_of_week = ((int32)DayOfWeek) + 1;
		}

		cef_time_to_basetime(&CefTime, &CefCookie.expires);

		if (!CookieManager->SetCookie(TCHAR_TO_WCHAR(*URL), CefCookie, Callback))
		{
			Callback->OnComplete(false);
		}
	}

	virtual void DeleteCookies(const FString& URL, const FString& CookieName, TFunction<void(int)> Completed) override
	{
		CefRefPtr<FDeleteCookiesFunctionCallback> Callback = Completed ? new FDeleteCookiesFunctionCallback(Completed) : nullptr;
		if (!CookieManager->DeleteCookies(TCHAR_TO_WCHAR(*URL), TCHAR_TO_WCHAR(*CookieName), Callback))
		{
			Callback->OnComplete(-1);
		}
	}

private:

	FCefCookieManager(
		const CefRefPtr<CefCookieManager>& InCookieManager)
		: CookieManager(InCookieManager)
	{ }

	// Callback that will invoke the callback with the result on the UI thread.
	class FDeleteCookiesFunctionCallback
		: public CefDeleteCookiesCallback
	{
		TFunction<void(int)> Callback;
	public:
		FDeleteCookiesFunctionCallback(const TFunction<void(int)>& InCallback)
			: Callback(InCallback)
		{}

		virtual void OnComplete(int NumDeleted) override
		{
			// We're on the IO thread, so we'll have to schedule the callback on the main thread
			CefPostTask(TID_UI, new FDeleteCookiesNotificationTask(Callback, NumDeleted));
		}

		IMPLEMENT_REFCOUNTING(FDeleteCookiesFunctionCallback);
	};

	// Callback that will invoke the callback with the result on the UI thread.
	class FSetCookieFunctionCallback
		: public CefSetCookieCallback
	{
		TFunction<void(bool)> Callback;
	public:
		FSetCookieFunctionCallback(const TFunction<void(bool)>& InCallback)
			: Callback(InCallback)
		{}

		virtual void OnComplete(bool bSuccess) override
		{
			// We're on the IO thread, so we'll have to schedule the callback on the main thread
			CefPostTask(TID_UI, new FSetCookieNotificationTask(Callback, bSuccess));
		}

		IMPLEMENT_REFCOUNTING(FSetCookieFunctionCallback);
	};

	// Task for executing a callback on a given thread.
	class FDeleteCookiesNotificationTask
		: public CefTask
	{
		TFunction<void(int)> Callback;
		int Value;

	public:

		FDeleteCookiesNotificationTask(const TFunction<void(int)>& InCallback, int InValue)
			: Callback(InCallback)
			, Value(InValue)
		{}

		virtual void Execute() override
		{
			Callback(Value);
		}

		IMPLEMENT_REFCOUNTING(FDeleteCookiesNotificationTask);
	};

	// Task for executing a callback on a given thread.
	class FSetCookieNotificationTask
		: public CefTask
	{
		TFunction<void(bool)> Callback;
		bool bSuccess;

	public:

		FSetCookieNotificationTask(const TFunction<void(bool)>& InCallback, bool InSuccess)
			: Callback(InCallback)
			, bSuccess(InSuccess)
		{}

		virtual void Execute() override
		{
			Callback(bSuccess);
		}

		IMPLEMENT_REFCOUNTING(FSetCookieNotificationTask);
	};

private:

	const CefRefPtr<CefCookieManager> CookieManager;

	friend FCefWebBrowserCookieManagerFactory;
};

TSharedRef<IWebBrowserCookieManager> FCefWebBrowserCookieManagerFactory::Create(
	const CefRefPtr<CefCookieManager>& CookieManager)
{
	return MakeShareable(new FCefCookieManager(
		CookieManager));
}

#endif
