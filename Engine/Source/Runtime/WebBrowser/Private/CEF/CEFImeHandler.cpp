// Copyright Epic Games, Inc. All Rights Reserved.

#include "CEF/CEFImeHandler.h"

#if WITH_CEF3 && !PLATFORM_LINUX

#include "CEFTextInputMethodContext.h"


FCEFImeHandler::FCEFImeHandler(CefRefPtr<CefBrowser> Browser)
	: InternalCefBrowser(Browser)
	, TextInputMethodSystem(nullptr)
	, bHasFocus(false)
	, bContextWasActiveWhenFocusLost(false)
{

}

bool FCEFImeHandler::OnProcessMessageReceived(CefRefPtr<CefBrowser> Browser, CefProcessId SourceProcess, CefRefPtr<CefProcessMessage> Message)
{
	bool Result = false;
	FString MessageName = WCHAR_TO_TCHAR(Message->GetName().ToWString().c_str());
	if (MessageName == TEXT("UE::IME::FocusChanged"))
	{
		Result = HandleFocusChangedMessage(Message->GetArgumentList());
	}
	return Result;
}

void FCEFImeHandler::SendProcessMessage(CefRefPtr<CefProcessMessage> Message)
{
	if (IsValid() && InternalCefBrowser->GetMainFrame())
	{
		InternalCefBrowser->GetMainFrame()->SendProcessMessage(PID_RENDERER, Message);
	}
}

void FCEFImeHandler::BindInputMethodSystem(ITextInputMethodSystem* InTextInputMethodSystem)
{
	TextInputMethodSystem = InTextInputMethodSystem;
	if (TextInputMethodSystem && TextInputMethodContext.IsValid())
	{
		TextInputMethodChangeNotifier = TextInputMethodSystem->RegisterContext(TextInputMethodContext.ToSharedRef());
	}
}

void FCEFImeHandler::UnbindInputMethodSystem()
{
	if (TextInputMethodContext.IsValid())
	{
		DestroyContext();
	}
	TextInputMethodSystem = nullptr;
}

void FCEFImeHandler::ActivateContext()
{
	if (!TextInputMethodSystem)
	{
		return;
	}

	if (!TextInputMethodContext.IsValid())
	{
		TextInputMethodContext = FCEFTextInputMethodContext::Create(SharedThis(this));
		TextInputMethodChangeNotifier = TextInputMethodSystem->RegisterContext(TextInputMethodContext.ToSharedRef());
		if (TextInputMethodChangeNotifier.IsValid())
		{
			TextInputMethodChangeNotifier->NotifyLayoutChanged(ITextInputMethodChangeNotifier::ELayoutChangeType::Created);
		}
	}

	TextInputMethodContext->UpdateCachedSlateWindow();

	TSharedRef<FCEFTextInputMethodContext> TextInputMethodContextRef = TextInputMethodContext.ToSharedRef();
	if (!TextInputMethodSystem->IsActiveContext(TextInputMethodContextRef))
	{
		TextInputMethodSystem->ActivateContext(TextInputMethodContextRef);
	}
}

void FCEFImeHandler::DeactivateContext()
{
	if (!TextInputMethodSystem || !TextInputMethodContext.IsValid())
	{
		return;
	}

	TSharedRef<FCEFTextInputMethodContext> TextInputMethodContextRef = TextInputMethodContext.ToSharedRef();
	if (TextInputMethodSystem->IsActiveContext(TextInputMethodContextRef))
	{
		// Make sure that the composition is aborted to avoid any IME calls to EndComposition from trying to mutate our dying owner widget
		if (TextInputMethodContextRef->IsComposing())
		{
			TextInputMethodContextRef->AbortComposition();
			if (TextInputMethodChangeNotifier.IsValid())
			{
				TextInputMethodChangeNotifier->CancelComposition();
			}
		}
		TextInputMethodSystem->DeactivateContext(TextInputMethodContextRef);
	}
}

void FCEFImeHandler::DestroyContext()
{
	if (!TextInputMethodContext.IsValid())
	{
		return;
	}

	if (TextInputMethodSystem)
	{
		DeactivateContext();
		TextInputMethodSystem->UnregisterContext(TextInputMethodContext.ToSharedRef());
	}

	TextInputMethodChangeNotifier.Reset();
	TextInputMethodContext.Reset();
}

bool FCEFImeHandler::HandleFocusChangedMessage(CefRefPtr<CefListValue> MessageArguments)
{
	if (MessageArguments->GetSize() == 1 &&
		MessageArguments->GetType(0) == VTYPE_STRING)
	{
		DeactivateContext();
		return true;
	}
	else if (MessageArguments->GetSize() == 8 &&
		MessageArguments->GetType(0) == VTYPE_STRING &&
		MessageArguments->GetType(1) == VTYPE_STRING &&
		MessageArguments->GetType(2) == VTYPE_BOOL &&
		MessageArguments->GetType(3) == VTYPE_STRING &&
		MessageArguments->GetType(4) == VTYPE_INT &&
		MessageArguments->GetType(5) == VTYPE_INT &&
		MessageArguments->GetType(6) == VTYPE_INT &&
		MessageArguments->GetType(7) == VTYPE_INT)
	{
		FString Type = WCHAR_TO_TCHAR(MessageArguments->GetString(0).ToWString().c_str());
		bool bIsEditable = MessageArguments->GetBool(2);

		if (Type == TEXT("DOM_NODE_TYPE_ELEMENT") && bIsEditable)
		{
			ActivateContext();
		}
		else
		{
			DeactivateContext();
		}
		return true;
	}

	return false;
}

void FCEFImeHandler::UnbindCefBrowser()
{
	if (TextInputMethodContext.IsValid())
	{
		DestroyContext();
	}
}

void FCEFImeHandler::CacheBrowserSlateInfo(const TSharedRef<SWidget>& Widget)
{
	InternalBrowserSlateWidget = Widget;
}

void FCEFImeHandler::SetFocus(bool bInFocus)
{
	// Skip duplicate SetFocus calls sent by Slate e.g. when docking/undocking tabs
	if (bInFocus == bHasFocus)
	{
		return;
	}

	bHasFocus = bInFocus;

	if (!TextInputMethodSystem || !TextInputMethodContext.IsValid())
	{
		return;
	}

	if (!bInFocus)
	{
		// Save the current activated state of the context, so we can restore it if we get the focus back.
		bContextWasActiveWhenFocusLost = TextInputMethodSystem->IsActiveContext(TextInputMethodContext.ToSharedRef());
		DeactivateContext();
	}
	else if (bContextWasActiveWhenFocusLost)
	{
		// Reactivate the context when we get focus back, for the case where the focused element in the page hasn't changed.
		// If the user clicked on a different element to focus the browser, then we'll get a FocusChanged message and we'll adjust activation accordingly
		ActivateContext();
	}
}

void FCEFImeHandler::UpdateCachedGeometry(const FGeometry& AllottedGeometry)
{
	if (TextInputMethodContext.IsValid() &&
		TextInputMethodContext->UpdateCachedGeometry(AllottedGeometry))
	{
		if (TextInputMethodChangeNotifier.IsValid())
		{
			TextInputMethodChangeNotifier->NotifyLayoutChanged(ITextInputMethodChangeNotifier::ELayoutChangeType::Changed);
		}
	}
}

void FCEFImeHandler::CEFCompositionRangeChanged(const CefRange& SelectionRange, const CefRenderHandler::RectList& CharacterBounds)
{
	if (TextInputMethodContext.IsValid() &&
		TextInputMethodContext->CEFCompositionRangeChanged(SelectionRange, CharacterBounds))
	{
		if (TextInputMethodChangeNotifier.IsValid())
		{
			TextInputMethodChangeNotifier->NotifyLayoutChanged(ITextInputMethodChangeNotifier::ELayoutChangeType::Changed);
		}
	}
}

void FCEFImeHandler::CEFTextSelectionChanged(const CefString& SelectedText, const CefRange& SelectionRange)
{
	if (TextInputMethodContext.IsValid() &&
		TextInputMethodContext->CEFTextSelectionChanged(SelectedText, SelectionRange))
	{
		if (TextInputMethodChangeNotifier.IsValid())
		{
			TextInputMethodChangeNotifier->NotifySelectionChanged();
		}
	}
}

#endif
