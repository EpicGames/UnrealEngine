// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_CEF3 && !PLATFORM_LINUX

#include "Widgets/SWidget.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/AllowWindowsPlatformAtomics.h"
#endif
#pragma push_macro("OVERRIDE")
#undef OVERRIDE // cef headers provide their own OVERRIDE macro
THIRD_PARTY_INCLUDES_START
#if PLATFORM_APPLE
PRAGMA_DISABLE_DEPRECATION_WARNINGS
#endif
#include "include/cef_client.h"
#include "include/cef_values.h"
#if PLATFORM_APPLE
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
THIRD_PARTY_INCLUDES_END
#pragma pop_macro("OVERRIDE")
#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformAtomics.h"
#include "Windows/HideWindowsPlatformTypes.h"
#endif
#include "Layout/Geometry.h"

class ITextInputMethodSystem;
class FCEFTextInputMethodContext;
class ITextInputMethodChangeNotifier;
class SWidget;

class FCEFImeHandler
	: public TSharedFromThis<FCEFImeHandler>
{
public:
	FCEFImeHandler(CefRefPtr<CefBrowser> Browser);

	void UnbindCefBrowser();
	void CacheBrowserSlateInfo(const TSharedRef<SWidget>& Widget);
	void SetFocus(bool bInFocus);
	void UpdateCachedGeometry(const FGeometry& AllottedGeometry);

	/**
	* Called when the IME composition DOM node has changed.
	*
	* @param SelectionRange The range of characters that have been selected.
	* @param CharacterBounds The bounds of each character in view coordinates.
	*/
	void CEFCompositionRangeChanged(const CefRange& SelectionRange, const CefRenderHandler::RectList& CharacterBounds);

	/**
	 * Called when the text selection has changed for a text field in a web page, used for IME.
	 * @param SelectedText The selected text.
	 * @param SelectionRange The range of characters that have been selected.
	 */
	void CEFTextSelectionChanged(const CefString& SelectedText, const CefRange& SelectionRange);

	/**
	 * Called when a message was received from the renderer process.
	 *
	 * @param Browser The CefBrowser for this window.
	 * @param SourceProcess The process id of the sender of the message. Currently always PID_RENDERER.
	 * @param Message The actual message.
	 * @return true if the message was handled, else false.
	 */
	bool OnProcessMessageReceived(CefRefPtr<CefBrowser> Browser, CefProcessId SourceProcess, CefRefPtr<CefProcessMessage> Message);

	/**
	 * Sends a message to the renderer process.
	 * See https://bitbucket.org/chromiumembedded/cef/wiki/GeneralUsage#markdown-header-inter-process-communication-ipc for more information.
	 *
	 * @param Message the message to send to the renderer process
	 */
	void SendProcessMessage(CefRefPtr<CefProcessMessage> Message);

	// FWebImeHandler Interface

	void BindInputMethodSystem(ITextInputMethodSystem* InTextInputMethodSystem);
	void UnbindInputMethodSystem();


private:

	bool IsValid()
	{
		return InternalCefBrowser.get() != nullptr;
	}

	void ActivateContext();
	void DeactivateContext();
	void DestroyContext();

	/** Message handling helpers */
	bool HandleFocusChangedMessage(CefRefPtr<CefListValue> MessageArguments);

	/** Pointer to the CEF Browser for this window. */
	CefRefPtr<CefBrowser> InternalCefBrowser;

	TWeakPtr<SWidget> InternalBrowserSlateWidget;

	ITextInputMethodSystem* TextInputMethodSystem;

	/** IME context for this browser window.  This gets recreated whenever we change focus to an editable input field. */
	TSharedPtr<FCEFTextInputMethodContext> TextInputMethodContext;

	/** Notification interface object for IMEs */
	TSharedPtr<ITextInputMethodChangeNotifier> TextInputMethodChangeNotifier;

	/** Caches the focus state from Slate. */
	bool bHasFocus;

	/** Saves the active state of the TextInputMethodContext when the browser loses focus. */
	bool bContextWasActiveWhenFocusLost;

	// Allow IME context to access functions only it needs.
	friend class FCEFTextInputMethodContext;
};

#endif
