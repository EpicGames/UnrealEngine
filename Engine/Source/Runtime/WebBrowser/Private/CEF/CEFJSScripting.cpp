// Copyright Epic Games, Inc. All Rights Reserved.

#include "CEF/CEFJSScripting.h"

#if WITH_CEF3

#include "WebJSScripting.h"
#include "WebJSFunction.h"
#include "CEFWebBrowserWindow.h"
#include "CEFJSStructSerializerBackend.h"
#include "CEFJSStructDeserializerBackend.h"
#include "StructSerializer.h"
#include "StructDeserializer.h"


// Internal utility function(s)
namespace
{

	template<typename DestContainerType, typename SrcContainerType, typename DestKeyType, typename SrcKeyType>
	bool CopyContainerValue(DestContainerType DestContainer, SrcContainerType SrcContainer, DestKeyType DestKey, SrcKeyType SrcKey )
	{
		switch (SrcContainer->GetType(SrcKey))
		{
			case VTYPE_NULL:
				return DestContainer->SetNull(DestKey);
			case VTYPE_BOOL:
				return DestContainer->SetBool(DestKey, SrcContainer->GetBool(SrcKey));
			case VTYPE_INT:
				return DestContainer->SetInt(DestKey, SrcContainer->GetInt(SrcKey));
			case VTYPE_DOUBLE:
				return DestContainer->SetDouble(DestKey, SrcContainer->GetDouble(SrcKey));
			case VTYPE_STRING:
				return DestContainer->SetString(DestKey, SrcContainer->GetString(SrcKey));
			case VTYPE_BINARY:
				return DestContainer->SetBinary(DestKey, SrcContainer->GetBinary(SrcKey));
			case VTYPE_DICTIONARY:
				return DestContainer->SetDictionary(DestKey, SrcContainer->GetDictionary(SrcKey));
			case VTYPE_LIST:
				return DestContainer->SetList(DestKey, SrcContainer->GetList(SrcKey));
			case VTYPE_INVALID:
			default:
				return false;
		}

	}
}

CefRefPtr<CefDictionaryValue> FCEFJSScripting::ConvertStruct(UStruct* TypeInfo, const void* StructPtr)
{
	FCEFJSStructSerializerBackend Backend (SharedThis(this));
	FStructSerializer::Serialize(StructPtr, *TypeInfo, Backend);

	CefRefPtr<CefDictionaryValue> Result = CefDictionaryValue::Create();
	Result->SetString("$type", "struct");
	Result->SetString("$ue4Type", TCHAR_TO_WCHAR(*GetBindingName(TypeInfo)));
	Result->SetDictionary("$value", Backend.GetResult());
	return Result;
}

CefRefPtr<CefDictionaryValue> FCEFJSScripting::ConvertObject(UObject* Object)
{
	CefRefPtr<CefDictionaryValue> Result = CefDictionaryValue::Create();
	RetainBinding(Object);

	UClass* Class = Object->GetClass();
	CefRefPtr<CefListValue> MethodNames = CefListValue::Create();
	int32 MethodIndex = 0;
	for (TFieldIterator<UFunction> FunctionIt(Class, EFieldIteratorFlags::IncludeSuper); FunctionIt; ++FunctionIt)
	{
		UFunction* Function = *FunctionIt;
		MethodNames->SetString(MethodIndex++, TCHAR_TO_WCHAR(*GetBindingName(Function)));
	}

	Result->SetString("$type", "uobject");
	Result->SetString("$id", TCHAR_TO_WCHAR(*PtrToGuid(Object).ToString(EGuidFormats::Digits)));
	Result->SetList("$methods", MethodNames);
	return Result;
}


bool FCEFJSScripting::OnProcessMessageReceived(CefRefPtr<CefBrowser> Browser, CefProcessId SourceProcess, CefRefPtr<CefProcessMessage> Message)
{
	bool Result = false;
	FString MessageName = WCHAR_TO_TCHAR(Message->GetName().ToWString().c_str());
	if (MessageName == TEXT("UE::ExecuteUObjectMethod"))
	{
		Result = HandleExecuteUObjectMethodMessage(Message->GetArgumentList());
	}
	else if (MessageName == TEXT("UE::ReleaseUObject"))
	{
		Result = HandleReleaseUObjectMessage(Message->GetArgumentList());
	}
	return Result;
}

void FCEFJSScripting::SendProcessMessage(CefRefPtr<CefProcessMessage> Message)
{
	if (IsValid() && InternalCefBrowser->GetMainFrame())
	{
		InternalCefBrowser->GetMainFrame()->SendProcessMessage(PID_RENDERER, Message);
	}
}

CefRefPtr<CefDictionaryValue> FCEFJSScripting::GetPermanentBindings()
{
	CefRefPtr<CefDictionaryValue> Result = CefDictionaryValue::Create();

	TMap<FString, UObject*> CachedPermanentUObjectsByName = PermanentUObjectsByName;

	for(auto& Entry : CachedPermanentUObjectsByName)
	{
		Result->SetDictionary(TCHAR_TO_WCHAR(*Entry.Key), ConvertObject(Entry.Value));
	}
	return Result;
}


void FCEFJSScripting::BindUObject(const FString& Name, UObject* Object, bool bIsPermanent)
{
	const FString ExposedName = GetBindingName(Name, Object);
	CefRefPtr<CefDictionaryValue> Converted = ConvertObject(Object);
	if (bIsPermanent)
	{
		// Each object can only have one permanent binding
		if (BoundObjects[Object].bIsPermanent)
		{
			return;
		}
		// Existing permanent objects must be removed first
		if (PermanentUObjectsByName.Contains(ExposedName))
		{
			return;
		}
		BoundObjects[Object]={true, -1};
		PermanentUObjectsByName.Add(ExposedName, Object);
	}

	CefRefPtr<CefProcessMessage> SetValueMessage = CefProcessMessage::Create(TCHAR_TO_WCHAR(TEXT("UE::SetValue")));
	CefRefPtr<CefListValue>MessageArguments = SetValueMessage->GetArgumentList();
	CefRefPtr<CefDictionaryValue> Value = CefDictionaryValue::Create();
	Value->SetString("name", TCHAR_TO_WCHAR(*ExposedName));
	Value->SetDictionary("value", Converted);
	Value->SetBool("permanent", bIsPermanent);

	MessageArguments->SetDictionary(0, Value);
	SendProcessMessage(SetValueMessage);
}

void FCEFJSScripting::UnbindUObject(const FString& Name, UObject* Object, bool bIsPermanent)
{
	const FString ExposedName = GetBindingName(Name, Object);

	if (bIsPermanent)
	{
		// If overriding an existing permanent object, make it non-permanent
		if (PermanentUObjectsByName.Contains(ExposedName) && (Object == nullptr || PermanentUObjectsByName[ExposedName] == Object))
		{
			Object = PermanentUObjectsByName.FindAndRemoveChecked(ExposedName);
			BoundObjects.Remove(Object);
			return;
		}
		else
		{
			return;
		}
	}

	CefRefPtr<CefProcessMessage> DeleteValueMessage = CefProcessMessage::Create(TCHAR_TO_WCHAR(TEXT("UE::DeleteValue")));
	CefRefPtr<CefListValue>MessageArguments = DeleteValueMessage->GetArgumentList();
	CefRefPtr<CefDictionaryValue> Info = CefDictionaryValue::Create();
	Info->SetString("name", TCHAR_TO_WCHAR(*ExposedName));
	Info->SetString("id", TCHAR_TO_WCHAR(*PtrToGuid(Object).ToString(EGuidFormats::Digits)));
	Info->SetBool("permanent", bIsPermanent);

	MessageArguments->SetDictionary(0, Info);
	SendProcessMessage(DeleteValueMessage);
}

bool FCEFJSScripting::HandleReleaseUObjectMessage(CefRefPtr<CefListValue> MessageArguments)
{
	FGuid ObjectKey;
	// Message arguments are Name, Value, bGlobal
	if (MessageArguments->GetSize() != 1 || MessageArguments->GetType(0) != VTYPE_STRING)
	{
		// Wrong message argument types or count
		return false;
	}

	if (!FGuid::Parse(FString(WCHAR_TO_TCHAR(MessageArguments->GetString(0).ToWString().c_str())), ObjectKey))
	{
		// Invalid GUID
		return false;
	}

	UObject* Object = GuidToPtr(ObjectKey);
	if ( Object == nullptr )
	{
		// Invalid object
		return false;
	}
	ReleaseBinding(Object);
	return true;
}

bool FCEFJSScripting::HandleExecuteUObjectMethodMessage(CefRefPtr<CefListValue> MessageArguments)
{
	FGuid ObjectKey;
	// Message arguments are Name, Value, bGlobal
	if (MessageArguments->GetSize() != 4
		|| MessageArguments->GetType(0) != VTYPE_STRING
		|| MessageArguments->GetType(1) != VTYPE_STRING
		|| MessageArguments->GetType(2) != VTYPE_STRING
		|| MessageArguments->GetType(3) != VTYPE_LIST
		)
	{
		// Wrong message argument types or count
		return false;
	}

	if (!FGuid::Parse(FString(WCHAR_TO_TCHAR(MessageArguments->GetString(0).ToWString().c_str())), ObjectKey))
	{
		// Invalid GUID
		return false;
	}

	// Get the promise callback and use that to report any results from executing this function.
	FGuid ResultCallbackId;
	if (!FGuid::Parse(FString(WCHAR_TO_TCHAR(MessageArguments->GetString(2).ToWString().c_str())), ResultCallbackId))
	{
		// Invalid GUID
		return false;
	}

	UObject* Object = GuidToPtr(ObjectKey);
	if (Object == nullptr)
	{
		// Unknown uobject id
		InvokeJSErrorResult(ResultCallbackId, TEXT("Unknown UObject ID"));
		return true;
	}

	FName MethodName = WCHAR_TO_TCHAR(MessageArguments->GetString(1).ToWString().c_str());
	UFunction* Function = Object->FindFunction(MethodName);
	if (!Function)
	{
		InvokeJSErrorResult(ResultCallbackId, TEXT("Unknown UObject Function"));
		return true;
	}
	// Coerce arguments to function arguments.
	uint16 ParamsSize = Function->ParmsSize;
	uint8* Params  = nullptr;
	FProperty* ReturnParam = nullptr;
	FProperty* PromiseParam = nullptr;

	if (ParamsSize > 0)
	{
		// Convert cef argument list to a dictionary, so we can use FStructDeserializer to convert it for us
		CefRefPtr<CefDictionaryValue> NamedArgs = CefDictionaryValue::Create();
		int32 CurrentArg = 0;
		CefRefPtr<CefListValue> CefArgs = MessageArguments->GetList(3);
		for ( TFieldIterator<FProperty> It(Function); It; ++It )
		{
			FProperty* Param = *It;
			if (Param->PropertyFlags & CPF_Parm)
			{
				if (Param->PropertyFlags & CPF_ReturnParm)
				{
					ReturnParam = Param;
				}
				else
				{
					FStructProperty *StructProperty = CastField<FStructProperty>(Param);
					if (StructProperty && StructProperty->Struct->IsChildOf(FWebJSResponse::StaticStruct()))
					{
						PromiseParam = Param;
					}
					else
					{
						CopyContainerValue(NamedArgs, CefArgs, CefString(TCHAR_TO_WCHAR(*GetBindingName(Param))), CurrentArg);
						CurrentArg++;
					}
				}
			}
		}

		// UFunction is a subclass of UStruct, so we can treat the arguments as a struct for deserialization
		check(nullptr == Params);
		Params = (uint8*)FMemory::Malloc(Function->GetStructureSize());
		Function->InitializeStruct(Params);
		FCEFJSStructDeserializerBackend Backend = FCEFJSStructDeserializerBackend(SharedThis(this), NamedArgs);
		FStructDeserializer::Deserialize(Params, *Function, Backend);
	}

	if (PromiseParam)
	{
		FWebJSResponse* PromisePtr = PromiseParam->ContainerPtrToValuePtr<FWebJSResponse>(Params);
		if (PromisePtr)
		{
			*PromisePtr = FWebJSResponse(SharedThis(this), ResultCallbackId);
		}
	}

	Object->ProcessEvent(Function, Params);
	CefRefPtr<CefListValue> Results = CefListValue::Create();

	if ( ! PromiseParam ) // If PromiseParam is set, we assume that the UFunction will ensure it is called with the result
	{
		if ( ReturnParam )
		{
			FStructSerializerPolicies ReturnPolicies;
			ReturnPolicies.PropertyFilter = [&](const FProperty* CandidateProperty, const FProperty* ParentProperty)
			{
				return ParentProperty != nullptr || CandidateProperty == ReturnParam;
			};
			FCEFJSStructSerializerBackend ReturnBackend(SharedThis(this));
			FStructSerializer::Serialize(Params, *Function, ReturnBackend, ReturnPolicies);
			CefRefPtr<CefDictionaryValue> ResultDict = ReturnBackend.GetResult();

			// Extract the single return value from the serialized dictionary to an array
			CopyContainerValue(Results, ResultDict, 0, TCHAR_TO_WCHAR(*GetBindingName(ReturnParam)));
		}
		InvokeJSFunction(ResultCallbackId, Results, false);
	}

	if (Params)
	{
		// Invoke the parameters destructors as needed before freeing the Params memory itself
		for (TFieldIterator<FProperty> It(Function); It && It->HasAnyPropertyFlags(CPF_Parm); ++It)
		{
			if (It->HasAnyPropertyFlags(CPF_NoDestructor) || !It->IsInContainer(Function->GetStructureSize()))
			{
				continue;
			}

			It->DestroyValue_InContainer(Params);
		}

		FMemory::Free(Params);
		Params = nullptr;
	}

	return true;
}

void FCEFJSScripting::UnbindCefBrowser()
{
	InternalCefBrowser = nullptr;
}

void FCEFJSScripting::InvokeJSErrorResult(FGuid FunctionId, const FString& Error)
{
	CefRefPtr<CefListValue> FunctionArguments = CefListValue::Create();
	FunctionArguments->SetString(0, TCHAR_TO_WCHAR(*Error));
	InvokeJSFunction(FunctionId, FunctionArguments, true);
}

void FCEFJSScripting::InvokeJSFunction(FGuid FunctionId, int32 ArgCount, FWebJSParam Arguments[], bool bIsError)
{
	CefRefPtr<CefListValue> FunctionArguments = CefListValue::Create();
	for ( int32 i=0; i<ArgCount; i++)
	{
		SetConverted(FunctionArguments, i, Arguments[i]);
	}
	InvokeJSFunction(FunctionId, FunctionArguments, bIsError);
}

void FCEFJSScripting::InvokeJSFunction(FGuid FunctionId, const CefRefPtr<CefListValue>& FunctionArguments, bool bIsError)
{
	CefRefPtr<CefProcessMessage> Message = CefProcessMessage::Create(TCHAR_TO_WCHAR(TEXT("UE::ExecuteJSFunction")));
	CefRefPtr<CefListValue> MessageArguments = Message->GetArgumentList();
	MessageArguments->SetString(0, TCHAR_TO_WCHAR(*FunctionId.ToString(EGuidFormats::Digits)));
	MessageArguments->SetList(1, FunctionArguments);
	MessageArguments->SetBool(2, bIsError);
	SendProcessMessage(Message);
}

#endif