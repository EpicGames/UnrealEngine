// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_CEF3
#include "WebJSFunction.h"
#include "WebJSScripting.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/AllowWindowsPlatformAtomics.h"
#endif
#pragma push_macro("OVERRIDE")
#undef OVERRIDE // cef headers provide their own OVERRIDE macro
THIRD_PARTY_INCLUDES_START
#if PLATFORM_APPLE
PRAGMA_DISABLE_DEPRECATION_WARNINGS
#endif
#include "include/cef_client.h"
#include "include/cef_values.h"
#if PLATFORM_APPLE
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
THIRD_PARTY_INCLUDES_END
#pragma pop_macro("OVERRIDE")
#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformAtomics.h"
#include "Windows/HideWindowsPlatformTypes.h"
#endif
#endif

class Error;
class FWebJSScripting;
struct FWebJSParam;

#if WITH_CEF3

/**
 * Implements handling of bridging UObjects client side with JavaScript renderer side.
 */
class FCEFJSScripting
	: public FWebJSScripting
	, public TSharedFromThis<FCEFJSScripting>
{
public:
	FCEFJSScripting(CefRefPtr<CefBrowser> Browser, bool bJSBindingToLoweringEnabled)
		: FWebJSScripting(bJSBindingToLoweringEnabled)
		, InternalCefBrowser(Browser)
	{}

	void UnbindCefBrowser();

	virtual void BindUObject(const FString& Name, UObject* Object, bool bIsPermanent = true) override;
	virtual void UnbindUObject(const FString& Name, UObject* Object = nullptr, bool bIsPermanent = true) override;

	/**
	 * Called when a message was received from the renderer process.
	 *
	 * @param Browser The CefBrowser for this window.
	 * @param SourceProcess The process id of the sender of the message. Currently always PID_RENDERER.
	 * @param Message The actual message.
	 * @return true if the message was handled, else false.
	 */
	bool OnProcessMessageReceived(CefRefPtr<CefBrowser> Browser, CefProcessId SourceProcess, CefRefPtr<CefProcessMessage> Message);

	/**
	 * Sends a message to the renderer process.
	 * See https://bitbucket.org/chromiumembedded/cef/wiki/GeneralUsage#markdown-header-inter-process-communication-ipc for more information.
	 *
	 * @param Message the message to send to the renderer process
	 */
	void SendProcessMessage(CefRefPtr<CefProcessMessage> Message);

	CefRefPtr<CefDictionaryValue> ConvertStruct(UStruct* TypeInfo, const void* StructPtr);
	CefRefPtr<CefDictionaryValue> ConvertObject(UObject* Object);

	// Works for CefListValue and CefDictionaryValues
	template<typename ContainerType, typename KeyType>
	bool SetConverted(CefRefPtr<ContainerType> Container, KeyType Key, FWebJSParam& Param)
	{
		switch (Param.Tag)
		{
			case FWebJSParam::PTYPE_NULL:
				return Container->SetNull(Key);
			case FWebJSParam::PTYPE_BOOL:
				return Container->SetBool(Key, Param.BoolValue);
			case FWebJSParam::PTYPE_DOUBLE:
				return Container->SetDouble(Key, Param.DoubleValue);
			case FWebJSParam::PTYPE_INT:
				return Container->SetInt(Key, Param.IntValue);
			case FWebJSParam::PTYPE_STRING:
			{
				CefString ConvertedString = TCHAR_TO_WCHAR(**Param.StringValue);
				return Container->SetString(Key, ConvertedString);
			}
			case FWebJSParam::PTYPE_OBJECT:
			{
				if (Param.ObjectValue == nullptr)
				{
					return Container->SetNull(Key);
				}
				else
				{
					CefRefPtr<CefDictionaryValue> ConvertedObject = ConvertObject(Param.ObjectValue);
					return Container->SetDictionary(Key, ConvertedObject);
				}
			}
			case FWebJSParam::PTYPE_STRUCT:
			{
				CefRefPtr<CefDictionaryValue> ConvertedStruct = ConvertStruct(Param.StructValue->GetTypeInfo(), Param.StructValue->GetData());
				return Container->SetDictionary(Key, ConvertedStruct);
			}
			case FWebJSParam::PTYPE_ARRAY:
			{
				CefRefPtr<CefListValue> ConvertedArray = CefListValue::Create();
				for(int i=0; i < Param.ArrayValue->Num(); ++i)
				{
					SetConverted(ConvertedArray, i, (*Param.ArrayValue)[i]);
				}
				return Container->SetList(Key, ConvertedArray);
			}
			case FWebJSParam::PTYPE_MAP:
			{
				CefRefPtr<CefDictionaryValue> ConvertedMap = CefDictionaryValue::Create();
				for(auto& Pair : *Param.MapValue)
				{
					SetConverted(ConvertedMap, TCHAR_TO_WCHAR(*Pair.Key), Pair.Value);
				}
				return Container->SetDictionary(Key, ConvertedMap);
			}
			default:
				return false;
		}
	}

	CefRefPtr<CefDictionaryValue> GetPermanentBindings();

	void InvokeJSFunction(FGuid FunctionId, int32 ArgCount, FWebJSParam Arguments[], bool bIsError=false) override;
	void InvokeJSFunction(FGuid FunctionId, const CefRefPtr<CefListValue>& FunctionArguments, bool bIsError=false);
	void InvokeJSErrorResult(FGuid FunctionId, const FString& Error) override;

private:
	bool ConvertStructArgImpl(uint8* Args, FProperty* Param, CefRefPtr<CefListValue> List, int32 Index);

	bool IsValid()
	{
		return InternalCefBrowser.get() != nullptr;
	}

	/** Message handling helpers */

	bool HandleExecuteUObjectMethodMessage(CefRefPtr<CefListValue> MessageArguments);
	bool HandleReleaseUObjectMessage(CefRefPtr<CefListValue> MessageArguments);

	/** Pointer to the CEF Browser for this window. */
	CefRefPtr<CefBrowser> InternalCefBrowser;
};

#endif
