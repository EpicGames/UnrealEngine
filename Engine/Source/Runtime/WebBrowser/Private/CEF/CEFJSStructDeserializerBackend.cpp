// Copyright Epic Games, Inc. All Rights Reserved.

#include "CEF/CEFJSStructDeserializerBackend.h"
#if WITH_CEF3
#include "UObject/EnumProperty.h"
#include "UObject/TextProperty.h"
#include "WebJSFunction.h"

/* Internal helpers
 *****************************************************************************/
namespace {

	template<typename ValueType, typename ContainerType, typename KeyType>
	ValueType GetNumeric(CefRefPtr<ContainerType> Container, KeyType Key)
	{
		switch(Container->GetType(Key))
		{
			case VTYPE_BOOL:
				return static_cast<ValueType>(Container->GetBool(Key));
			case VTYPE_INT:
				return static_cast<ValueType>(Container->GetInt(Key));
			case VTYPE_DOUBLE:
				return static_cast<ValueType>(Container->GetDouble(Key));
			case VTYPE_STRING:
			case VTYPE_DICTIONARY:
			case VTYPE_LIST:
			case VTYPE_NULL:
			case VTYPE_BINARY:
			default:
				return static_cast<ValueType>(0);
		}
	}

	template<typename ContainerType, typename KeyType>
	void AssignTokenFromContainer(ContainerType Container, KeyType Key,  EStructDeserializerBackendTokens& OutToken, FString& PropertyName, TSharedPtr<ICefContainerWalker>& Retval)
	{
		switch (Container->GetType(Key))
		{
			case VTYPE_NULL:
			case VTYPE_BOOL:
			case VTYPE_INT:
			case VTYPE_DOUBLE:
			case VTYPE_STRING:
				OutToken = EStructDeserializerBackendTokens::Property;
				break;
			case VTYPE_DICTIONARY:
			{
				CefRefPtr<CefDictionaryValue> Dictionary = Container->GetDictionary(Key);
				if (Dictionary->GetType("$type") == VTYPE_STRING )
				{
					OutToken = EStructDeserializerBackendTokens::Property;
				}
				else
				{
					TSharedPtr<ICefContainerWalker> NewWalker(new FCefDictionaryValueWalker(Retval, Dictionary));
					Retval = NewWalker->GetNextToken(OutToken, PropertyName);
				}
				break;
			}
			case VTYPE_LIST:
			{
				TSharedPtr<ICefContainerWalker> NewWalker(new FCefListValueWalker(Retval, Container->GetList(Key)));
				Retval = NewWalker->GetNextToken(OutToken, PropertyName);
				break;
			}
			case VTYPE_BINARY:
			case VTYPE_INVALID:
			default:
				OutToken = EStructDeserializerBackendTokens::Error;
				break;
		}
	}

	/**
	 * Gets a pointer to object of the given property.
	 *
	 * @param Property The property to get.
	 * @param Outer The property that contains the property to be get, if any.
	 * @param Data A pointer to the memory holding the property's data.
	 * @param ArrayIndex The index of the element to set (if the property is an array).
	 * @return A pointer to the object represented by the property, null otherwise..
	 * @see ClearPropertyValue
	 */
	void* GetPropertyValuePtr( FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex )
	{
		check(Property);

		if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Outer))
		{
			if (ArrayProperty->Inner != Property)
			{
				return nullptr;
			}

			FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->template ContainerPtrToValuePtr<void>(Data));
			int32 Index = ArrayHelper.AddValue();

			return ArrayHelper.GetRawPtr(Index);
		}

		if (ArrayIndex >= Property->ArrayDim)
		{
			return nullptr;
		}

		return Property->template ContainerPtrToValuePtr<void>(Data, ArrayIndex);
	}

	/**
	 * Sets the value of the given property.
	 *
	 * @param Property The property to set.
	 * @param Outer The property that contains the property to be set, if any.
	 * @param Data A pointer to the memory holding the property's data.
	 * @param ArrayIndex The index of the element to set (if the property is an array).
	 * @return true on success, false otherwise.
	 * @see ClearPropertyValue
	 */
	template<typename PropertyType, typename ValueType>
	bool SetPropertyValue( PropertyType* Property, FProperty* Outer, void* Data, int32 ArrayIndex, const ValueType& Value )
	{
		if (void* Ptr = GetPropertyValuePtr(Property, Outer, Data, ArrayIndex))
		{
			*(ValueType*)Ptr = Value;
			return true;
		}

		return false;
	}

	template<typename PropertyType, typename ContainerType, typename KeyType>
	bool ReadNumericProperty(FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex, CefRefPtr<ContainerType> Container, KeyType Key )
	{
		typedef typename PropertyType::TCppType TCppType;
		if (PropertyType* TypedProperty = CastField<PropertyType>(Property))
		{
			return SetPropertyValue(TypedProperty, Outer, Data, ArrayIndex, GetNumeric<TCppType>(Container, Key));
		}
		else
		{
			return false;
		}
	}

	template<typename ContainerType, typename KeyType>
	bool ReadBoolProperty(FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex, CefRefPtr<ContainerType> Container, KeyType Key )
	{
		if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
		{
			return SetPropertyValue(BoolProperty, Outer, Data, ArrayIndex, GetNumeric<int>(Container, Key)!=0);
		}
		return false;

	}

	template<typename ContainerType, typename KeyType>
	bool ReadJSFunctionProperty(TSharedPtr<FCEFJSScripting> Scripting, FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex, CefRefPtr<ContainerType> Container, KeyType Key )
	{
		if (Container->GetType(Key) != VTYPE_DICTIONARY || !Property->IsA<FStructProperty>())
		{
			return false;
		}
		CefRefPtr<CefDictionaryValue> Dictionary = Container->GetDictionary(Key);
		FStructProperty* StructProperty = CastField<FStructProperty>(Property);

		if ( !StructProperty || StructProperty->Struct != FWebJSFunction::StaticStruct())
		{
			return false;
		}

		FGuid CallbackID;
		if (!FGuid::Parse(FString(WCHAR_TO_TCHAR(Dictionary->GetString("$id").ToWString().c_str())), CallbackID))
		{
			// Invalid GUID
			return false;
		}
		FWebJSFunction CallbackObject(Scripting, CallbackID);
		return SetPropertyValue(StructProperty, Outer, Data, ArrayIndex, CallbackObject);
	}

	template<typename ContainerType, typename KeyType>
	bool ReadStringProperty(FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex, CefRefPtr<ContainerType> Container, KeyType Key )
	{
		if (Container->GetType(Key) == VTYPE_STRING)
		{
			FString StringValue = WCHAR_TO_TCHAR(Container->GetString(Key).ToWString().c_str());

			if (FStrProperty* StrProperty = CastField<FStrProperty>(Property))
			{
				return SetPropertyValue(StrProperty, Outer, Data, ArrayIndex, StringValue);
			}

			if (FNameProperty* NameProperty = CastField<FNameProperty>(Property))
			{
				return SetPropertyValue(NameProperty, Outer, Data, ArrayIndex, FName(*StringValue));
			}

			if (FTextProperty* TextProperty = CastField<FTextProperty>(Property))
			{
				return SetPropertyValue(TextProperty, Outer, Data, ArrayIndex, FText::FromString(StringValue));
			}

			if (FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
			{
				if (!ByteProperty->Enum)
				{
					return false;
				}

				int32 Index = ByteProperty->Enum->GetIndexByNameString(StringValue);
				if (Index == INDEX_NONE)
				{
					return false;
				}

				return SetPropertyValue(ByteProperty, Outer, Data, ArrayIndex, (uint8)ByteProperty->Enum->GetValueByIndex(Index));
			}

			if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
			{
				int32 Index = EnumProperty->GetEnum()->GetIndexByNameString(StringValue);
				if (Index == INDEX_NONE)
				{
					return false;
				}

				if (void* ElementPtr = GetPropertyValuePtr(EnumProperty, Outer, Data, ArrayIndex))
				{
					EnumProperty->GetUnderlyingProperty()->SetIntPropertyValue(ElementPtr, EnumProperty->GetEnum()->GetValueByIndex(Index));
					return true;
				}

				return false;
			}
		}

		return false;
	}

	template<typename ContainerType, typename KeyType>
	bool ReadProperty(TSharedPtr<FCEFJSScripting> Scripting, FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex, CefRefPtr<ContainerType> Container, KeyType Key )
	{
		return ReadBoolProperty(Property, Outer, Data, ArrayIndex, Container, Key)
			|| ReadStringProperty(Property, Outer, Data, ArrayIndex, Container, Key)
			|| ReadNumericProperty<FByteProperty>(Property, Outer, Data, ArrayIndex, Container, Key)
			|| ReadNumericProperty<FInt8Property>(Property, Outer, Data, ArrayIndex, Container, Key)
			|| ReadNumericProperty<FInt16Property>(Property, Outer, Data, ArrayIndex, Container, Key)
			|| ReadNumericProperty<FIntProperty>(Property, Outer, Data, ArrayIndex, Container, Key)
			|| ReadNumericProperty<FInt64Property>(Property, Outer, Data, ArrayIndex, Container, Key)
			|| ReadNumericProperty<FUInt16Property>(Property, Outer, Data, ArrayIndex, Container, Key)
			|| ReadNumericProperty<FUInt32Property>(Property, Outer, Data, ArrayIndex, Container, Key)
			|| ReadNumericProperty<FUInt64Property>(Property, Outer, Data, ArrayIndex, Container, Key)
			|| ReadNumericProperty<FFloatProperty>(Property, Outer, Data, ArrayIndex, Container, Key)
			|| ReadNumericProperty<FDoubleProperty>(Property, Outer, Data, ArrayIndex, Container, Key)
			|| ReadJSFunctionProperty(Scripting, Property, Outer, Data, ArrayIndex, Container, Key);
	}
}

TSharedPtr<ICefContainerWalker> FCefListValueWalker::GetNextToken(EStructDeserializerBackendTokens& OutToken, FString& PropertyName)
{
	TSharedPtr<ICefContainerWalker> Retval = SharedThis(this);
	Index++;
	if (Index == -1)
	{
		OutToken = EStructDeserializerBackendTokens::ArrayStart;
	}
	else if ( Index < List->GetSize() )
	{
		AssignTokenFromContainer(List, Index, OutToken, PropertyName, Retval);
		PropertyName = FString();
	}
	else
	{
		OutToken = EStructDeserializerBackendTokens::ArrayEnd;
		Retval = Parent;
	}
	return Retval;
}

bool FCefListValueWalker::ReadProperty(TSharedPtr<FCEFJSScripting> Scripting, FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex)
{
	return ::ReadProperty(Scripting, Property, Outer, Data, ArrayIndex, List, Index);
}

TSharedPtr<ICefContainerWalker> FCefDictionaryValueWalker::GetNextToken(EStructDeserializerBackendTokens& OutToken, FString& PropertyName)
{
	TSharedPtr<ICefContainerWalker> Retval = SharedThis(this);
	Index++;
	if (Index == -1)
	{
		OutToken = EStructDeserializerBackendTokens::StructureStart;
	}
	else if ( Index < Keys.size() )
	{
		AssignTokenFromContainer(Dictionary, Keys[Index], OutToken, PropertyName, Retval);
		PropertyName = WCHAR_TO_TCHAR(Keys[Index].ToWString().c_str());
	}
	else
	{
		OutToken = EStructDeserializerBackendTokens::StructureEnd;
		Retval = Parent;
	}
	return Retval;
}

bool FCefDictionaryValueWalker::ReadProperty(TSharedPtr<FCEFJSScripting> Scripting, FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex)
{
	return ::ReadProperty(Scripting, Property, Outer, Data, ArrayIndex, Dictionary, Keys[Index]);
}


/* IStructDeserializerBackend interface
 *****************************************************************************/



const FString& FCEFJSStructDeserializerBackend::GetCurrentPropertyName() const
{
	return CurrentPropertyName;
}


FString FCEFJSStructDeserializerBackend::GetDebugString() const
{
	return CurrentPropertyName;
}


const FString& FCEFJSStructDeserializerBackend::GetLastErrorMessage() const
{
	return CurrentPropertyName;
}


bool FCEFJSStructDeserializerBackend::GetNextToken( EStructDeserializerBackendTokens& OutToken )
{
	if (Walker.IsValid())
	{
		Walker = Walker->GetNextToken(OutToken, CurrentPropertyName);
		return true;
	}
	else
	{
		return false;
	}
}


bool FCEFJSStructDeserializerBackend::ReadProperty( FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex )
{
	return Walker->ReadProperty(Scripting, Property, Outer, Data, ArrayIndex);
}


void FCEFJSStructDeserializerBackend::SkipArray()
{
	EStructDeserializerBackendTokens Token;
	int32 depth = 1;
	while (GetNextToken(Token) && depth > 0)
	{
		switch (Token)
		{
		case EStructDeserializerBackendTokens::ArrayEnd:
			depth --;
			break;
		case EStructDeserializerBackendTokens::ArrayStart:
			depth ++;
			break;
		default:
			break;
		}
	}
}

void FCEFJSStructDeserializerBackend::SkipStructure()
{
	EStructDeserializerBackendTokens Token;
	int32 depth = 1;
	while (GetNextToken(Token) && depth > 0)
	{
		switch (Token)
		{
		case EStructDeserializerBackendTokens::StructureEnd:
			depth --;
			break;
		case EStructDeserializerBackendTokens::StructureStart:
			depth ++;
			break;
		default:
			break;
		}
	}
}

#endif
