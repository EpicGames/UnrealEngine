// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_CEF3

#include "WebBrowserSingleton.h"
#include "UObject/UnrealType.h"
#include "IStructDeserializerBackend.h"
#include "CEFJSScripting.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/AllowWindowsPlatformAtomics.h"
#endif
#pragma push_macro("OVERRIDE")
#undef OVERRIDE // cef headers provide their own OVERRIDE macro
THIRD_PARTY_INCLUDES_START
#if PLATFORM_APPLE
PRAGMA_DISABLE_DEPRECATION_WARNINGS
#endif
#include "include/cef_values.h"
#if PLATFORM_APPLE
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
THIRD_PARTY_INCLUDES_END
#pragma pop_macro("OVERRIDE")
#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformAtomics.h"
#include "Windows/HideWindowsPlatformTypes.h"
#endif

#endif

class FCEFJSScripting;
class IStructDeserializerBackend;
enum class EStructDeserializerBackendTokens;
class FProperty;
class UStruct;

#if WITH_CEF3

class ICefContainerWalker
	: public TSharedFromThis<ICefContainerWalker>
{
public:
	ICefContainerWalker(TSharedPtr<ICefContainerWalker> InParent)
		: Parent(InParent)
	{}
	virtual ~ICefContainerWalker() = default;

	virtual TSharedPtr<ICefContainerWalker> GetNextToken( EStructDeserializerBackendTokens& OutToken, FString& PropertyName ) = 0;
	virtual bool ReadProperty(TSharedPtr<FCEFJSScripting> Scripting, FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex) = 0;

	TSharedPtr<ICefContainerWalker> Parent;
};

class FCefListValueWalker
	: public ICefContainerWalker
{
public:
	FCefListValueWalker(TSharedPtr<ICefContainerWalker> InParent, CefRefPtr<CefListValue> InList)
		: ICefContainerWalker(InParent)
		, List(InList)
		, Index(-2)
	{}

	virtual TSharedPtr<ICefContainerWalker> GetNextToken( EStructDeserializerBackendTokens& OutToken, FString& PropertyName ) override;
	virtual bool ReadProperty(TSharedPtr<FCEFJSScripting> Scripting, FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex ) override;

	CefRefPtr<CefListValue> List;
	size_t Index;
};

class FCefDictionaryValueWalker
	: public ICefContainerWalker
{
public:
	FCefDictionaryValueWalker(TSharedPtr<ICefContainerWalker> InParent, CefRefPtr<CefDictionaryValue> InDictionary)
		: ICefContainerWalker(InParent)
		, Dictionary(InDictionary)
		, Index(-2)
	{
		Dictionary->GetKeys(Keys);
	}

	virtual TSharedPtr<ICefContainerWalker> GetNextToken( EStructDeserializerBackendTokens& OutToken, FString& PropertyName ) override;
	virtual bool ReadProperty(TSharedPtr<FCEFJSScripting> Scripting, FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex ) override;

private:
	CefRefPtr<CefDictionaryValue> Dictionary;
	size_t Index;
	CefDictionaryValue::KeyList Keys;
};

/**
 * Implements a writer for UStruct serialization using CefDictionary.
 */
class FCEFJSStructDeserializerBackend
	: public IStructDeserializerBackend
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param Archive The archive to deserialize from.
	 */
	FCEFJSStructDeserializerBackend(TSharedPtr<FCEFJSScripting> InScripting, CefRefPtr<CefDictionaryValue> InDictionary)
		: Scripting(InScripting)
		, Walker(new FCefDictionaryValueWalker(nullptr, InDictionary))
		, CurrentPropertyName()
	{ }

public:

	// IStructDeserializerBackend interface

	virtual const FString& GetCurrentPropertyName() const override;
	virtual FString GetDebugString() const override;
	virtual const FString& GetLastErrorMessage() const override;
	virtual bool GetNextToken( EStructDeserializerBackendTokens& OutToken ) override;
	virtual bool ReadProperty( FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex ) override;
	virtual void SkipArray() override;
	virtual void SkipStructure() override;

private:
	TSharedPtr<FCEFJSScripting> Scripting;
	/** Holds the source CEF dictionary containing a serialized verion of the structure. */
	TSharedPtr<ICefContainerWalker> Walker;
	FString CurrentPropertyName;
};

#endif
