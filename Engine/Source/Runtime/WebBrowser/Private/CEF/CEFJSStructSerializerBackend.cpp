// Copyright Epic Games, Inc. All Rights Reserved.

#include "CEF/CEFJSStructSerializerBackend.h"
#if WITH_CEF3

#include "UObject/EnumProperty.h"
#include "UObject/TextProperty.h"
#include "UObject/PropertyPortFlags.h"
#include "Misc/CommandLine.h"

static FString GetBindingName(const TSharedPtr<FCEFJSScripting>& Scripting, const FProperty* ValueProperty)
{
	return Scripting->GetBindingName(ValueProperty);
}

/* Private methods
 *****************************************************************************/

void FCEFJSStructSerializerBackend::AddNull(const FStructSerializerState& State)
{
	StackItem& Current = Stack.Top();
	switch (Current.Kind) {
		case StackItem::STYPE_DICTIONARY:
			Current.DictionaryValue->SetNull(TCHAR_TO_WCHAR(*GetBindingName(Scripting, State.ValueProperty)));
			break;
		case StackItem::STYPE_LIST:
			Current.ListValue->SetNull(Current.ListValue->GetSize());
		break;
	}
}


void FCEFJSStructSerializerBackend::Add(const FStructSerializerState& State, bool Value)
{
	StackItem& Current = Stack.Top();
	switch (Current.Kind) {
		case StackItem::STYPE_DICTIONARY:
			Current.DictionaryValue->SetBool(TCHAR_TO_WCHAR(*GetBindingName(Scripting, State.ValueProperty)), Value);
			break;
		case StackItem::STYPE_LIST:
			Current.ListValue->SetBool(Current.ListValue->GetSize(), Value);
		break;
	}
}


void FCEFJSStructSerializerBackend::Add(const FStructSerializerState& State, int32 Value)
{
	StackItem& Current = Stack.Top();
	switch (Current.Kind) {
		case StackItem::STYPE_DICTIONARY:
			Current.DictionaryValue->SetInt(TCHAR_TO_WCHAR(*GetBindingName(Scripting, State.ValueProperty)), Value);
			break;
		case StackItem::STYPE_LIST:
			Current.ListValue->SetInt(Current.ListValue->GetSize(), Value);
		break;
	}
}


void FCEFJSStructSerializerBackend::Add(const FStructSerializerState& State, double Value)
{
	StackItem& Current = Stack.Top();
	switch (Current.Kind) {
		case StackItem::STYPE_DICTIONARY:
			Current.DictionaryValue->SetDouble(TCHAR_TO_WCHAR(*GetBindingName(Scripting, State.ValueProperty)), Value);
			break;
		case StackItem::STYPE_LIST:
			Current.ListValue->SetDouble(Current.ListValue->GetSize(), Value);
		break;
	}
}


void FCEFJSStructSerializerBackend::Add(const FStructSerializerState& State, FString Value)
{
	StackItem& Current = Stack.Top();
	switch (Current.Kind) {
		case StackItem::STYPE_DICTIONARY:
			Current.DictionaryValue->SetString(TCHAR_TO_WCHAR(*GetBindingName(Scripting, State.ValueProperty)), TCHAR_TO_WCHAR(*Value));
			break;
		case StackItem::STYPE_LIST:
			Current.ListValue->SetString(Current.ListValue->GetSize(), TCHAR_TO_WCHAR(*Value));
		break;
	}
}


void FCEFJSStructSerializerBackend::Add(const FStructSerializerState& State, UObject* Value)
{
	StackItem& Current = Stack.Top();
	switch (Current.Kind) {
		case StackItem::STYPE_DICTIONARY:
			Current.DictionaryValue->SetDictionary(TCHAR_TO_WCHAR(*GetBindingName(Scripting, State.ValueProperty)), Scripting->ConvertObject(Value));
			break;
		case StackItem::STYPE_LIST:
			Current.ListValue->SetDictionary(Current.ListValue->GetSize(), Scripting->ConvertObject(Value));
		break;
	}
}


/* IStructSerializerBackend interface
 *****************************************************************************/

void FCEFJSStructSerializerBackend::BeginArray(const FStructSerializerState& State)
{
	CefRefPtr<CefListValue> ListValue = CefListValue::Create();
	Stack.Push(StackItem(GetBindingName(Scripting, State.ValueProperty), ListValue));
}


void FCEFJSStructSerializerBackend::BeginStructure(const FStructSerializerState& State)
{
	if (State.KeyProperty != nullptr)
	{
		FString KeyString;
		State.KeyProperty->ExportTextItem_Direct(KeyString, State.KeyData, nullptr, nullptr, PPF_None);

		CefRefPtr<CefDictionaryValue> DictionaryValue = CefDictionaryValue::Create();
		Stack.Push(StackItem(KeyString, DictionaryValue));
	}
	else if (State.ValueProperty != nullptr)
	{
		CefRefPtr<CefDictionaryValue> DictionaryValue = CefDictionaryValue::Create();
		Stack.Push(StackItem(GetBindingName(Scripting, State.ValueProperty), DictionaryValue));
	}
	else
	{
		Result = CefDictionaryValue::Create();
		Stack.Push(StackItem(FString(), Result));
	}
}


void FCEFJSStructSerializerBackend::EndArray(const FStructSerializerState& /*State*/)
{
	StackItem Previous = Stack.Pop();
	check(Previous.Kind == StackItem::STYPE_LIST);
	check(Stack.Num() > 0); // The root level object is always a struct
	StackItem& Current = Stack.Top();

	switch (Current.Kind) {
		case StackItem::STYPE_DICTIONARY:
			Current.DictionaryValue->SetList(TCHAR_TO_WCHAR(*Previous.Name), Previous.ListValue);
			break;
		case StackItem::STYPE_LIST:
			Current.ListValue->SetList(Current.ListValue->GetSize(), Previous.ListValue);
		break;
	}
}


void FCEFJSStructSerializerBackend::EndStructure(const FStructSerializerState& /*State*/)
{
	StackItem Previous = Stack.Pop();
	check(Previous.Kind == StackItem::STYPE_DICTIONARY);

	if (Stack.Num() > 0)
	{
		StackItem& Current = Stack.Top();

		switch (Current.Kind) {
			case StackItem::STYPE_DICTIONARY:
				Current.DictionaryValue->SetDictionary(TCHAR_TO_WCHAR(*Previous.Name), Previous.DictionaryValue);
				break;
			case StackItem::STYPE_LIST:
				Current.ListValue->SetDictionary(Current.ListValue->GetSize(), Previous.DictionaryValue);
			break;
		}
	}
	else
	{
		check(Result == Previous.DictionaryValue);
	}
}


void FCEFJSStructSerializerBackend::WriteComment(const FString& Comment)
{
	// Cef values do not support comments
}


void FCEFJSStructSerializerBackend::WriteProperty(const FStructSerializerState& State, int32 ArrayIndex)
{
	// booleans
	if (State.FieldType == FBoolProperty::StaticClass())
	{
		Add(State, CastFieldChecked<FBoolProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}

	// unsigned bytes & enumerations
	else if (State.FieldType == FEnumProperty::StaticClass())
	{
		FEnumProperty* EnumProperty = CastFieldChecked<FEnumProperty>(State.ValueProperty);

		Add(State, EnumProperty->GetEnum()->GetNameStringByValue(EnumProperty->GetUnderlyingProperty()->GetSignedIntPropertyValue(EnumProperty->ContainerPtrToValuePtr<void>(State.ValueData, ArrayIndex))));
	}
	else if (State.FieldType == FByteProperty::StaticClass())
	{
		FByteProperty* ByteProperty = CastFieldChecked<FByteProperty>(State.ValueProperty);

		if (ByteProperty->IsEnum())
		{
			Add(State, ByteProperty->Enum->GetNameStringByValue(ByteProperty->GetPropertyValue_InContainer(State.ValueData, ArrayIndex)));
		}
		else
		{
			Add(State, (double)ByteProperty->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
		}
	}

	// floating point numbers
	else if (State.FieldType == FDoubleProperty::StaticClass())
	{
		Add(State, CastFieldChecked<FDoubleProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}
	else if (State.FieldType == FFloatProperty::StaticClass())
	{
		Add(State, CastFieldChecked<FFloatProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}

	// signed integers
	else if (State.FieldType == FIntProperty::StaticClass())
	{
		Add(State, (int32)CastFieldChecked<FIntProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}
	else if (State.FieldType == FInt8Property::StaticClass())
	{
		Add(State, (int32)CastFieldChecked<FInt8Property>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}
	else if (State.FieldType == FInt16Property::StaticClass())
	{
		Add(State, (int32)CastFieldChecked<FInt16Property>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}
	else if (State.FieldType == FInt64Property::StaticClass())
	{
		Add(State, (double)CastFieldChecked<FInt64Property>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}

	// unsigned integers
	else if (State.FieldType == FUInt16Property::StaticClass())
	{
		Add(State, (int32)CastFieldChecked<FUInt16Property>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}
	else if (State.FieldType == FUInt32Property::StaticClass())
	{
		Add(State, (double)CastFieldChecked<FUInt32Property>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}
	else if (State.FieldType == FUInt64Property::StaticClass())
	{
		Add(State, (double)CastFieldChecked<FUInt64Property>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}

	// names & strings
	else if (State.FieldType == FNameProperty::StaticClass())
	{
		Add(State, CastFieldChecked<FNameProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex).ToString());
	}
	else if (State.FieldType == FStrProperty::StaticClass())
	{
		Add(State, CastFieldChecked<FStrProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}
	else if (State.FieldType == FTextProperty::StaticClass())
	{
		Add(State, CastFieldChecked<FTextProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex).ToString());
	}

	// classes & objects
	else if (FClassProperty* ClassProperty = CastField<FClassProperty>(State.ValueProperty))
	{
		Add(State, ClassProperty->GetPropertyValue_InContainer(State.ValueData, ArrayIndex)->GetPathName());
	}
	else if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(State.ValueProperty))
	{
		Add(State, ObjectProperty->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}

	// unsupported property type
	else
	{
		GLog->Logf(ELogVerbosity::Warning, TEXT("FCEFJSStructSerializerBackend: Property %s cannot be serialized, because its type (%s) is not supported"), *State.ValueProperty->GetName(), *State.ValueType->GetName());
	}
}


#endif
