// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_CEF3

#include "Containers/UnrealString.h"
#include "WebBrowserSingleton.h"
#include "UObject/UnrealType.h"
#include "IStructSerializerBackend.h"
#include "CEFJSScripting.h"

#if PLATFORM_WINDOWS
	#include "Windows/AllowWindowsPlatformTypes.h"
#endif

#pragma push_macro("OVERRIDE")
#undef OVERRIDE // cef headers provide their own OVERRIDE macro
THIRD_PARTY_INCLUDES_START
#if PLATFORM_APPLE
PRAGMA_DISABLE_DEPRECATION_WARNINGS
#endif
#include "include/cef_values.h"
#if PLATFORM_APPLE
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
THIRD_PARTY_INCLUDES_END
#pragma pop_macro("OVERRIDE")

#if PLATFORM_WINDOWS
	#include "Windows/HideWindowsPlatformTypes.h"
#endif

#endif

class FCEFJSScripting;
class IStructSerializerBackend;
struct FStructSerializerState;
class FWebJSScripting;
class UObject;
class FProperty;
class UStruct;

#if WITH_CEF3

/**
 * Implements a writer for UStruct serialization using CefDictionary.
 */
class FCEFJSStructSerializerBackend
	: public IStructSerializerBackend
{
public:

	/**
	 * Creates and initializes a new instance.
	 *	 */
	FCEFJSStructSerializerBackend(TSharedPtr<FCEFJSScripting> InScripting)
		: Scripting(InScripting), Stack(), Result()
	{ }

	CefRefPtr<CefDictionaryValue> GetResult()
	{
		return Result;
	}

public:

	// IStructSerializerBackend interface

	virtual void BeginArray(const FStructSerializerState& State) override;
	virtual void BeginStructure(const FStructSerializerState& State) override;
	virtual void EndArray(const FStructSerializerState& State) override;
	virtual void EndStructure(const FStructSerializerState& State) override;
	virtual void WriteComment(const FString& Comment) override;
	virtual void WriteProperty(const FStructSerializerState& State, int32 ArrayIndex = 0) override;

private:

	struct StackItem
	{
		enum {STYPE_DICTIONARY, STYPE_LIST} Kind;
		FString Name;
		CefRefPtr<CefDictionaryValue> DictionaryValue;
		CefRefPtr<CefListValue> ListValue;

		StackItem(const FString& InName, CefRefPtr<CefDictionaryValue> InDictionary)
			: Kind(STYPE_DICTIONARY)
			, Name(InName)
			, DictionaryValue(InDictionary)
			, ListValue()
		{}

		StackItem(const FString& InName, CefRefPtr<CefListValue> InList)
			: Kind(STYPE_LIST)
			, Name(InName)
			, DictionaryValue()
			, ListValue(InList)
		{}

	};

	TSharedPtr<FCEFJSScripting> Scripting;
	TArray<StackItem> Stack;
	CefRefPtr<CefDictionaryValue> Result;

	void AddNull(const FStructSerializerState& State);
	void Add(const FStructSerializerState& State, bool Value);
	void Add(const FStructSerializerState& State, int32 Value);
	void Add(const FStructSerializerState& State, double Value);
	void Add(const FStructSerializerState& State, FString Value);
	void Add(const FStructSerializerState& State, UObject* Value);
};


#endif
