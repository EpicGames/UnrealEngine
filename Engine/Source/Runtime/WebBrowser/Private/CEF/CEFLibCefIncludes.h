// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_CEF3

#ifndef OVERRIDE
#	define OVERRIDE override
#endif //OVERRIDE

#include "HAL/Platform.h"

#if PLATFORM_WINDOWS
#	include "Windows/AllowWindowsPlatformTypes.h"
#	include "Windows/AllowWindowsPlatformAtomics.h"
#endif

THIRD_PARTY_INCLUDES_START

#	if PLATFORM_APPLE
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
#	endif //PLATFORM_APPLE

#	pragma push_macro("OVERRIDE")
#		undef OVERRIDE // cef headers provide their own OVERRIDE macro

#		include "include/cef_app.h"
#		include "include/cef_client.h"
#		include "include/cef_request.h"
#		include "include/cef_task.h"
#		include "include/cef_render_handler.h"
#		include "include/cef_resource_handler.h"
#		include "include/cef_resource_request_handler.h"
#		include "include/cef_request_context_handler.h"
#		include "include/cef_jsdialog_handler.h"
#		include "include/cef_scheme.h"
#		include "include/cef_origin_whitelist.h"
#		include "include/cef_version.h"
#		include "include/internal/cef_ptr.h"

#	pragma pop_macro("OVERRIDE")

#	if PLATFORM_APPLE
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
#	endif //PLATFORM_APPLE

THIRD_PARTY_INCLUDES_END

#if PLATFORM_WINDOWS
#	include "Windows/HideWindowsPlatformAtomics.h"
#	include "Windows/HideWindowsPlatformTypes.h"
#endif

#endif //WITH_CEF3
