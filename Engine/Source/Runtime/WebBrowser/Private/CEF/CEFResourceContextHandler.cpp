// Copyright Epic Games, Inc. All Rights Reserved.

#include "CEF/CEFResourceContextHandler.h"
#include "HAL/PlatformApplicationMisc.h"

#if WITH_CEF3

//#define DEBUG_ONBEFORELOAD // Debug print beforebrowse steps

#include "CEFBrowserClosureTask.h"
#include "WebBrowserSingleton.h"

#define LOCTEXT_NAMESPACE "WebBrowserHandler"

FCEFResourceContextHandler::FCEFResourceContextHandler(FWebBrowserSingleton * InOwningSingleton) :
	OwningSingleton(InOwningSingleton)
{ }


FString ResourceTypeToString(const CefRequest::ResourceType& Type)
{
	const static FString ResourceType_MainFrame("MAIN_FRAME");
	const static FString ResourceType_SubFrame("SUB_FRAME");
	const static FString ResourceType_StyleSheet("STYLESHEET");
	const static FString ResourceType_Script("SCRIPT");
	const static FString ResourceType_Image("IMAGE");
	const static FString ResourceType_FontResource("FONT_RESOURCE");
	const static FString ResourceType_SubResource("SUB_RESOURCE");
	const static FString ResourceType_Object("OBJECT");
	const static FString ResourceType_Media("MEDIA");
	const static FString ResourceType_Worker("WORKER");
	const static FString ResourceType_SharedWorker("SHARED_WORKER");
	const static FString ResourceType_Prefetch("PREFETCH");
	const static FString ResourceType_Favicon("FAVICON");
	const static FString ResourceType_XHR("XHR");
	const static FString ResourceType_Ping("PING");
	const static FString ResourceType_ServiceWorker("SERVICE_WORKER");
	const static FString ResourceType_CspReport("CSP_REPORT");
	const static FString ResourceType_PluginResource("PLUGIN_RESOURCE");
	const static FString ResourceType_Unknown("UNKNOWN");

	FString TypeStr;
	switch (Type)
	{
	case CefRequest::ResourceType::RT_MAIN_FRAME:
		TypeStr = ResourceType_MainFrame;
		break;
	case CefRequest::ResourceType::RT_SUB_FRAME:
		TypeStr = ResourceType_SubFrame;
		break;
	case CefRequest::ResourceType::RT_STYLESHEET:
		TypeStr = ResourceType_StyleSheet;
		break;
	case CefRequest::ResourceType::RT_SCRIPT:
		TypeStr = ResourceType_Script;
		break;
	case CefRequest::ResourceType::RT_IMAGE:
		TypeStr = ResourceType_Image;
		break;
	case CefRequest::ResourceType::RT_FONT_RESOURCE:
		TypeStr = ResourceType_FontResource;
		break;
	case CefRequest::ResourceType::RT_SUB_RESOURCE:
		TypeStr = ResourceType_SubResource;
		break;
	case CefRequest::ResourceType::RT_OBJECT:
		TypeStr = ResourceType_Object;
		break;
	case CefRequest::ResourceType::RT_MEDIA:
		TypeStr = ResourceType_Media;
		break;
	case CefRequest::ResourceType::RT_WORKER:
		TypeStr = ResourceType_Worker;
		break;
	case CefRequest::ResourceType::RT_SHARED_WORKER:
		TypeStr = ResourceType_SharedWorker;
		break;
	case CefRequest::ResourceType::RT_PREFETCH:
		TypeStr = ResourceType_Prefetch;
		break;
	case CefRequest::ResourceType::RT_FAVICON:
		TypeStr = ResourceType_Favicon;
		break;
	case CefRequest::ResourceType::RT_XHR:
		TypeStr = ResourceType_XHR;
		break;
	case CefRequest::ResourceType::RT_PING:
		TypeStr = ResourceType_Ping;
		break;
	case CefRequest::ResourceType::RT_SERVICE_WORKER:
		TypeStr = ResourceType_ServiceWorker;
		break;
	case CefRequest::ResourceType::RT_CSP_REPORT:
		TypeStr = ResourceType_CspReport;
		break;
	case CefRequest::ResourceType::RT_PLUGIN_RESOURCE:
		TypeStr = ResourceType_PluginResource;
		break;
	default:
		TypeStr = ResourceType_Unknown;
		break;
	}
	return TypeStr;
}

CefResourceRequestHandler::ReturnValue FCEFResourceContextHandler::OnBeforeResourceLoad(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefRequest> Request, CefRefPtr<CefCallback> Callback)
{
#ifdef DEBUG_ONBEFORELOAD
	auto url = Request->GetURL();
	auto type = Request->GetResourceType();
	auto method = Request->GetMethod();
	if (type == CefRequest::ResourceType::RT_MAIN_FRAME || type == CefRequest::ResourceType::RT_XHR || type == CefRequest::ResourceType::RT_SUB_RESOURCE)
	{
		GLog->Logf(ELogVerbosity::Display, TEXT("FCEFResourceContextHandler::OnBeforeResourceLoad :%s type:%s method:%s"), url.c_str(), *ResourceTypeToString(type), method.c_str());
	}
#endif

	if (Request->IsReadOnly())
	{
		// we can't alter this request so just allow it through
		return RV_CONTINUE;
	}

	// Current thread is IO thread. We need to invoke BrowserWindow->GetResourceContent on the UI (aka Game) thread:
	CefPostTask(TID_UI, new FCEFBrowserClosureTask(this, [=, this]()
	{
		const FString LanguageHeaderText("Accept-Language");
		const FString LocaleCode = FWebBrowserSingleton::GetCurrentLocaleCode();
		CefRequest::HeaderMap HeaderMap;
		Request->GetHeaderMap(HeaderMap);
		auto LanguageHeader = HeaderMap.find(TCHAR_TO_WCHAR(*LanguageHeaderText));
		if (LanguageHeader != HeaderMap.end())
		{
			(*LanguageHeader).second = TCHAR_TO_WCHAR(*LocaleCode);
		}
		else
		{
			HeaderMap.insert(std::pair<CefString, CefString>(TCHAR_TO_WCHAR(*LanguageHeaderText), TCHAR_TO_WCHAR(*LocaleCode)));
		}

		bool bAllowCredentials = false;
		if (OwningSingleton != nullptr)
		{
			bAllowCredentials = OwningSingleton->URLRequestAllowsCredentials(WCHAR_TO_TCHAR(Request->GetURL().ToWString().c_str()));
		}
		FContextRequestHeaders AdditionalHeaders;
		BeforeResourceLoadDelegate.ExecuteIfBound(WCHAR_TO_TCHAR(Request->GetURL().ToWString().c_str()), ResourceTypeToString(Request->GetResourceType()), AdditionalHeaders, bAllowCredentials);

		for (auto Iter = AdditionalHeaders.CreateConstIterator(); Iter; ++Iter)
		{
			const FString Header = Iter.Key();
			const FString Value = Iter.Value();
			HeaderMap.insert(std::pair<CefString, CefString>(TCHAR_TO_WCHAR(*Header), TCHAR_TO_WCHAR(*Value)));
		}

		Request->SetHeaderMap(HeaderMap);

		Callback->Continue();
	}));

	// Tell CEF that we're handling this asynchronously.
	return RV_CONTINUE_ASYNC;
}



CefRefPtr<CefResourceRequestHandler> FCEFResourceContextHandler::GetResourceRequestHandler( CefRefPtr<CefBrowser> browser, CefRefPtr<CefFrame> frame,
	CefRefPtr<CefRequest> request, bool is_navigation, bool is_download, const CefString& request_initiator, bool& disable_default_handling) 
{
#ifdef DEBUG_ONBEFORELOAD
	auto url = request->GetURL();
	auto type = request->GetResourceType();
	if (type == CefRequest::ResourceType::RT_MAIN_FRAME || type == CefRequest::ResourceType::RT_XHR || type == CefRequest::ResourceType::RT_SUB_RESOURCE)
	{
		GLog->Logf(ELogVerbosity::Display, TEXT("FCEFResourceContextHandler::GetResourceRequestHandler :%s type:%s"), url.c_str(), *ResourceTypeToString(type));
	}
#endif
	return this;
}

/*void FCEFResourceContextHandler::AddLoadCallback(IWebBrowserOnBeforeResourceLoadHandler* pCallback)
{
	callbacks.Add(pCallback);
}*/
#undef LOCTEXT_NAMESPACE

#endif // WITH_CEF
