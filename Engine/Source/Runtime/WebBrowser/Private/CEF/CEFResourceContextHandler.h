// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_CEF3

#include "IWebBrowserResourceLoader.h"

#include "CEFLibCefIncludes.h"


FString ResourceTypeToString(const CefRequest::ResourceType& Type);

class FWebBrowserSingleton;

/**
 * Implements CEF Request handling for when a browser window is still being constructed
 */
class FCEFResourceContextHandler :
	  public CefRequestContextHandler
	, public CefResourceRequestHandler
{
public:

	/** Default constructor. */
	FCEFResourceContextHandler(FWebBrowserSingleton *InOwningSingleton);

public:

	// CefResourceRequestHandler Interface
	virtual CefResourceRequestHandler::ReturnValue OnBeforeResourceLoad(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefRequest> Request, CefRefPtr<CefCallback> Callback) override;

	// CefRequestContextHandler Interface
	virtual CefRefPtr<CefResourceRequestHandler> GetResourceRequestHandler(CefRefPtr<CefBrowser> browser, CefRefPtr<CefFrame> frame, CefRefPtr<CefRequest> request, bool is_navigation, bool is_download, const CefString& request_initiator, bool& disable_default_handling) override;


public:
	FOnBeforeContextResourceLoadDelegate& OnBeforeLoad()
	{
		return BeforeResourceLoadDelegate;
	}

private:
	/** Delegate for handling resource load requests */
	FOnBeforeContextResourceLoadDelegate BeforeResourceLoadDelegate;

	/** Singleton that owns this context handler, so we can lookup browser objects from it */
	FWebBrowserSingleton* OwningSingleton;

	// Include the default reference counting implementation.
	IMPLEMENT_REFCOUNTING(FCEFResourceContextHandler);
};

#endif
