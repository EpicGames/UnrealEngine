// Copyright Epic Games, Inc. All Rights Reserved.

#include "CEF/CEFSchemeHandler.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "IWebBrowserSchemeHandler.h"

#if WITH_CEF3

class FHandlerHeaderSetter
	: public IWebBrowserSchemeHandler::IHeaders
{
public:
	FHandlerHeaderSetter(CefRefPtr<CefResponse>& InResponse, int64_t& InContentLength, CefString& InRedirectUrl)
		: Response(InResponse)
		, ContentLength(InContentLength)
		, RedirectUrl(InRedirectUrl)
		, StatusCode(INDEX_NONE)
	{
	}

	virtual ~FHandlerHeaderSetter()
	{
		if (Headers.size() > 0)
		{
			Response->SetHeaderMap(Headers);
		}
		if (StatusCode != INDEX_NONE)
		{
			Response->SetStatus(StatusCode);
		}
		if (MimeType.length() > 0)
		{
			Response->SetMimeType(MimeType);
		}
	}

	virtual void SetMimeType(const TCHAR* InMimeType) override
	{
		MimeType = TCHAR_TO_WCHAR(InMimeType);
	}

	virtual void SetStatusCode(int32 InStatusCode) override
	{
		StatusCode = InStatusCode;
	}

	virtual void SetContentLength(int32 InContentLength) override
	{
		ContentLength = InContentLength;
	}

	virtual void SetRedirect(const TCHAR* InRedirectUrl) override
	{
		RedirectUrl = TCHAR_TO_WCHAR(InRedirectUrl);
	}

	virtual void SetHeader(const TCHAR* Key, const TCHAR* Value) override
	{
		Headers.insert(std::make_pair(CefString(TCHAR_TO_WCHAR(Key)), CefString(TCHAR_TO_WCHAR(Value))));
	}

private:
	CefRefPtr<CefResponse>& Response;
	int64_t& ContentLength;
	CefString& RedirectUrl;
	CefResponse::HeaderMap Headers;
	CefString MimeType;
	int32 StatusCode;
};

class FCefSchemeHandler
	: public CefResourceHandler
{
public:
	FCefSchemeHandler(TUniquePtr<IWebBrowserSchemeHandler>&& InHandlerImplementation)
		: HandlerImplementation(MoveTemp(InHandlerImplementation))
	{
	}

	virtual ~FCefSchemeHandler()
	{
	}

	// Begin CefResourceHandler interface.
	virtual bool ProcessRequest(CefRefPtr<CefRequest> Request, CefRefPtr<CefCallback> Callback) override
	{
		if (HandlerImplementation.IsValid())
		{
			return HandlerImplementation->ProcessRequest(
				WCHAR_TO_TCHAR(Request->GetMethod().ToWString().c_str()),
				WCHAR_TO_TCHAR(Request->GetURL().ToWString().c_str()),
				FSimpleDelegate::CreateLambda([Callback](){ Callback->Continue(); })
			);
		}
		return false;
	}

	virtual void GetResponseHeaders(CefRefPtr<CefResponse> Response, int64_t& ResponseLength, CefString& RedirectUrl) override
	{
		if (ensure(HandlerImplementation.IsValid()))
		{
			FHandlerHeaderSetter Headers(Response, ResponseLength, RedirectUrl);
			HandlerImplementation->GetResponseHeaders(Headers);
		}
	}

	virtual bool ReadResponse(void* DataOut, int BytesToRead, int& BytesRead, CefRefPtr<CefCallback> Callback) override
	{
		if (ensure(HandlerImplementation.IsValid()))
		{
			return HandlerImplementation->ReadResponse(
				(uint8*)DataOut, 
				BytesToRead,
				BytesRead,
				FSimpleDelegate::CreateLambda([Callback](){ Callback->Continue(); })
			);
		}
		BytesRead = 0;
		return false;
	}

	virtual void Cancel() override
	{
		if (HandlerImplementation.IsValid())
		{
			HandlerImplementation->Cancel();
		}
	}
	// End CefResourceHandler interface.

private:
	TUniquePtr<IWebBrowserSchemeHandler> HandlerImplementation;

	// Include CEF ref counting.
	IMPLEMENT_REFCOUNTING(FCefSchemeHandler);
};


class FCefSchemeHandlerFactory
	: public CefSchemeHandlerFactory
{
public:

	FCefSchemeHandlerFactory(IWebBrowserSchemeHandlerFactory* InWebBrowserSchemeHandlerFactory)
		: WebBrowserSchemeHandlerFactory(InWebBrowserSchemeHandlerFactory)
	{
	}

	// Begin CefSchemeHandlerFactory interface.
	virtual CefRefPtr<CefResourceHandler> Create(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, const CefString& Scheme, CefRefPtr<CefRequest> Request) override
	{
		return new FCefSchemeHandler(WebBrowserSchemeHandlerFactory->Create(
			WCHAR_TO_TCHAR(Request->GetMethod().ToWString().c_str()),
			WCHAR_TO_TCHAR(Request->GetURL().ToWString().c_str())));
	}
	// End CefSchemeHandlerFactory interface.

	bool IsUsing(IWebBrowserSchemeHandlerFactory* InWebBrowserSchemeHandlerFactory)
	{
		return WebBrowserSchemeHandlerFactory == InWebBrowserSchemeHandlerFactory;
	}

private:
	IWebBrowserSchemeHandlerFactory* WebBrowserSchemeHandlerFactory;

	// Include CEF ref counting.
	IMPLEMENT_REFCOUNTING(FCefSchemeHandlerFactory);
};

void FCefSchemeHandlerFactories::AddSchemeHandlerFactory(FString Scheme, FString Domain, IWebBrowserSchemeHandlerFactory* WebBrowserSchemeHandlerFactory)
{
	checkf(WebBrowserSchemeHandlerFactory != nullptr, TEXT("WebBrowserSchemeHandlerFactory must be provided."));
	CefRefPtr<CefSchemeHandlerFactory> Factory = new FCefSchemeHandlerFactory(WebBrowserSchemeHandlerFactory);
	CefRegisterSchemeHandlerFactory(TCHAR_TO_WCHAR(*Scheme), TCHAR_TO_WCHAR(*Domain), Factory);
	SchemeHandlerFactories.Emplace(MoveTemp(Scheme), MoveTemp(Domain), MoveTemp(Factory));
}

void FCefSchemeHandlerFactories::RemoveSchemeHandlerFactory(IWebBrowserSchemeHandlerFactory* WebBrowserSchemeHandlerFactory)
{
	checkf(WebBrowserSchemeHandlerFactory != nullptr, TEXT("WebBrowserSchemeHandlerFactory must be provided."));
	SchemeHandlerFactories.RemoveAll([WebBrowserSchemeHandlerFactory](const FFactory& Element)
	{
		return ((FCefSchemeHandlerFactory*)Element.Factory.get())->IsUsing(WebBrowserSchemeHandlerFactory);
	});
}

void FCefSchemeHandlerFactories::RegisterFactoriesWith(CefRefPtr<CefRequestContext>& Context)
{
	if (Context)
	{
		for (const FFactory& SchemeHandlerFactory : SchemeHandlerFactories)
		{
			Context->RegisterSchemeHandlerFactory(TCHAR_TO_WCHAR(*SchemeHandlerFactory.Scheme), TCHAR_TO_WCHAR(*SchemeHandlerFactory.Domain), SchemeHandlerFactory.Factory);
		}
	}
}

FCefSchemeHandlerFactories::FFactory::FFactory(FString InScheme, FString InDomain, CefRefPtr<CefSchemeHandlerFactory> InFactory)
	: Scheme(MoveTemp(InScheme))
	, Domain(MoveTemp(InDomain))
	, Factory(MoveTemp(InFactory))
{
}

#endif
