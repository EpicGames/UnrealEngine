// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IWebBrowserSchemeHandler.h"

#if WITH_CEF3
#include "CEFLibCefIncludes.h"

/**
 * Implementation for managing CEF custom scheme handlers.
 */
class FCefSchemeHandlerFactories
{
public:
	/**
	 * Adds a custom scheme handler factory, for a given scheme and domain. The domain is ignored if the scheme is not a browser built in scheme,
	 * and all requests will go through this factory.
	 * @param Scheme                            The scheme name to handle.
	 * @param Domain                            The domain name to handle on the scheme. Ignored if scheme is not a built in scheme.
	 * @param WebBrowserSchemeHandlerFactory    The factory implementation for creating request handlers for this scheme.
	 */
	void AddSchemeHandlerFactory(FString Scheme, FString Domain, IWebBrowserSchemeHandlerFactory* WebBrowserSchemeHandlerFactory);

	/**
	 * Remove a custom scheme handler factory. The factory may still be used by existing open browser windows, but will no longer be provided for new ones.
	 * @param WebBrowserSchemeHandlerFactory    The factory implementation to remove.
	 */
	void RemoveSchemeHandlerFactory(IWebBrowserSchemeHandlerFactory* WebBrowserSchemeHandlerFactory);

	/**
	 * Register all scheme handler factories with the provided request context.
	 * @param Context   The context.
	 */
	void RegisterFactoriesWith(CefRefPtr<CefRequestContext>& Context);

private:
	/**
	 * A struct to wrap storage of a factory with it's provided scheme and domain, inc ref counting for the cef representation.
	 */
	struct FFactory
	{
	public:
		FFactory(FString Scheme, FString Domain, CefRefPtr<CefSchemeHandlerFactory> Factory);
		FString Scheme;
		FString Domain;
		CefRefPtr<CefSchemeHandlerFactory> Factory;
	};

	// Array of registered handler factories.
	TArray<FFactory> SchemeHandlerFactories;
};


#endif
