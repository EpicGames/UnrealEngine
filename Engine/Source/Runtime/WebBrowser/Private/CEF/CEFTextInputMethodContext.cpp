// Copyright Epic Games, Inc. All Rights Reserved.

#include "CEF/CEFTextInputMethodContext.h"

#if WITH_CEF3 && !PLATFORM_LINUX

#include "CEF/CEFWebBrowserWindow.h"
#include "CEFImeHandler.h"
#include "Framework/Application/SlateApplication.h"


TSharedRef<FCEFTextInputMethodContext> FCEFTextInputMethodContext::Create(const TSharedRef<FCEFImeHandler>& InOwner)
{
	return MakeShareable(new FCEFTextInputMethodContext(InOwner));
}

FCEFTextInputMethodContext::FCEFTextInputMethodContext(const TSharedRef<FCEFImeHandler>& InOwner)
	: Owner(InOwner)
	, bIsComposing(false)
	, CompositionBeginIndex(0)
	, SelectionRangeBeginIndex(0)
	, SelectionRangeLength(0)
	, SelectionCaretPosition(ECaretPosition::Ending)
{

}

void FCEFTextInputMethodContext::AbortComposition()
{
	bIsComposing = false;
	Owner->InternalCefBrowser->GetHost()->ImeCancelComposition();
	ResetComposition();
}

bool FCEFTextInputMethodContext::UpdateCachedGeometry(const FGeometry& AllottedGeometry)
{
	bool bCachedGeometryUpdated = false;
	if (CachedGeometry != AllottedGeometry)
	{
		CachedGeometry = AllottedGeometry;
		bCachedGeometryUpdated = true;
	}

	return bCachedGeometryUpdated;
}

bool FCEFTextInputMethodContext::CEFCompositionRangeChanged(const CefRange& SelectionRange, const CefRenderHandler::RectList& CharacterBounds)
{
	if (bIsComposing)
	{
		if (CharacterBounds != CefCompositionBounds)
		{
			CefCompositionBounds = CharacterBounds;
			return true;
		}
	}
	return false;
}

bool FCEFTextInputMethodContext::CEFTextSelectionChanged(const CefString& SelectedText, const CefRange& SelectionRange)
{
	// We get selection changes while composing as the caret follows the ongoing composition, these can be ignored.
	// But any other selection change means the user clicked somewhere in the text field, and these must end any ongoing composition.
	if (bIsComposing &&
		(SelectionRangeBeginIndex == SelectionRange.from || SelectionRangeBeginIndex == (SelectionRange.from + CompositionString.Len())))
	{
		return false;
	}

	SelectionRangeBeginIndex = SelectionRange.from;
	SelectionRangeLength = (SelectionRange.to - SelectionRange.from);

	// Clicking anywhere in the textfield starts a new composition sequence
	CompositionBeginIndex = SelectionRangeBeginIndex;
	CompositionSequence.Empty();
	return bIsComposing;
}

bool FCEFTextInputMethodContext::IsComposing()
{
	return bIsComposing;
}

bool FCEFTextInputMethodContext::IsReadOnly()
{
	return false;
}

uint32 FCEFTextInputMethodContext::GetTextLength()
{
	return CompositionSequence.Len();
}

void FCEFTextInputMethodContext::GetSelectionRange(uint32& BeginIndex, uint32& Length, ECaretPosition& CaretPosition)
{
	BeginIndex = SelectionRangeBeginIndex - CompositionBeginIndex; // convert from textinput index to composition string index
	Length = SelectionRangeLength;
	CaretPosition = SelectionCaretPosition;
}

void FCEFTextInputMethodContext::SetSelectionRange(const uint32 BeginIndex, const uint32 Length, const ECaretPosition CaretPosition)
{
	SelectionRangeBeginIndex = BeginIndex + CompositionBeginIndex; // convert from composition string index to textinput index
	SelectionRangeLength = Length;
	SelectionCaretPosition = CaretPosition;
}

void FCEFTextInputMethodContext::GetTextInRange(const uint32 BeginIndex, const uint32 Length, FString& OutString)
{
	OutString = CompositionSequence.Mid(BeginIndex, Length);
}

void FCEFTextInputMethodContext::SetTextInRange(const uint32 BeginIndex, const uint32 Length, const FString& InString)
{
	FString NewCompositionSequence;
	if (BeginIndex > 0)
	{
		NewCompositionSequence = CompositionSequence.Mid(0, BeginIndex);
	}

	NewCompositionSequence += InString;

	if ((int32)(BeginIndex + Length) < CompositionSequence.Len())
	{
		NewCompositionSequence += CompositionSequence.Mid(BeginIndex + Length, CompositionSequence.Len() - (BeginIndex + Length));
	}
	// this is the full sequence of chars typed with IME since we've clicked in the text field
	CompositionSequence = NewCompositionSequence;
	// this is the last char(s) that was typed with IME
	CompositionString = InString;

	CefString Str = TCHAR_TO_WCHAR(*CompositionString);
	
	Owner->InternalCefBrowser->GetHost()->ImeSetComposition(
		Str,
		{},
		CefRange(UINT32_MAX, UINT32_MAX),
		CefRange(0, CompositionString.Len())); // this selection range is within the composition string, not the whole text field!
}

int32 FCEFTextInputMethodContext::GetCharacterIndexFromPoint(const FVector2D& Point)
{
	int32 ResultIdx = INDEX_NONE;

	const FVector2D LocalPoint = CachedGeometry.AbsoluteToLocal(Point);
	CefPoint CefLocalPoint = CefPoint(FMath::RoundToInt(LocalPoint.X), FMath::RoundToInt(LocalPoint.Y));

	for (uint32 CharIdx = 0; CharIdx < CefCompositionBounds.size(); CharIdx++)
	{
		if (CefCompositionBounds[CharIdx].Contains(CefLocalPoint))
		{
			ResultIdx = CharIdx;
			break;
		}
	}
	return ResultIdx;
}

bool FCEFTextInputMethodContext::GetTextBounds(const uint32 BeginIndex, const uint32 Length, FVector2D& Position, FVector2D& Size)
{
	if (CefCompositionBounds.size() < BeginIndex ||
		CefCompositionBounds.size() < BeginIndex + Length)
	{
		if (CefCompositionBounds.size() > 0)
		{
			// Fall back to the start of the composition
			Position = CachedGeometry.LocalToAbsolute(FVector2D(CefCompositionBounds[0].x, CefCompositionBounds[0].y));
			Size = FVector2D(CefCompositionBounds[0].width, CefCompositionBounds[0].height);
			return false;
		}
		else
		{
			// We  don't have any updated composition bounds so we'll just default to the window bounds and say we are clipped.
			GetScreenBounds(Position, Size);
			return true;
		}
	}

	FVector2D LocalSpaceMin(FLT_MAX, FLT_MAX);
	FVector2D LocalSpaceMax(-FLT_MAX, -FLT_MAX);

	for (uint32 CharIdx = BeginIndex; CharIdx < BeginIndex + Length; CharIdx++)
	{
		if (LocalSpaceMin.X > CefCompositionBounds[CharIdx].x)
		{
			LocalSpaceMin.X = CefCompositionBounds[CharIdx].x;
		}

		if (LocalSpaceMax.X < CefCompositionBounds[CharIdx].x + CefCompositionBounds[CharIdx].width)
		{
			LocalSpaceMax.X = CefCompositionBounds[CharIdx].x + CefCompositionBounds[CharIdx].width;
		}

		if (LocalSpaceMin.Y > CefCompositionBounds[CharIdx].y)
		{
			LocalSpaceMin.Y = CefCompositionBounds[CharIdx].y;
		}

		if (LocalSpaceMax.Y < CefCompositionBounds[CharIdx].y + CefCompositionBounds[CharIdx].height)
		{
			LocalSpaceMax.Y = CefCompositionBounds[CharIdx].y + CefCompositionBounds[CharIdx].height;
		}
	}

	Position = CachedGeometry.LocalToAbsolute(LocalSpaceMin);
	Size = LocalSpaceMax - LocalSpaceMin;

	return false; // false means "not clipped"
}

void FCEFTextInputMethodContext::GetScreenBounds(FVector2D& Position, FVector2D& Size)
{
	Position = CachedGeometry.GetAccumulatedRenderTransform().GetTranslation();
	Size = TransformVector(CachedGeometry.GetAccumulatedRenderTransform(), CachedGeometry.GetLocalSize());
}

bool FCEFTextInputMethodContext::UpdateCachedSlateWindow()
{
	const TSharedPtr<SWidget> CachedSlateWidgetPtr = Owner->InternalBrowserSlateWidget.Pin();
	if (!CachedSlateWidgetPtr.IsValid())
	{
		return false;
	}

	TWeakPtr<SWindow> OldSlateWindow = CachedSlateWindow;
	CachedSlateWindow = FSlateApplication::Get().FindWidgetWindow(CachedSlateWidgetPtr.ToSharedRef());

	return OldSlateWindow != CachedSlateWindow;
}

TSharedPtr<FGenericWindow> FCEFTextInputMethodContext::GetWindow()
{
	const TSharedPtr<SWindow> SlateWindow = CachedSlateWindow.Pin();
	return SlateWindow.IsValid() ? SlateWindow->GetNativeWindow() : nullptr;
}

void FCEFTextInputMethodContext::BeginComposition()
{
	if (!bIsComposing)
	{
		bIsComposing = true;
	}
}

void FCEFTextInputMethodContext::UpdateCompositionRange(const int32 InBeginIndex, const uint32 InLength)
{
}

void FCEFTextInputMethodContext::EndComposition()
{
	if (bIsComposing)
	{
		bIsComposing = false;

		if (CompositionString.Len() > 0)
		{
			CefString Result = TCHAR_TO_WCHAR(*CompositionString);
			Owner->InternalCefBrowser->GetHost()->ImeCommitText(Result, CefRange(UINT32_MAX, UINT32_MAX), 0);
		}
		else
		{
			Owner->InternalCefBrowser->GetHost()->ImeCancelComposition();
		}
		ResetComposition();
	}
}

void FCEFTextInputMethodContext::ResetComposition()
{
	CompositionString.Empty();
	CefCompositionBounds.clear();
}

#endif


