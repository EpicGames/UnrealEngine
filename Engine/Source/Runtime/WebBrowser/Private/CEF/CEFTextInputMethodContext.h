// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_CEF3 && !PLATFORM_LINUX

#include "Layout/Geometry.h"
#include "Widgets/SWindow.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/AllowWindowsPlatformAtomics.h"
#endif

#pragma push_macro("OVERRIDE")
#undef OVERRIDE // cef headers provide their own OVERRIDE macro
THIRD_PARTY_INCLUDES_START
#if PLATFORM_APPLE
PRAGMA_DISABLE_DEPRECATION_WARNINGS
#endif
#include "include/cef_client.h"
#if PLATFORM_APPLE
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
THIRD_PARTY_INCLUDES_END
#pragma pop_macro("OVERRIDE")

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformAtomics.h"
#include "Windows/HideWindowsPlatformTypes.h"
#endif

#include "GenericPlatform/ITextInputMethodSystem.h"
#include "Layout/Geometry.h"

class FCEFWebBrowserWindow;
class FCEFImeHandler;
class SWindow;

class FCEFTextInputMethodContext : public ITextInputMethodContext
{
public:

	virtual ~FCEFTextInputMethodContext() = default;

	static TSharedRef<FCEFTextInputMethodContext> Create(const TSharedRef<FCEFImeHandler>& InOwner);

	void AbortComposition();

	bool UpdateCachedGeometry(const FGeometry& AllottedGeometry);
	bool UpdateCachedSlateWindow();

	bool CEFCompositionRangeChanged(const CefRange& SelectionRange, const CefRenderHandler::RectList& CharacterBounds);
	bool CEFTextSelectionChanged(const CefString& SelectedText, const CefRange& SelectionRange);

private:
	void ResetComposition();

public:
	
	// ITextInputMethodContext Interface
	virtual bool IsComposing() override;

private:
	virtual bool IsReadOnly() override;
	virtual uint32 GetTextLength() override;
	virtual void GetSelectionRange(uint32& BeginIndex, uint32& Length, ECaretPosition& CaretPosition) override;
	virtual void SetSelectionRange(const uint32 BeginIndex, const uint32 Length, const ECaretPosition CaretPosition) override;
	virtual void GetTextInRange(const uint32 BeginIndex, const uint32 Length, FString& OutString) override;
	virtual void SetTextInRange(const uint32 BeginIndex, const uint32 Length, const FString& InString) override;
	virtual int32 GetCharacterIndexFromPoint(const FVector2D& Point) override;
	virtual bool GetTextBounds(const uint32 BeginIndex, const uint32 Length, FVector2D& Position, FVector2D& Size) override;
	virtual void GetScreenBounds(FVector2D& Position, FVector2D& Size) override;
	virtual TSharedPtr<FGenericWindow> GetWindow() override;
	virtual void BeginComposition() override;
	virtual void UpdateCompositionRange(const int32 InBeginIndex, const uint32 InLength) override;
	virtual void EndComposition() override;

private:
	FCEFTextInputMethodContext(const TSharedRef<FCEFImeHandler>& InOwner);
	TSharedRef<FCEFImeHandler> Owner;
	TWeakPtr<SWindow> CachedSlateWindow;

	FGeometry CachedGeometry;
	bool bIsComposing;
	int32 CompositionBeginIndex;

	uint32 SelectionRangeBeginIndex;
	uint32 SelectionRangeLength;
	ECaretPosition SelectionCaretPosition;

	std::vector<CefRect> CefCompositionBounds;

	FString CompositionSequence;
	FString CompositionString;
};

#endif
