// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/AssertionMacros.h"
#include "Internationalization/Text.h"

#if WITH_CEF3

#include "HAL/PlatformAtomics.h"

#if PLATFORM_WINDOWS
#	include "Windows/AllowWindowsPlatformTypes.h"
#	include "Windows/AllowWindowsPlatformAtomics.h"
#endif
#pragma push_macro("OVERRIDE")
#	undef OVERRIDE // cef headers provide their own OVERRIDE macro
#if PLATFORM_APPLE
PRAGMA_DISABLE_DEPRECATION_WARNINGS
#endif
#	include "include/cef_jsdialog_handler.h"
#if PLATFORM_APPLE
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
#pragma pop_macro("OVERRIDE")
#if PLATFORM_WINDOWS
#	include "Windows/HideWindowsPlatformAtomics.h"
#	include "Windows/HideWindowsPlatformTypes.h"
#endif

#include "IWebBrowserDialog.h"

class FCEFWebBrowserDialog
	: public IWebBrowserDialog
{
public:
	virtual ~FCEFWebBrowserDialog() = default;

	// IWebBrowserDialog interface:

	virtual EWebBrowserDialogType GetType() override
	{
		return Type;
	}

	virtual const FText& GetMessageText() override
	{
		return MessageText;
	}

	virtual const FText& GetDefaultPrompt() override
	{
		return DefaultPrompt;
	}

	virtual bool IsReload() override
	{
		check(Type == EWebBrowserDialogType::Unload);
		return bIsReload;
	}

	virtual void Continue(bool Success = true, const FText& UserResponse = FText::GetEmpty()) override
	{
		check(Type == EWebBrowserDialogType::Prompt || UserResponse.IsEmpty());
		Callback->Continue(Success, TCHAR_TO_WCHAR(*UserResponse.ToString()));
	}

private:


	EWebBrowserDialogType Type;
	FText MessageText;
	FText DefaultPrompt;
	bool bIsReload;

	CefRefPtr<CefJSDialogCallback> Callback;

	// Create a dialog from OnJSDialog arguments
	FCEFWebBrowserDialog(CefJSDialogHandler::JSDialogType InDialogType, const CefString& InMessageText, const CefString& InDefaultPrompt, CefRefPtr<CefJSDialogCallback> InCallback)
		: Type((EWebBrowserDialogType)InDialogType)
		, MessageText(FText::FromString(WCHAR_TO_TCHAR(InMessageText.ToWString().c_str())))
		, DefaultPrompt(FText::FromString(WCHAR_TO_TCHAR(InDefaultPrompt.ToWString().c_str())))
		, bIsReload(false)
		, Callback(InCallback)
	{}

	// Create a dialog from OnBeforeUnloadDialog arguments
	FCEFWebBrowserDialog(const CefString& InMessageText, bool InIsReload, CefRefPtr<CefJSDialogCallback> InCallback)
		: Type(EWebBrowserDialogType::Unload)
		, MessageText(FText::FromString(WCHAR_TO_TCHAR(InMessageText.ToWString().c_str())))
		, DefaultPrompt(FText::GetEmpty())
		, bIsReload(InIsReload)
		, Callback(InCallback)
	{};

	friend class FCEFWebBrowserWindow;
};

typedef FCEFWebBrowserDialog FWebBrowserDialog;

#endif
