// Copyright Epic Games, Inc. All Rights Reserved.

#include "CEF/CEFWebBrowserWindow.h"
#include "IWebBrowserDialog.h"
#include "UObject/Stack.h"
#include "Framework/Application/SlateApplication.h"
#include "Textures/SlateUpdatableTexture.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "PlatformHttp.h"
#include "WebBrowserLog.h"

#if WITH_CEF3

#include "CEFBrowserPopupFeatures.h"
#include "CEFWebBrowserDialog.h"
#include "CEFBrowserClosureTask.h"
#include "CEFJSScripting.h"
#include "CEFImeHandler.h"
#include "CEFWebBrowserWindowRHIHelper.h"
#include "CEF3Utils.h"
#include "Async/Async.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
// Needed for character code definitions
#include <Carbon/Carbon.h>
#include <AppKit/NSEvent.h>
#endif

#if PLATFORM_WINDOWS
#include "Windows/WindowsCursor.h"
typedef FWindowsCursor FPlatformCursor;
#elif PLATFORM_MAC
#include "Mac/CocoaThread.h"
#include "Mac/MacApplication.h"
#include "Mac/MacCursor.h"
typedef FMacCursor FPlatformCursor;
#else
#endif

#if PLATFORM_LINUX

// From ui/events/keycodes/keyboard_codes_posix.h.
enum KeyboardCode {
  VKEY_BACK = 0x08,
  VKEY_TAB = 0x09,
  VKEY_BACKTAB = 0x0A,
  VKEY_CLEAR = 0x0C,
  VKEY_RETURN = 0x0D,
  VKEY_SHIFT = 0x10,
  VKEY_CONTROL = 0x11,
  VKEY_MENU = 0x12,
  VKEY_PAUSE = 0x13,
  VKEY_CAPITAL = 0x14,
  VKEY_KANA = 0x15,
  VKEY_HANGUL = 0x15,
  VKEY_JUNJA = 0x17,
  VKEY_FINAL = 0x18,
  VKEY_HANJA = 0x19,
  VKEY_KANJI = 0x19,
  VKEY_ESCAPE = 0x1B,
  VKEY_CONVERT = 0x1C,
  VKEY_NONCONVERT = 0x1D,
  VKEY_ACCEPT = 0x1E,
  VKEY_MODECHANGE = 0x1F,
  VKEY_SPACE = 0x20,
  VKEY_PRIOR = 0x21,
  VKEY_NEXT = 0x22,
  VKEY_END = 0x23,
  VKEY_HOME = 0x24,
  VKEY_LEFT = 0x25,
  VKEY_UP = 0x26,
  VKEY_RIGHT = 0x27,
  VKEY_DOWN = 0x28,
  VKEY_SELECT = 0x29,
  VKEY_PRINT = 0x2A,
  VKEY_EXECUTE = 0x2B,
  VKEY_SNAPSHOT = 0x2C,
  VKEY_INSERT = 0x2D,
  VKEY_DELETE = 0x2E,
  VKEY_HELP = 0x2F,
  VKEY_0 = 0x30,
  VKEY_1 = 0x31,
  VKEY_2 = 0x32,
  VKEY_3 = 0x33,
  VKEY_4 = 0x34,
  VKEY_5 = 0x35,
  VKEY_6 = 0x36,
  VKEY_7 = 0x37,
  VKEY_8 = 0x38,
  VKEY_9 = 0x39,
  VKEY_A = 0x41,
  VKEY_B = 0x42,
  VKEY_C = 0x43,
  VKEY_D = 0x44,
  VKEY_E = 0x45,
  VKEY_F = 0x46,
  VKEY_G = 0x47,
  VKEY_H = 0x48,
  VKEY_I = 0x49,
  VKEY_J = 0x4A,
  VKEY_K = 0x4B,
  VKEY_L = 0x4C,
  VKEY_M = 0x4D,
  VKEY_N = 0x4E,
  VKEY_O = 0x4F,
  VKEY_P = 0x50,
  VKEY_Q = 0x51,
  VKEY_R = 0x52,
  VKEY_S = 0x53,
  VKEY_T = 0x54,
  VKEY_U = 0x55,
  VKEY_V = 0x56,
  VKEY_W = 0x57,
  VKEY_X = 0x58,
  VKEY_Y = 0x59,
  VKEY_Z = 0x5A,
  VKEY_LWIN = 0x5B,
  VKEY_COMMAND = VKEY_LWIN,  // Provide the Mac name for convenience.
  VKEY_RWIN = 0x5C,
  VKEY_APPS = 0x5D,
  VKEY_SLEEP = 0x5F,
  VKEY_NUMPAD0 = 0x60,
  VKEY_NUMPAD1 = 0x61,
  VKEY_NUMPAD2 = 0x62,
  VKEY_NUMPAD3 = 0x63,
  VKEY_NUMPAD4 = 0x64,
  VKEY_NUMPAD5 = 0x65,
  VKEY_NUMPAD6 = 0x66,
  VKEY_NUMPAD7 = 0x67,
  VKEY_NUMPAD8 = 0x68,
  VKEY_NUMPAD9 = 0x69,
  VKEY_MULTIPLY = 0x6A,
  VKEY_ADD = 0x6B,
  VKEY_SEPARATOR = 0x6C,
  VKEY_SUBTRACT = 0x6D,
  VKEY_DECIMAL = 0x6E,
  VKEY_DIVIDE = 0x6F,
  VKEY_F1 = 0x70,
  VKEY_F2 = 0x71,
  VKEY_F3 = 0x72,
  VKEY_F4 = 0x73,
  VKEY_F5 = 0x74,
  VKEY_F6 = 0x75,
  VKEY_F7 = 0x76,
  VKEY_F8 = 0x77,
  VKEY_F9 = 0x78,
  VKEY_F10 = 0x79,
  VKEY_F11 = 0x7A,
  VKEY_F12 = 0x7B,
  VKEY_F13 = 0x7C,
  VKEY_F14 = 0x7D,
  VKEY_F15 = 0x7E,
  VKEY_F16 = 0x7F,
  VKEY_F17 = 0x80,
  VKEY_F18 = 0x81,
  VKEY_F19 = 0x82,
  VKEY_F20 = 0x83,
  VKEY_F21 = 0x84,
  VKEY_F22 = 0x85,
  VKEY_F23 = 0x86,
  VKEY_F24 = 0x87,
  VKEY_NUMLOCK = 0x90,
  VKEY_SCROLL = 0x91,
  VKEY_LSHIFT = 0xA0,
  VKEY_RSHIFT = 0xA1,
  VKEY_LCONTROL = 0xA2,
  VKEY_RCONTROL = 0xA3,
  VKEY_LMENU = 0xA4,
  VKEY_RMENU = 0xA5,
  VKEY_BROWSER_BACK = 0xA6,
  VKEY_BROWSER_FORWARD = 0xA7,
  VKEY_BROWSER_REFRESH = 0xA8,
  VKEY_BROWSER_STOP = 0xA9,
  VKEY_BROWSER_SEARCH = 0xAA,
  VKEY_BROWSER_FAVORITES = 0xAB,
  VKEY_BROWSER_HOME = 0xAC,
  VKEY_VOLUME_MUTE = 0xAD,
  VKEY_VOLUME_DOWN = 0xAE,
  VKEY_VOLUME_UP = 0xAF,
  VKEY_MEDIA_NEXT_TRACK = 0xB0,
  VKEY_MEDIA_PREV_TRACK = 0xB1,
  VKEY_MEDIA_STOP = 0xB2,
  VKEY_MEDIA_PLAY_PAUSE = 0xB3,
  VKEY_MEDIA_LAUNCH_MAIL = 0xB4,
  VKEY_MEDIA_LAUNCH_MEDIA_SELECT = 0xB5,
  VKEY_MEDIA_LAUNCH_APP1 = 0xB6,
  VKEY_MEDIA_LAUNCH_APP2 = 0xB7,
  VKEY_OEM_1 = 0xBA,
  VKEY_OEM_PLUS = 0xBB,
  VKEY_OEM_COMMA = 0xBC,
  VKEY_OEM_MINUS = 0xBD,
  VKEY_OEM_PERIOD = 0xBE,
  VKEY_OEM_2 = 0xBF,
  VKEY_OEM_3 = 0xC0,
  VKEY_OEM_4 = 0xDB,
  VKEY_OEM_5 = 0xDC,
  VKEY_OEM_6 = 0xDD,
  VKEY_OEM_7 = 0xDE,
  VKEY_OEM_8 = 0xDF,
  VKEY_OEM_102 = 0xE2,
  VKEY_OEM_103 = 0xE3,  // GTV KEYCODE_MEDIA_REWIND
  VKEY_OEM_104 = 0xE4,  // GTV KEYCODE_MEDIA_FAST_FORWARD
  VKEY_PROCESSKEY = 0xE5,
  VKEY_PACKET = 0xE7,
  VKEY_DBE_SBCSCHAR = 0xF3,
  VKEY_DBE_DBCSCHAR = 0xF4,
  VKEY_ATTN = 0xF6,
  VKEY_CRSEL = 0xF7,
  VKEY_EXSEL = 0xF8,
  VKEY_EREOF = 0xF9,
  VKEY_PLAY = 0xFA,
  VKEY_ZOOM = 0xFB,
  VKEY_NONAME = 0xFC,
  VKEY_PA1 = 0xFD,
  VKEY_OEM_CLEAR = 0xFE,
  VKEY_UNKNOWN = 0,

  // POSIX specific VKEYs. Note that as of Windows SDK 7.1, 0x97-9F, 0xD8-DA,
  // and 0xE8 are unassigned.
  VKEY_WLAN = 0x97,
  VKEY_POWER = 0x98,
  VKEY_BRIGHTNESS_DOWN = 0xD8,
  VKEY_BRIGHTNESS_UP = 0xD9,
  VKEY_KBD_BRIGHTNESS_DOWN = 0xDA,
  VKEY_KBD_BRIGHTNESS_UP = 0xE8,

  // Windows does not have a specific key code for AltGr. We use the unused 0xE1
  // (VK_OEM_AX) code to represent AltGr, matching the behaviour of Firefox on
  // Linux.
  VKEY_ALTGR = 0xE1,
  // Windows does not have a specific key code for Compose. We use the unused
  // 0xE6 (VK_ICO_CLEAR) code to represent Compose.
  VKEY_COMPOSE = 0xE6,
};

#endif

#if PLATFORM_MAC
// enable buffered video so we don't DoS the OpenGL API with texture uploads causing a downstream crash on macOS
#define USE_BUFFERED_VIDEO 1
#else
#define USE_BUFFERED_VIDEO 0
#endif

#define UE_CEF_HAS_RESIZE_BUG (CEF_VERSION_MAJOR >= 128)
// CEF is usually 1-2 frames behind our current viewport size when drag-resizing,
// this value gives it just enough time to send a correctly-sized frame without us forcing a resize
// (which invalidates a bunch of things on the CEF side)
#define MIN_FRAMES_FOR_CEF_RESIZE 3

namespace {
	// Private helper class to post a callback to GetSource.
	class FWebBrowserClosureVisitor
		: public CefStringVisitor
	{
	public:
		FWebBrowserClosureVisitor(TFunction<void (const FString&)> InClosure)
			: Closure(InClosure)
		{ }

		virtual void Visit(const CefString& String) override
		{
			Closure(FString(WCHAR_TO_TCHAR(String.ToWString().c_str())));
		}

	private:
		TFunction<void (const FString&)> Closure;
		IMPLEMENT_REFCOUNTING(FWebBrowserClosureVisitor);
	};
}


// Private helper class to smooth out video buffering, using a ringbuffer
// (cef sometimes submits multiple frames per engine frame)
class FBrowserBufferedVideo
{
public:
	FBrowserBufferedVideo(uint32 NumFrames)
		: FrameWriteIndex(0)
		, FrameReadIndex(0)
		, FrameCountThisEngineTick(0)
		, FrameCount(0)
		, FrameNumberOfLastRender(-1)
	{
		Frames.SetNum(NumFrames);
	}

	~FBrowserBufferedVideo()
	{
	}

	/**
	* Submits a frame to the video buffer
	* @return true if this is the first frame submitted this engine tick, or false otherwise
	*/
	bool SubmitFrame(
		int32 InWidth,
		int32 InHeight,
		const void* Buffer,
		FIntRect Dirty)
	{
		check(IsInGameThread());
		check(Buffer != nullptr);

		const uint32 NumBytesPerPixel = 4;
		FFrame& Frame = Frames[FrameWriteIndex];

		// If the write buffer catches up to the read buffer, we need to release the read buffer and increment its index
		if (FrameWriteIndex == FrameReadIndex && FrameCount > 0)
		{
			Frame.ReleaseTextureData();
			FrameReadIndex = (FrameReadIndex + 1) % Frames.Num();
		}

		check(Frame.SlateTextureData == nullptr);
		Frame.SlateTextureData = new FSlateTextureData((uint8*)Buffer, InWidth, InHeight, NumBytesPerPixel);

		FrameWriteIndex = (FrameWriteIndex + 1) % Frames.Num();
		FrameCount = FMath::Min(Frames.Num(), FrameCount + 1);
		FrameCountThisEngineTick++;

		return FrameCountThisEngineTick == 1;
	}

    /**
     * Called once per frame to get the next frame's texturedata
     * @return The texture data. Can be nullptr if no frame is available
     */
	FSlateTextureData* GetNextFrameTextureData()
	{
		// Grab the next available frame if available. Ensure we don't grab more than one frame per engine tick
		check(IsInGameThread());
		FSlateTextureData* SlateTextureData = nullptr;
		if (FrameCount > 0)
		{
			// Grab the first frame we haven't submitted yet 
			FFrame& Frame = Frames[FrameReadIndex];
			SlateTextureData = Frame.SlateTextureData;
			
			// Set this to NULL because the renderthread is taking ownership
			Frame.SlateTextureData = nullptr;
			FrameReadIndex = (FrameReadIndex + 1) % Frames.Num();
			FrameCount--;
		}
		FrameCountThisEngineTick = 0;
		return SlateTextureData;
	}

private:
	struct FFrame
	{
		FFrame()
			: SlateTextureData(nullptr)
		{}
		
		~FFrame()
		{
			ReleaseTextureData();
		}

		void ReleaseTextureData()
		{
			if (SlateTextureData)
			{
				delete SlateTextureData;
			}
			SlateTextureData = nullptr;
		}

		FSlateTextureData* SlateTextureData;
	};

	TArray<FFrame> Frames;

	// Read/write position in the ringbuffer
	int32 FrameWriteIndex;
	int32 FrameReadIndex;

	int32 FrameCountThisEngineTick;
	int32 FrameCount;
	int32 FrameNumberOfLastRender;
};



FCEFWebBrowserWindow::FCEFWebBrowserWindow(CefRefPtr<CefBrowser> InBrowser, CefRefPtr<FCEFBrowserHandler> InHandler, FString InUrl, TOptional<FString> InContentsToLoad, FString InUserAgentProduct, bool bInShowErrorMessage, bool bInThumbMouseButtonNavigation, bool bInUseTransparency, bool bInJSBindingToLoweringEnabled, bool bInUsingAcceleratedPaint)
	: DocumentState(EWebBrowserDocumentState::NoDocument)
	, InternalCefBrowser(InBrowser)
	, WebBrowserHandler(InHandler)
	, CurrentUrl(InUrl)
	, UserAgentProduct(InUserAgentProduct)
	, ViewportSize(FIntPoint::ZeroValue)
	, ViewportDPIScaleFactor(1.0f)
	, ViewportScaleFactor(1.0f)
	, bIsClosing(false)
	, bIsInitialized(false)
	, ContentsToLoad(InContentsToLoad)
	, bShowErrorMessage(bInShowErrorMessage)
	, bThumbMouseButtonNavigation(bInThumbMouseButtonNavigation)
	, bUseTransparency(bInUseTransparency)
	, bUsingAcceleratedPaint(bInUsingAcceleratedPaint)
	, Cursor(EMouseCursor::Default)
	, bIsDisabled(false)
	, bIsHidden(false)
	, bTickedLastFrame(true)
	, bDraggingWindow(false)
	, PreviousKeyDownEvent()
	, PreviousKeyUpEvent()
	, PreviousCharacterEvent()
	, bIgnoreKeyDownEvent(false)
	, bIgnoreKeyUpEvent(false)
	, bIgnoreCharacterEvent(false)
	, bMainHasFocus(false)
	, bPopupHasFocus(false)
	, bSupportsMouseWheel(true)
	, bShowPopupRequested(false)
	, bRecoverFromRenderProcessCrash(false)
	, ErrorCode(0)
	, bDeferNavigations(false)
#if PLATFORM_MAC
	, LastPaintedSharedHandle(nullptr)
#endif
	, Scripting(new FCEFJSScripting(InBrowser, bInJSBindingToLoweringEnabled))
#if !PLATFORM_LINUX
	, Ime(new FCEFImeHandler(InBrowser))
#endif
	, RHIRenderHelper(nullptr)
#if PLATFORM_WINDOWS || PLATFORM_MAC
	, bInDirectHwndMode(false)
#endif
	, bHasCorrectNativeCefBuffer(true)
	, NumFramesSinceViewportResize(0)
{
	check(InBrowser.get() != nullptr);
	check(!bUsingAcceleratedPaint || CanSupportAcceleratedPaint()); // make sure if accelerated paint is selected we can support it

	UpdatableTextures[PET_VIEW] = nullptr;
	UpdatableTextures[PET_POPUP] = nullptr;

	if (!CreateInitialTextures())
	{
		ReleaseTextures();
	}

#if PLATFORM_WINDOWS || PLATFORM_MAC
	if (InternalCefBrowser->GetHost()->GetWindowHandle() != nullptr)
	{
		bInDirectHwndMode = true;
	}
#endif

#if USE_BUFFERED_VIDEO
	BufferedVideo = TUniquePtr<FBrowserBufferedVideo>(new FBrowserBufferedVideo(4));
#endif
}

void FCEFWebBrowserWindow::ReleaseTextures()
{
	for (int Type : { PET_VIEW, PET_POPUP })
	{
		if (UpdatableTextures[Type] != nullptr)
		{
			FSlateUpdatableTexture* TextureToRelease = UpdatableTextures[Type];

			if (IsInGameThread())
			{
				if (FSlateApplication::IsInitialized())
				{
					if (bUsingAcceleratedPaint && RHIRenderHelper)
					{
						RHIRenderHelper->ReleaseSlateUpdatableTexture(TextureToRelease);
					}
					else if (FSlateRenderer* Renderer = FSlateApplication::Get().GetRenderer())
					{
						Renderer->ReleaseUpdatableTexture(TextureToRelease);
					}
				}
			}
			else if (FTaskGraphInterface::IsRunning())
			{
				AsyncTask(ENamedThreads::GameThread, [this, TextureToRelease]()
				{
					if (FSlateApplication::IsInitialized())
					{
						if (bUsingAcceleratedPaint && RHIRenderHelper)
						{
							RHIRenderHelper->ReleaseSlateUpdatableTexture(TextureToRelease);
						}
						else if (FSlateRenderer* Renderer = FSlateApplication::Get().GetRenderer())
						{
							Renderer->ReleaseUpdatableTexture(TextureToRelease);
						}
					}
				});
			}

			UpdatableTextures[Type] = nullptr;
		}
	}
}

bool FCEFWebBrowserWindow::CreateInitialTextures()
{
	if (FSlateApplication::IsInitialized())
	{
		if (FSlateRenderer* Renderer = FSlateApplication::Get().GetRenderer())
		{
			if (Renderer->HasLostDevice())
			{
				return false;
			}

			if (bUsingAcceleratedPaint)
			{
				if (FCEFWebBrowserWindowRHIHelper::BUseSupportedRHIRenderer() && !RHIRenderHelper)
				{
					RHIRenderHelper.Reset(new FCEFWebBrowserWindowRHIHelper);

					// Render a transparent texture until we receive the first update from CEF
					UpdatableTextures[PET_VIEW] = RHIRenderHelper->CreateSlateUpdatableTexture({ 1, 1 });
					UpdatableTextures[PET_POPUP] = RHIRenderHelper->CreateSlateUpdatableTexture({ 1, 1 });

					return true;
				}

				return false;
			}

			// Create a transparent dummy texture for our buffers which will prevent slate from applying an 
			// undesirable quad if it happens to ask for this buffer before we get a chance to paint to it.
			TArray<uint8> RawData;
			RawData.AddZeroed(4);
			UpdatableTextures[PET_VIEW] = Renderer->CreateUpdatableTexture(1, 1);

			if (Renderer->HasLostDevice())
			{
				return false;
			}

			UpdatableTextures[PET_VIEW]->UpdateTextureThreadSafeRaw(1, 1, RawData.GetData());

			if (Renderer->HasLostDevice())
			{
				return false;
			}

			UpdatableTextures[PET_POPUP] = Renderer->CreateUpdatableTexture(1, 1);

			if (Renderer->HasLostDevice())
			{
				return false;
			}

			UpdatableTextures[PET_POPUP]->UpdateTextureThreadSafeRaw(1, 1, RawData.GetData());

			if (Renderer->HasLostDevice())
			{
				return false;
			}

			return true;
		}
	}

	return false;
}

FCEFWebBrowserWindow::~FCEFWebBrowserWindow()
{
	WebBrowserHandler->OnCreateWindow().Unbind();
	WebBrowserHandler->OnBeforePopup().Unbind();
	WebBrowserHandler->OnBeforeResourceLoad().Unbind();
	WebBrowserHandler->OnResourceLoadComplete().Unbind();
	WebBrowserHandler->OnConsoleMessage().Unbind();
	if (IsValid())
	{
		CloseBrowser(true, false);
	}

	ReleaseTextures();

	BufferedVideo.Reset();

	UE_LOGF(LogWebBrowser, Log, "Deleting browser for Url=%ls.", *CurrentUrl);
}

void FCEFWebBrowserWindow::LoadURL(FString NewURL)
{
	RequestNavigationInternal(NewURL, FString());
}

void FCEFWebBrowserWindow::LoadString(FString Contents, FString DummyURL)
{
	RequestNavigationInternal(DummyURL, Contents);
}

TSharedRef<SViewport> FCEFWebBrowserWindow::CreateWidget()
{
	TSharedRef<SViewport> BrowserWidgetRef =
		SNew(SViewport)
		.EnableGammaCorrection(false)
		.EnableBlending(bUseTransparency)
		.IgnoreTextureAlpha(!bUseTransparency);

#if !PLATFORM_LINUX
	Ime->CacheBrowserSlateInfo(BrowserWidgetRef);
#endif

	return BrowserWidgetRef;
}

bool FCEFWebBrowserWindow::BlockInputInDirectHwndMode() const
{
#if PLATFORM_WINDOWS
	return bInDirectHwndMode;
#elif PLATFORM_MAC
	return bInDirectHwndMode;
#else
	return false;
#endif
}

void FCEFWebBrowserWindow::SetViewportSize(FIntPoint WindowSize, FIntPoint WindowPos)
{
	// SetViewportSize is called from the browser viewport tick method, which means that since we are receiving ticks, we can mark the browser as visible.
	if (!bIsDisabled)
	{
		SetIsHidden(false);
	}
	bTickedLastFrame=true;

	float WindowDPIScaleFactor = 1.0f;
	if (TSharedPtr<SWindow> ParentWindowPtr = ParentWindow.Pin())
	{
		WindowDPIScaleFactor = ParentWindowPtr->GetNativeWindow()->GetDPIScaleFactor();
	}

	ViewportPos = WindowPos;

	// Ignore sizes that can't be seen as it forces CEF to re-render whole image
	if ((WindowSize.X > 0 && WindowSize.Y > 0 && ViewportSize != WindowSize) || WindowDPIScaleFactor != ViewportDPIScaleFactor)
	{
		ViewportSize = WindowSize;
		ViewportDPIScaleFactor = WindowDPIScaleFactor;
		FIntPoint WindowSizeScaled = (FVector2D(WindowSize) * WindowDPIScaleFactor).IntPoint();

		if (IsValid() && WindowSizeScaled.X > 0 && WindowSizeScaled.Y > 0)
		{
#if PLATFORM_WINDOWS
			HWND NativeHandle = InternalCefBrowser->GetHost()->GetWindowHandle();
			if (NativeHandle)
			{
				HWND Parent = ::GetParent(NativeHandle);
				// Position is in screen coordinates, so we'll need to get the parent window location first.
				RECT ParentRect = { 0, 0, 0, 0 };
				if (Parent)
				{
					::GetWindowRect(Parent, &ParentRect);
				}

				::SetWindowPos(NativeHandle, 0, WindowPos.X - ParentRect.left, WindowPos.Y - ParentRect.top, WindowSizeScaled.X, WindowSizeScaled.Y, 0);
			}
#elif PLATFORM_MAC
			CefWindowHandle NativeWindowHandle = InternalCefBrowser->GetHost()->GetWindowHandle();
			if (NativeWindowHandle)
			{
				NSView* browserView = CAST_CEF_WINDOW_HANDLE_TO_NSVIEW(NativeWindowHandle);
				if (TSharedPtr<SWindow> ParentWindowPtr = ParentWindow.Pin())
				{
					NSWindow* parentWindow = (NSWindow*)ParentWindowPtr->GetNativeWindow()->GetOSWindowHandle();

					const FVector2D CocoaPosition = FMacApplication::ConvertSlatePositionToCocoa(WindowPos.X, WindowPos.Y);
					NSRect parentFrame = [parentWindow frame];
					NSRect Rect = NSMakeRect(CocoaPosition.X - parentFrame.origin.x, (CocoaPosition.Y - parentFrame.origin.y) - WindowSize.Y, FMath::Max(WindowSize.X, 1), FMath::Max(WindowSize.Y, 1));
					Rect = [parentWindow frameRectForContentRect : Rect];

					[browserView setFrame : Rect] ;
				}
			}
#endif
			RequestCefResize();

			// The lines below create a new window texture with the new size, also copying the old texture contents 1:1 into the new one, so they don't look stretched.
			// But, in the non-accelerated case we only want to use CapturedCefFrame's contents if they're from a recent frame (ie. an ongoing drag-resize)
			if (bUsingAcceleratedPaint && RHIRenderHelper)
			{
				UpdatableTextures[PET_VIEW] = RHIRenderHelper->ResizeSlateUpdatableTexture(UpdatableTextures[PET_VIEW], WindowSizeScaled);
			}
			else if (CapturedCefFrame.IsValid() && !CapturedCefFrame->IsEmpty() && NumFramesSinceViewportResize < 10)
			{
				FSlateRenderer* Renderer = GetRenderer();
				if (!Renderer)
				{
					return;
				}

				FSlateUpdatableTexture* ResizedTexture = Renderer->CreateUpdatableTexture(WindowSizeScaled.X, WindowSizeScaled.Y);
				HandleRenderingError();
				if (!ResizedTexture)
				{
					return;
				}

				// will safely delete the FSlateTextureData pointer when done with it on the rendering thread
				ResizedTexture->UpdateTextureThreadSafeWithTextureData(CapturedCefFrame->CreateSlateTextureData(WindowSizeScaled));
				HandleRenderingError();
				
				Renderer->ReleaseUpdatableTexture(UpdatableTextures[PET_VIEW]);
				HandleRenderingError();

				UpdatableTextures[PET_VIEW] = ResizedTexture;
			}

			NumFramesSinceViewportResize = 0;
		}
	}
}

FSlateShaderResource* FCEFWebBrowserWindow::GetTexture(bool bIsPopup)
{
	if (UpdatableTextures[bIsPopup? PET_POPUP : PET_VIEW] != nullptr)
	{
		return UpdatableTextures[bIsPopup? PET_POPUP : PET_VIEW]->GetSlateResource();
	}
	return nullptr;
}

bool FCEFWebBrowserWindow::IsValid() const
{
	return InternalCefBrowser.get() != nullptr;
}

bool FCEFWebBrowserWindow::IsInitialized() const
{
	return bIsInitialized;
}

bool FCEFWebBrowserWindow::IsClosing() const
{
	return bIsClosing;
}

EWebBrowserDocumentState FCEFWebBrowserWindow::GetDocumentLoadingState() const
{
	return DocumentState;
}

FString FCEFWebBrowserWindow::GetTitle() const
{
	return Title;
}

FString FCEFWebBrowserWindow::GetUrl() const
{
	if (InternalCefBrowser != nullptr)
	{
		CefRefPtr<CefFrame> MainFrame = InternalCefBrowser->GetMainFrame();

		if (MainFrame != nullptr)
		{
			return CurrentUrl;
		}
	}

	return FString();
}

void FCEFWebBrowserWindow::GetSource(TFunction<void (const FString&)> Callback) const
{
	if (IsValid())
	{
		InternalCefBrowser->GetMainFrame()->GetSource(new FWebBrowserClosureVisitor(Callback));
	}
	else
	{
		Callback(FString());
	}
}

void FCEFWebBrowserWindow::PopulateCefKeyEvent(const FKeyEvent& InKeyEvent, CefKeyEvent& OutKeyEvent)
{
#if PLATFORM_MAC
	OutKeyEvent.native_key_code = InKeyEvent.GetKeyCode();

	FKey Key = InKeyEvent.GetKey();
	if (Key == EKeys::BackSpace)
	{
		OutKeyEvent.unmodified_character = kBackspaceCharCode;
	}
	else if (Key == EKeys::Tab)
	{
		OutKeyEvent.unmodified_character = kTabCharCode;
	}
	else if (Key == EKeys::Enter)
	{
		OutKeyEvent.unmodified_character = kReturnCharCode;
	}
	else if (Key == EKeys::Pause)
	{
		OutKeyEvent.unmodified_character = NSPauseFunctionKey;
	}
	else if (Key == EKeys::Escape)
	{
		OutKeyEvent.unmodified_character = kEscapeCharCode;
	}
	else if (Key == EKeys::PageUp)
	{
		OutKeyEvent.unmodified_character = NSPageUpFunctionKey;
	}
	else if (Key == EKeys::PageDown)
	{
		OutKeyEvent.unmodified_character = NSPageDownFunctionKey;
	}
	else if (Key == EKeys::End)
	{
		OutKeyEvent.unmodified_character = NSEndFunctionKey;
	}
	else if (Key == EKeys::Home)
	{
		OutKeyEvent.unmodified_character = NSHomeFunctionKey;
	}
	else if (Key == EKeys::Left)
	{
		OutKeyEvent.unmodified_character = NSLeftArrowFunctionKey;
	}
	else if (Key == EKeys::Up)
	{
		OutKeyEvent.unmodified_character = NSUpArrowFunctionKey;
	}
	else if (Key == EKeys::Right)
	{
		OutKeyEvent.unmodified_character = NSRightArrowFunctionKey;
	}
	else if (Key == EKeys::Down)
	{
		OutKeyEvent.unmodified_character = NSDownArrowFunctionKey;
	}
	else if (Key == EKeys::Insert)
	{
		OutKeyEvent.unmodified_character = NSInsertFunctionKey;
	}
	else if (Key == EKeys::Delete)
	{
		OutKeyEvent.unmodified_character = kDeleteCharCode;
	}
	else if (Key == EKeys::F1)
	{
		OutKeyEvent.unmodified_character = NSF1FunctionKey;
	}
	else if (Key == EKeys::F2)
	{
		OutKeyEvent.unmodified_character = NSF2FunctionKey;
	}
	else if (Key == EKeys::F3)
	{
		OutKeyEvent.unmodified_character = NSF3FunctionKey;
	}
	else if (Key == EKeys::F4)
	{
		OutKeyEvent.unmodified_character = NSF4FunctionKey;
	}
	else if (Key == EKeys::F5)
	{
		OutKeyEvent.unmodified_character = NSF5FunctionKey;
	}
	else if (Key == EKeys::F6)
	{
		OutKeyEvent.unmodified_character = NSF6FunctionKey;
	}
	else if (Key == EKeys::F7)
	{
		OutKeyEvent.unmodified_character = NSF7FunctionKey;
	}
	else if (Key == EKeys::F8)
	{
		OutKeyEvent.unmodified_character = NSF8FunctionKey;
	}
	else if (Key == EKeys::F9)
	{
		OutKeyEvent.unmodified_character = NSF9FunctionKey;
	}
	else if (Key == EKeys::F10)
	{
		OutKeyEvent.unmodified_character = NSF10FunctionKey;
	}
	else if (Key == EKeys::F11)
	{
		OutKeyEvent.unmodified_character = NSF11FunctionKey;
	}
	else if (Key == EKeys::F12)
	{
		OutKeyEvent.unmodified_character = NSF12FunctionKey;
	}
	else if (Key == EKeys::CapsLock)
	{
		OutKeyEvent.unmodified_character = 0;
		OutKeyEvent.native_key_code = kVK_CapsLock;
	}
	else if (Key.IsModifierKey())
	{
		// Setting both unmodified_character and character to 0 tells CEF that it needs to generate a NSFlagsChanged event instead of NSKeyDown/Up
		OutKeyEvent.unmodified_character = 0;

		// CEF expects modifier key codes as one of the Carbon kVK_* key codes.
		if (Key == EKeys::LeftCommand)
		{
			OutKeyEvent.native_key_code = kVK_Command;
		}
		else if (Key == EKeys::LeftShift)
		{
			OutKeyEvent.native_key_code = kVK_Shift;
		}
		else if (Key == EKeys::LeftAlt)
		{
			OutKeyEvent.native_key_code = kVK_Option;
		}
		else if (Key == EKeys::LeftControl)
		{
			OutKeyEvent.native_key_code = kVK_Control;
		}
		else if (Key == EKeys::RightCommand)
		{
			// There isn't a separate code for the right hand command key defined, but CEF seems to use the unused value before the left command keycode
			OutKeyEvent.native_key_code = kVK_Command-1;
		}
		else if (Key == EKeys::RightShift)
		{
			OutKeyEvent.native_key_code = kVK_RightShift;
		}
		else if (Key == EKeys::RightAlt)
		{
			OutKeyEvent.native_key_code = kVK_RightOption;
		}
		else if (Key == EKeys::RightControl)
		{
			OutKeyEvent.native_key_code = kVK_RightControl;
		}
	}
	else
	{
		OutKeyEvent.unmodified_character = InKeyEvent.GetCharacter();
	}
	OutKeyEvent.character = OutKeyEvent.unmodified_character;

#elif PLATFORM_LINUX
	OutKeyEvent.native_key_code = InKeyEvent.GetKeyCode();
	FKey Key = InKeyEvent.GetKey();
	// helper macro so we can fill in all the A-Z, 0-9 keys
#define LETTER_KEY_MACRO(val, vkey) else if (Key == EKeys::val) \
	{ \
		OutKeyEvent.unmodified_character = InKeyEvent.GetCharacter(); \
		OutKeyEvent.windows_key_code = vkey; \
	} \

	if (Key == EKeys::BackSpace)
	{
		OutKeyEvent.windows_key_code = VKEY_BACK;
	}
	else if (Key == EKeys::Tab)
	{
		OutKeyEvent.windows_key_code = VKEY_TAB;
	}
	else if (Key == EKeys::Enter)
	{
		OutKeyEvent.windows_key_code = VKEY_RETURN;
	}
	else if (Key == EKeys::Pause)
	{
		OutKeyEvent.windows_key_code = VKEY_PAUSE;
	}
	else if (Key == EKeys::Escape)
	{
		OutKeyEvent.windows_key_code = VKEY_ESCAPE;
	}
	else if (Key == EKeys::PageUp)
	{
		OutKeyEvent.windows_key_code = VKEY_PRIOR;
	}
	else if (Key == EKeys::PageDown)
	{
		OutKeyEvent.windows_key_code = VKEY_NEXT;
	}
	else if (Key == EKeys::End)
	{
		OutKeyEvent.windows_key_code = VKEY_END;
	}
	else if (Key == EKeys::Home)
	{
		OutKeyEvent.windows_key_code = VKEY_HOME;
	}
	else if (Key == EKeys::Left)
	{
		OutKeyEvent.windows_key_code = VKEY_LEFT;
	}
	else if (Key == EKeys::Up)
	{
		OutKeyEvent.windows_key_code = VKEY_UP;
	}
	else if (Key == EKeys::Right)
	{
		OutKeyEvent.windows_key_code = VKEY_RIGHT;
	}
	else if (Key == EKeys::Down)
	{
		OutKeyEvent.windows_key_code = VKEY_DOWN;
	}
	else if (Key == EKeys::Insert)
	{
		OutKeyEvent.windows_key_code = VKEY_INSERT;
	}
	else if (Key == EKeys::Delete)
	{
		OutKeyEvent.windows_key_code = VKEY_DELETE;
	}
	else if (Key == EKeys::F1)
	{
		OutKeyEvent.windows_key_code = VKEY_F1;
	}
	else if (Key == EKeys::F2)
	{
		OutKeyEvent.windows_key_code = VKEY_F2;
	}
	else if (Key == EKeys::F3)
	{
		OutKeyEvent.windows_key_code = VKEY_F3;
	}
	else if (Key == EKeys::F4)
	{
		OutKeyEvent.windows_key_code = VKEY_F4;
	}
	else if (Key == EKeys::F5)
	{
		OutKeyEvent.windows_key_code = VKEY_F5;
	}
	else if (Key == EKeys::F6)
	{
		OutKeyEvent.windows_key_code = VKEY_F6;
	}
	else if (Key == EKeys::F7)
	{
		OutKeyEvent.windows_key_code = VKEY_F7;
	}
	else if (Key == EKeys::F8)
	{
		OutKeyEvent.windows_key_code = VKEY_F8;
	}
	else if (Key == EKeys::F9)
	{
		OutKeyEvent.windows_key_code = VKEY_F9;
	}
	else if (Key == EKeys::F10)
	{
		OutKeyEvent.windows_key_code = VKEY_F10;
	}
	else if (Key == EKeys::F11)
	{
		OutKeyEvent.windows_key_code = VKEY_F11;
	}
	else if (Key == EKeys::F12)
	{
		OutKeyEvent.windows_key_code = VKEY_F12;
	}
	else if (Key == EKeys::CapsLock)
	{
		OutKeyEvent.windows_key_code = VKEY_CAPITAL;
	}
	else if (Key == EKeys::LeftCommand)
	{
		OutKeyEvent.windows_key_code = VKEY_MENU;
	}
	else if (Key == EKeys::LeftShift)
	{
		OutKeyEvent.windows_key_code = VKEY_SHIFT;
	}
	else if (Key == EKeys::LeftAlt)
	{
		OutKeyEvent.windows_key_code = VKEY_MENU;
	}
	else if (Key == EKeys::LeftControl)
	{
		OutKeyEvent.windows_key_code = VKEY_CONTROL;
	}
	else if (Key == EKeys::RightCommand)
	{
		OutKeyEvent.windows_key_code = VKEY_MENU;
	}
	else if (Key == EKeys::RightShift)
	{
		OutKeyEvent.windows_key_code = VKEY_SHIFT;
	}
	else if (Key == EKeys::RightAlt)
	{
		OutKeyEvent.windows_key_code = VKEY_MENU;
	}
	else if (Key == EKeys::RightControl)
	{
		OutKeyEvent.windows_key_code = VKEY_CONTROL;
	}
	else if (Key == EKeys::NumPadOne)
	{
		OutKeyEvent.windows_key_code = VKEY_NUMPAD1;
	}
	else if (Key == EKeys::NumPadTwo)
	{
		OutKeyEvent.windows_key_code = VKEY_NUMPAD2;
	}
	else if (Key == EKeys::NumPadThree)
	{
		OutKeyEvent.windows_key_code = VKEY_NUMPAD3;
	}
	else if (Key == EKeys::NumPadFour)
	{
		OutKeyEvent.windows_key_code = VKEY_NUMPAD4;
	}
	else if (Key == EKeys::NumPadFive)
	{
		OutKeyEvent.windows_key_code = VKEY_NUMPAD5;
	}
	else if (Key == EKeys::NumPadSix)
	{
		OutKeyEvent.windows_key_code = VKEY_NUMPAD6;
	}
	else if (Key == EKeys::NumPadSeven)
	{
		OutKeyEvent.windows_key_code = VKEY_NUMPAD7;
	}
	else if (Key == EKeys::NumPadEight)
	{
		OutKeyEvent.windows_key_code = VKEY_NUMPAD8;
	}
	else if (Key == EKeys::NumPadNine)
	{
		OutKeyEvent.windows_key_code = VKEY_NUMPAD9;
	}
	else if (Key == EKeys::NumPadZero)
	{
		OutKeyEvent.windows_key_code = VKEY_NUMPAD0;
	}
	LETTER_KEY_MACRO( A, VKEY_A)
	LETTER_KEY_MACRO( B, VKEY_B)
	LETTER_KEY_MACRO( C, VKEY_C)
	LETTER_KEY_MACRO( D, VKEY_D)
	LETTER_KEY_MACRO( E, VKEY_E)
	LETTER_KEY_MACRO( F, VKEY_F)
	LETTER_KEY_MACRO( G, VKEY_G)
	LETTER_KEY_MACRO( H, VKEY_H)
	LETTER_KEY_MACRO( I, VKEY_I)
	LETTER_KEY_MACRO( J, VKEY_J)
	LETTER_KEY_MACRO( K, VKEY_K)
	LETTER_KEY_MACRO( L, VKEY_L)
	LETTER_KEY_MACRO( M, VKEY_M)
	LETTER_KEY_MACRO( N, VKEY_N)
	LETTER_KEY_MACRO( O, VKEY_O)
	LETTER_KEY_MACRO( P, VKEY_P)
	LETTER_KEY_MACRO( Q, VKEY_Q)
	LETTER_KEY_MACRO( R, VKEY_R)
	LETTER_KEY_MACRO( S, VKEY_S)
	LETTER_KEY_MACRO( T, VKEY_T)
	LETTER_KEY_MACRO( U, VKEY_U)
	LETTER_KEY_MACRO( V, VKEY_V)
	LETTER_KEY_MACRO( W, VKEY_W)
	LETTER_KEY_MACRO( X, VKEY_X)
	LETTER_KEY_MACRO( Y, VKEY_Y)
	LETTER_KEY_MACRO( Z, VKEY_Z)
	LETTER_KEY_MACRO( Zero, VKEY_0)
	LETTER_KEY_MACRO( One, VKEY_1)
	LETTER_KEY_MACRO( Two, VKEY_2)
	LETTER_KEY_MACRO( Three, VKEY_3)
	LETTER_KEY_MACRO( Four, VKEY_4)
	LETTER_KEY_MACRO( Five, VKEY_5)
	LETTER_KEY_MACRO( Six, VKEY_6)
	LETTER_KEY_MACRO( Seven, VKEY_7)
	LETTER_KEY_MACRO( Eight, VKEY_8)
	LETTER_KEY_MACRO( Nine, VKEY_9)
	else
	{
		OutKeyEvent.unmodified_character = InKeyEvent.GetCharacter();
		OutKeyEvent.windows_key_code = VKEY_UNKNOWN;
	}
#else
	OutKeyEvent.windows_key_code = InKeyEvent.GetKeyCode();
#endif

	OutKeyEvent.modifiers = GetCefKeyboardModifiers(InKeyEvent);
	//UE_LOGF(LogWebBrowser, Log, "Modifiers: %i %i %i" , OutKeyEvent.unmodified_character, OutKeyEvent.windows_key_code, OutKeyEvent.modifiers);

}

#if PLATFORM_MAC
bool FilterSystemKeyChord(const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.IsControlDown())
	{
		// Special case for Mac - make sure Cmd+~ is always passed back to the OS
		if (InKeyEvent.GetKey() == EKeys::Tilde)
		{
			return true;
		}
		// Special case for Mac - make sure Cmd+H is always ignored by CEF
		if (InKeyEvent.GetKey() == EKeys::H)
		{
			return true;
		}
	}
	return false;
}
#endif

bool FCEFWebBrowserWindow::OnKeyDown(const FKeyEvent& InKeyEvent)
{
	if (IsValid() && !BlockInputInDirectHwndMode() && !bIgnoreKeyDownEvent)
	{
#if PLATFORM_MAC
		if (FilterSystemKeyChord(InKeyEvent))
		{
			return false;
		}
#endif
		PreviousKeyDownEvent = InKeyEvent;
		CefKeyEvent KeyEvent;
		PopulateCefKeyEvent(InKeyEvent, KeyEvent);
		KeyEvent.type = KEYEVENT_RAWKEYDOWN;
		InternalCefBrowser->GetHost()->SendKeyEvent(KeyEvent);
		return true;
	}
	return false;
}

bool FCEFWebBrowserWindow::OnKeyUp(const FKeyEvent& InKeyEvent)
{
	if (IsValid() && !BlockInputInDirectHwndMode() && !bIgnoreKeyUpEvent)
	{
#if PLATFORM_MAC
		if (FilterSystemKeyChord(InKeyEvent))
		{
			return false;
		}
#endif
		PreviousKeyUpEvent = InKeyEvent;
		CefKeyEvent KeyEvent;
		PopulateCefKeyEvent(InKeyEvent, KeyEvent);
		KeyEvent.type = KEYEVENT_KEYUP;
		InternalCefBrowser->GetHost()->SendKeyEvent(KeyEvent);
		return true;
	}
	return false;
}

bool FCEFWebBrowserWindow::OnKeyChar(const FCharacterEvent& InCharacterEvent)
{
	if (IsValid() && !BlockInputInDirectHwndMode() && !bIgnoreCharacterEvent)
	{
		PreviousCharacterEvent = InCharacterEvent;
		CefKeyEvent KeyEvent;
#if PLATFORM_MAC || PLATFORM_LINUX
		KeyEvent.character = InCharacterEvent.GetCharacter();
		KeyEvent.windows_key_code = InCharacterEvent.GetCharacter();
#else
		KeyEvent.windows_key_code = InCharacterEvent.GetCharacter();
#endif
		KeyEvent.type = KEYEVENT_CHAR;
		KeyEvent.modifiers = GetCefInputModifiers(InCharacterEvent);
#if PLATFORM_WINDOWS
		if (InCharacterEvent.IsAltDown() && InCharacterEvent.IsControlDown())
		{
			// For german and other keyboards with an AltGR state, windows sets alt and left control down 
			// See OsrWindowWin::OnKeyEvent in 
			//https://bitbucket.org/chromiumembedded/cef/raw/c4baba880e0b28ce82845275b328a12b2407e2f0/tests/cefclient/browser/osr_window_win.cc 
			// from which the concept behind this check was taken
			HKL CurrentKBLayout = ::GetKeyboardLayout(0);
			SHORT ScanResult = ::VkKeyScanExW(InCharacterEvent.GetCharacter(), CurrentKBLayout);
			if (((ScanResult >> 8) & 0xFF) == (2 | 4))
			{  
				// ctrl-alt pressed from this single character event so convert to AltGR	
				KeyEvent.modifiers &= ~(EVENTFLAG_CONTROL_DOWN | EVENTFLAG_ALT_DOWN);
				KeyEvent.modifiers |= EVENTFLAG_ALTGR_DOWN;
			}
		}
#endif
		InternalCefBrowser->GetHost()->SendKeyEvent(KeyEvent);
		return true;
	}
	return false;
}

FModifierKeysState FCEFWebBrowserWindow::SlateModifiersFromCefModifiers(const CefKeyEvent& CefEvent)
{
	return FModifierKeysState((CefEvent.modifiers & EVENTFLAG_SHIFT_DOWN) != 0,
		(CefEvent.modifiers & EVENTFLAG_SHIFT_DOWN)   != 0,
		(CefEvent.modifiers & EVENTFLAG_CONTROL_DOWN) != 0,
		(CefEvent.modifiers & EVENTFLAG_CONTROL_DOWN) != 0,
		(CefEvent.modifiers & EVENTFLAG_ALT_DOWN)     != 0,
		(CefEvent.modifiers & EVENTFLAG_ALT_DOWN)     != 0,
		(CefEvent.modifiers & EVENTFLAG_COMMAND_DOWN) != 0,
		(CefEvent.modifiers & EVENTFLAG_COMMAND_DOWN) != 0,
		(CefEvent.modifiers & EVENTFLAG_CAPS_LOCK_ON) != 0);
}

/* This is an ugly hack to inject unhandled key events back into Slate.
   During processing of the initial keyboard event, we don't know whether it is handled by the Web browser or not.
   Not until after CEF calls OnKeyEvent in our CefKeyboardHandler implementation, which is after our own keyboard event handler
   has returned.
   The solution is to save a copy of the event and re-inject it into Slate while ensuring that we'll ignore it and bubble it up
   the widget hierarchy this time around. */
bool FCEFWebBrowserWindow::OnUnhandledKeyEvent(const CefKeyEvent& CefEvent)
{
	bool bWasHandled = false;
	if (IsValid())
	{
		CefWindowHandle NativeHandle = InternalCefBrowser->GetHost()->GetWindowHandle();

		switch (CefEvent.type)
		{
		case KEYEVENT_RAWKEYDOWN:
		case KEYEVENT_KEYDOWN:
			if (PreviousKeyDownEvent.IsSet())
			{
				bWasHandled = OnUnhandledKeyDown().IsBound() && OnUnhandledKeyDown().Execute(PreviousKeyDownEvent.GetValue());
				if (!bWasHandled && FSlateApplication::IsInitialized())
				{
					// If the keydown handler is not bound or if the handler returns false, indicating the key is unhandled, we bubble it up.
					bIgnoreKeyDownEvent = true;
					bWasHandled = FSlateApplication::Get().ProcessKeyDownEvent(PreviousKeyDownEvent.GetValue());
					bIgnoreKeyDownEvent = false;
				}
				PreviousKeyDownEvent.Reset();
			}
			else if (NativeHandle && FSlateApplication::IsInitialized())
			{
				FKey const Key = FInputKeyManager::Get().GetKeyFromCodes(CefEvent.windows_key_code, 0);

				if (Key.IsValid())
				{
					FKeyEvent KeyEvent(Key, SlateModifiersFromCefModifiers(CefEvent), FSlateApplication::Get().GetUserIndexForKeyboard(), false, 0, CefEvent.windows_key_code);

					bIgnoreKeyDownEvent = true;
					bWasHandled = FSlateApplication::Get().ProcessKeyDownEvent(KeyEvent);
					bIgnoreKeyDownEvent = false;
				}
			}
			break;
		case KEYEVENT_KEYUP:
			if (PreviousKeyUpEvent.IsSet())
			{
				bWasHandled = OnUnhandledKeyUp().IsBound() && OnUnhandledKeyUp().Execute(PreviousKeyUpEvent.GetValue());
				if (!bWasHandled && FSlateApplication::IsInitialized())
				{
					// If the keyup handler is not bound or if the handler returns false, indicating the key is unhandled, we bubble it up.
					bIgnoreKeyUpEvent = true;
					bWasHandled = FSlateApplication::Get().ProcessKeyUpEvent(PreviousKeyUpEvent.GetValue());
					bIgnoreKeyUpEvent = false;
				}
				PreviousKeyUpEvent.Reset();
			}
			else if (NativeHandle && FSlateApplication::IsInitialized())
			{
				FKey const Key = FInputKeyManager::Get().GetKeyFromCodes(CefEvent.windows_key_code, 0);
				FKeyEvent KeyEvent(Key, SlateModifiersFromCefModifiers(CefEvent), FSlateApplication::Get().GetUserIndexForKeyboard(), false, 0, CefEvent.windows_key_code);

				bIgnoreKeyUpEvent = true;
				bWasHandled = FSlateApplication::Get().ProcessKeyUpEvent(KeyEvent);
				bIgnoreKeyUpEvent = false;
			}

			break;
		case KEYEVENT_CHAR:
			if (PreviousCharacterEvent.IsSet())
			{
				bWasHandled = OnUnhandledKeyChar().IsBound() && OnUnhandledKeyChar().Execute(PreviousCharacterEvent.GetValue());
				if (!bWasHandled && FSlateApplication::IsInitialized())
				{
					// If the keychar handler is not bound or if the handler returns false, indicating the key is unhandled, we bubble it up.
					bIgnoreCharacterEvent = true;
					bWasHandled = FSlateApplication::Get().ProcessKeyCharEvent(PreviousCharacterEvent.GetValue());
					bIgnoreCharacterEvent = false;
				}
				PreviousCharacterEvent.Reset();
			}
			else if (NativeHandle && FSlateApplication::IsInitialized())
			{
				FCharacterEvent CharacterEvent(CefEvent.character, SlateModifiersFromCefModifiers(CefEvent), FSlateApplication::Get().GetUserIndexForKeyboard(), false);

				bIgnoreCharacterEvent = true;
				bWasHandled = FSlateApplication::Get().ProcessKeyCharEvent(CharacterEvent);
				bIgnoreCharacterEvent = false;
			}
			break;
		default:
			break;
		}
	}

#if PLATFORM_MAC
	// if this returns false, then CEF will bubble up the keyevent to eventually call NSMenu functions that are not safe on the
	// game thread, so by returning true, we stop the key handling in it's tracks before CEF can pass it to NSMenu
	return true;
#endif
	
	return bWasHandled;
}

bool FCEFWebBrowserWindow::OnJSDialog(CefJSDialogHandler::JSDialogType DialogType, const CefString& MessageText, const CefString& DefaultPromptText, CefRefPtr<CefJSDialogCallback> Callback, bool& OutSuppressMessage)
{
	bool Retval = false;
	if (OnShowDialog().IsBound())
	{
		TSharedPtr<IWebBrowserDialog> Dialog(new FCEFWebBrowserDialog(DialogType, MessageText, DefaultPromptText, Callback));
		EWebBrowserDialogEventResponse EventResponse = OnShowDialog().Execute(TWeakPtr<IWebBrowserDialog>(Dialog));
		switch (EventResponse)
		{
		case EWebBrowserDialogEventResponse::Handled:
			Retval = true;
			break;
		case EWebBrowserDialogEventResponse::Continue:
			if (DialogType == JSDIALOGTYPE_ALERT)
			{
				// Alert dialogs don't return a value, so treat Continue the same way as Ingore
				OutSuppressMessage = true;
				Retval = false;
			}
			else
			{
				Callback->Continue(true, DefaultPromptText);
				Retval = true;
			}
			break;
		case EWebBrowserDialogEventResponse::Ignore:
			OutSuppressMessage = true;
			Retval = false;
			break;
		case EWebBrowserDialogEventResponse::Unhandled:
		default:
			Retval = false;
			break;
		}
	}
	return Retval;
}

bool FCEFWebBrowserWindow::OnFileDialog(CefDialogHandler::FileDialogMode Mode, const CefString& DialogTitle, const CefString& DefaultFilePath, const std::vector<CefString>& AcceptFilters, const std::vector<CefString>& AcceptExtensions, const std::vector<CefString>& AcceptDescriptions, CefRefPtr<CefFileDialogCallback> Callback)
{
#if WITH_EDITOR
	return false;
#else
	bool bAllowFileDialogsInClientBuilds = false;
	GConfig->GetBool(TEXT("Browser"), TEXT("bAllowFileDialogsInClientBuilds"), bAllowFileDialogsInClientBuilds, GEngineIni);
	if (!bAllowFileDialogsInClientBuilds)
	{
		UE_LOGF(LogWebBrowser, Warning, "FileDialogs are prevented by default in client builds. Set bAllowFileDialogsInClientBuilds=true under [Browser] in Engine.ini to enable.");
		return true;
	}
	return false;
#endif
}

bool FCEFWebBrowserWindow::OnBeforeUnloadDialog(const CefString& MessageText, bool IsReload, CefRefPtr<CefJSDialogCallback> Callback)
{
	bool Retval = false;
	if (OnShowDialog().IsBound())
	{
		TSharedPtr<IWebBrowserDialog> Dialog(new FCEFWebBrowserDialog(MessageText, IsReload, Callback));
		EWebBrowserDialogEventResponse EventResponse = OnShowDialog().Execute(TWeakPtr<IWebBrowserDialog>(Dialog));
		switch (EventResponse)
		{
		case EWebBrowserDialogEventResponse::Handled:
			Retval = true;
			break;
		case EWebBrowserDialogEventResponse::Continue:
			Callback->Continue(true, CefString());
			Retval = true;
			break;
		case EWebBrowserDialogEventResponse::Ignore:
			Callback->Continue(false, CefString());
			Retval = true;
			break;
		case EWebBrowserDialogEventResponse::Unhandled:
		default:
			Retval = false;
			break;
		}
	}
	return Retval;
}

void FCEFWebBrowserWindow::OnResetDialogState()
{
	OnDismissAllDialogs().ExecuteIfBound();
}

void FCEFWebBrowserWindow::OnRenderProcessTerminated(CefRequestHandler::TerminationStatus Status)
{
	if (bRecoverFromRenderProcessCrash)
	{
		bRecoverFromRenderProcessCrash = false;
		NotifyDocumentError((int)ERR_FAILED); // Only attempt a single recovery at a time
	}

	bRecoverFromRenderProcessCrash = true;
	Reload();
}

FReply FCEFWebBrowserWindow::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup)
{
	FReply Reply = FReply::Unhandled();
	if (IsValid() && !BlockInputInDirectHwndMode())
	{
		FKey Button = MouseEvent.GetEffectingButton();
		// CEF only supports left, right, and middle mouse buttons
		bool bIsCefSupportedButton = (Button == EKeys::LeftMouseButton || Button == EKeys::RightMouseButton || Button == EKeys::MiddleMouseButton);

		if (bIsCefSupportedButton)
		{
			CefBrowserHost::MouseButtonType Type =
				(Button == EKeys::LeftMouseButton ? MBT_LEFT : (
				Button == EKeys::RightMouseButton ? MBT_RIGHT : MBT_MIDDLE));

			CefMouseEvent Event = GetCefMouseEvent(MyGeometry, MouseEvent, bIsPopup);

			// If the click happened inside a drag region we enable window dragging which will start firing OnDragWindow events on mouse move
			if (Type == MBT_LEFT && IsInDragRegion(FIntPoint(Event.x, Event.y)))
			{
				bDraggingWindow = true;
			}

			InternalCefBrowser->GetHost()->SendMouseClickEvent(Event, Type, false,1);
			Reply = FReply::Handled();
		}
	}
	return Reply;
}

FReply FCEFWebBrowserWindow::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup)
{
	FReply Reply = FReply::Unhandled();
	if (IsValid() && !BlockInputInDirectHwndMode())
	{
		FKey Button = MouseEvent.GetEffectingButton();
		// CEF only supports left, right, and middle mouse buttons
		bool bIsCefSupportedButton = (Button == EKeys::LeftMouseButton || Button == EKeys::RightMouseButton || Button == EKeys::MiddleMouseButton);

		if (bIsCefSupportedButton)
		{
			CefBrowserHost::MouseButtonType Type =
				(Button == EKeys::LeftMouseButton ? MBT_LEFT : (
				Button == EKeys::RightMouseButton ? MBT_RIGHT : MBT_MIDDLE));

			if (Type == MBT_LEFT)
			{
				bDraggingWindow = false;
			}

			CefMouseEvent Event = GetCefMouseEvent(MyGeometry, MouseEvent, bIsPopup);
			InternalCefBrowser->GetHost()->SendMouseClickEvent(Event, Type, true, 1);
			Reply = FReply::Handled();
		}
		else if (Button == EKeys::ThumbMouseButton && bThumbMouseButtonNavigation)
		{
			if (CanGoBack())
			{
				GoBack();
				Reply = FReply::Handled();
			}

		}
		else if (Button == EKeys::ThumbMouseButton2 && bThumbMouseButtonNavigation)
		{
			if (CanGoForward())
			{
				GoForward();
				Reply = FReply::Handled();
			}

		}
	}
	return Reply;
}

FReply FCEFWebBrowserWindow::OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup)
{
	FReply Reply = FReply::Unhandled();
	if (IsValid() && !BlockInputInDirectHwndMode())
	{
		FKey Button = MouseEvent.GetEffectingButton();
		// CEF only supports left, right, and middle mouse buttons
		bool bIsCefSupportedButton = (Button == EKeys::LeftMouseButton || Button == EKeys::RightMouseButton || Button == EKeys::MiddleMouseButton);

		if (bIsCefSupportedButton)
		{
			CefBrowserHost::MouseButtonType Type =
				(Button == EKeys::LeftMouseButton ? MBT_LEFT : (
				Button == EKeys::RightMouseButton ? MBT_RIGHT : MBT_MIDDLE));

			CefMouseEvent Event = GetCefMouseEvent(MyGeometry, MouseEvent, bIsPopup);
			InternalCefBrowser->GetHost()->SendMouseClickEvent(Event, Type, false, 2);
			Reply = FReply::Handled();
		}
	}
	return Reply;
}

FReply FCEFWebBrowserWindow::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup)
{
	FReply Reply = FReply::Unhandled();
	if (IsValid() && !BlockInputInDirectHwndMode())
	{
		CefMouseEvent Event = GetCefMouseEvent(MyGeometry, MouseEvent, bIsPopup);

		bool bEventConsumedByDragCallback = false;
		FIntRect test;
		if (bDraggingWindow && OnDragWindow().IsBound())
		{
			if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
			{
				bEventConsumedByDragCallback = OnDragWindow().Execute(MouseEvent);
			}
			else
			{
				bDraggingWindow = false;
			}
		}

		if (!bEventConsumedByDragCallback)
		{
			InternalCefBrowser->GetHost()->SendMouseMoveEvent(Event, false /*mouseLeave*/);
		}
		
		Reply = FReply::Handled();
	}
	return Reply;
}

void FCEFWebBrowserWindow::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	// Ensure we clear any tooltips if the mouse leaves the window.
	SetToolTip(CefString());

	// Don't send mouseLeave if we still have mouse buttons pressed, it indicates an outgoing drag with the mouse captured.
	// We'll receive another OnMouseLeave if we're still outside when the last button has been released.
	if (IsValid() && !BlockInputInDirectHwndMode() && MouseEvent.GetPressedButtons().IsEmpty())
	{
		// We have no geometry here to convert our mouse event to local space so we just make a dummy event and set the mouseLeave param to true
		CefMouseEvent DummyEvent;
		InternalCefBrowser->GetHost()->SendMouseMoveEvent(DummyEvent, true /*mouseLeave*/);
	}
}

void FCEFWebBrowserWindow::SetSupportsMouseWheel(bool bValue)
{
	bSupportsMouseWheel = bValue;
}

bool FCEFWebBrowserWindow::GetSupportsMouseWheel() const
{
	return bSupportsMouseWheel;
}

FReply FCEFWebBrowserWindow::OnTouchGesture(const FGeometry& MyGeometry, const FPointerEvent& GestureEvent, bool bIsPopup)
{
	FReply Reply = FReply::Unhandled();
	if (IsValid() && bSupportsMouseWheel && !BlockInputInDirectHwndMode())
	{
		const EGestureEvent GestureType = GestureEvent.GetGestureType();
		const FVector2D& GestureDelta = GestureEvent.GetGestureDelta();
		if (GestureType == EGestureEvent::Scroll)
		{
			CefMouseEvent Event = GetCefMouseEvent(MyGeometry, GestureEvent, bIsPopup);
			InternalCefBrowser->GetHost()->SendMouseWheelEvent(Event, GestureDelta.X, GestureDelta.Y);
			Reply = FReply::Handled();
		}
	}
	
	return Reply;
}

FReply FCEFWebBrowserWindow::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup)
{
	FReply Reply = FReply::Unhandled();
	if (IsValid() && bSupportsMouseWheel && !BlockInputInDirectHwndMode())
	{
#if PLATFORM_WINDOWS
		// The original delta is reduced so this should bring it back to what CEF expects
		// see WindowsApplication.cpp , case WM_MOUSEWHEEL:
		const float SpinFactor = 120.0f;
#else
		// other OS's seem to want us to scale by "line height" here, so pick a magic number
		// 50 matches a single mouse wheel tick in movement as compared to Chrome
		const float SpinFactor = 50.0f;
#endif
		const float TrueDelta = MouseEvent.GetWheelDelta() * SpinFactor;
		if (fabs(TrueDelta) > 0.001f)
		{
			CefMouseEvent Event = GetCefMouseEvent(MyGeometry, MouseEvent, bIsPopup);
			InternalCefBrowser->GetHost()->SendMouseWheelEvent(Event,
				MouseEvent.IsShiftDown() ? TrueDelta : 0,
				!MouseEvent.IsShiftDown() ? TrueDelta : 0);
		}
		Reply = FReply::Handled();
	}
	return Reply;
}

void FCEFWebBrowserWindow::OnFocus(bool SetFocus, bool bIsPopup)
{
	if (bIsPopup)
	{
		bPopupHasFocus = SetFocus;
	}
	else
	{
		bMainHasFocus = SetFocus;
	}

#if !PLATFORM_LINUX
	Ime->SetFocus(!bPopupHasFocus && bMainHasFocus);
#endif

	// Only notify focus if there is no popup menu with focus, as SetFocus will dismiss any popup menus.
	if (IsValid() && !bPopupHasFocus)
	{
		InternalCefBrowser->GetHost()->SetFocus(bMainHasFocus);
	}
}

void FCEFWebBrowserWindow::OnCaptureLost()
{
	if (IsValid())
	{
		InternalCefBrowser->GetHost()->SendCaptureLostEvent();
	}
}

bool FCEFWebBrowserWindow::CanGoBack() const
{
	if (IsValid())
	{
		return InternalCefBrowser->CanGoBack();
	}
	return false;
}

void FCEFWebBrowserWindow::GoBack()
{
	if (IsValid())
	{
		InternalCefBrowser->GoBack();
	}
}

bool FCEFWebBrowserWindow::CanGoForward() const
{
	if (IsValid())
	{
		return InternalCefBrowser->CanGoForward();
	}
	return false;
}

void FCEFWebBrowserWindow::GoForward()
{
	if (IsValid())
	{
		InternalCefBrowser->GoForward();
	}
}

bool FCEFWebBrowserWindow::IsLoading() const
{
	if (IsValid())
	{
		return InternalCefBrowser->IsLoading();
	}
	return false;
}

void FCEFWebBrowserWindow::Reload()
{
	if (IsValid())
	{
		InternalCefBrowser->Reload();
	}
}

void FCEFWebBrowserWindow::StopLoad()
{
	if (IsValid())
	{
		InternalCefBrowser->StopLoad();
	}
}

void FCEFWebBrowserWindow::ExecuteJavascript(const FString& Script)
{
	if (IsValid())
	{
		CefRefPtr<CefFrame> frame = InternalCefBrowser->GetMainFrame();
		frame->ExecuteJavaScript(TCHAR_TO_UTF8(*Script), frame->GetURL(), 0);
	}
}


void FCEFWebBrowserWindow::CloseBrowser(bool bForce, bool bBlockTillClosed)
{
	if (IsValid())
	{
		CefRefPtr<CefBrowserHost> Host = InternalCefBrowser->GetHost();
#if PLATFORM_MAC
		CefWindowHandle NativeWindowHandle = Host->GetWindowHandle();
		if (NativeWindowHandle != nullptr)
		{
			NSView *browserView = CAST_CEF_WINDOW_HANDLE_TO_NSVIEW(NativeWindowHandle);
			if (browserView != nil)
			{
				[browserView removeFromSuperview];
			}
		}
#endif
		// In case this is called from inside a CEF event handler, use CEF's task mechanism to
		// postpone the actual closing of the window until it is safe.
		CefPostTask(TID_UI, new FCEFBrowserClosureTask(nullptr, [=]()
		{
			// if blocking till closed for the close here
			Host->CloseBrowser(bForce||bBlockTillClosed);
		}));

		// We can only safely pump message if slate is active
		if (bBlockTillClosed && ensure(FSlateApplication::IsInitialized()))
		{
			SetIsHidden(true); // hide the window as we close it
			float CloseWaitTimeout = 1.0f;
			// BUGBUG Alfred - I think 1 second should be enough wait, remove these config settings
			//   if we don't end up having to tune timeouts in the field.
			GConfig->GetFloat(TEXT("Browser"), TEXT("CloseWaitTimeout"), CloseWaitTimeout, GEngineIni);
			if (IsEngineExitRequested())
			{
				// wait longer if the app is shutting down
				GConfig->GetFloat(TEXT("Browser"), TEXT("CloseWaitTimeoutAppExit"), CloseWaitTimeout, GEngineIni);
			}
			const double StartWaitAppTime = FPlatformTime::Seconds();
			while (InternalCefBrowser != nullptr)
			{
				if (FPlatformTime::Seconds() - StartWaitAppTime > CloseWaitTimeout)
				{
					UE_LOGF(LogWebBrowser, Error, "CloseBrowser - took more than %0.2f second to close. Abandoning wait...", CloseWaitTimeout);
					break; // don't spin forever
				}
				FPlatformProcess::Sleep(0.01f);
				// CEF needs the windows message pump run to be able to finish closing a browser, so run it manually here
				FSlateApplication::Get().PumpMessages();
				CefDoMessageLoopWork();
			}
		}
	}
}

CefRefPtr<CefBrowser> FCEFWebBrowserWindow::GetCefBrowser()
{
	return InternalCefBrowser;
}

void FCEFWebBrowserWindow::SetTitle(const CefString& InTitle)
{
	Title = WCHAR_TO_TCHAR(InTitle.ToWString().c_str());
	TitleChangedEvent.Broadcast(Title);
}

void FCEFWebBrowserWindow::SetUrl(const CefString& Url)
{
	CurrentUrl = WCHAR_TO_TCHAR(Url.ToWString().c_str());
	OnUrlChanged().Broadcast(CurrentUrl);
}

void FCEFWebBrowserWindow::SetToolTip(const CefString& CefToolTip)
{
	FString NewToolTipText = WCHAR_TO_TCHAR(CefToolTip.ToWString().c_str());
	if (ToolTipText != NewToolTipText)
	{
		ToolTipText = NewToolTipText;
		OnToolTip().Broadcast(ToolTipText);
	}
}

void FCEFWebBrowserWindow::GetViewRect(CefRect& Rect)
{
	if (ViewportSize == FIntPoint::ZeroValue)
	{
		Rect.width = 1; // CEF requires a minimum of a 1x1 window to correctly run
		Rect.height = 1;
	}
	else
	{
		// CEF requires a minimum of a 1px in each dimension to correctly run
		Rect.width = FMath::Max(ViewportSize.X, 1);
		Rect.height = FMath::Max(ViewportSize.Y, 1);
	}
}

int FCEFWebBrowserWindow::GetLoadError()
{
	return ErrorCode;
}

void FCEFWebBrowserWindow::NotifyDocumentError(
	CefLoadHandler::ErrorCode InErrorCode,
	const CefString& ErrorText,
	const CefString& FailedUrl)
{
	FString Url = WCHAR_TO_TCHAR(FailedUrl.ToWString().c_str());

	if (InErrorCode == ERR_ABORTED)
	{
		// Aborting navigation is not an error case but we do need to wait for any existing navigations, handled via OnBeforeBrowse(), to fully abort before we can initiate a new navigation.
		if (!PendingAbortUrl.IsEmpty() && PendingAbortUrl == Url)
		{
			PendingAbortUrl.Empty();
			bDeferNavigations = false;

			if (HasPendingNavigation())
			{
				ProcessPendingNavigation();
			}
		}
		return;
	}
	
	if (IsShowingErrorMessages())
	{
		// Display a load error message. Note: The user's code will still have a chance to handle this error after this error message is displayed.
		FFormatNamedArguments Args;
		{
			Args.Add(TEXT("FailedUrl"), FText::FromString(Url));
			Args.Add(TEXT("ErrorText"), FText::FromString(WCHAR_TO_TCHAR(ErrorText.ToWString().c_str())));
			Args.Add(TEXT("ErrorCode"), FText::AsNumber((int)InErrorCode));
		}
		FText ErrorMsg = FText::Format(NSLOCTEXT("WebBrowserHandler", "WebBrowserLoadError", "Failed to load URL {FailedUrl} with error {ErrorText} ({ErrorCode})."), Args);
		FString ErrorHTML = TEXT("<html><body bgcolor=\"white\"><h2>") + ErrorMsg.ToString() + TEXT("</h2></body></html>");

		LoadString(ErrorHTML, Url);
	}
	
	NotifyDocumentError((int)InErrorCode);
}

void FCEFWebBrowserWindow::NotifyDocumentError(int InErrorCode)
{
	ErrorCode = InErrorCode;
	DocumentState = EWebBrowserDocumentState::Error;
	DocumentStateChangedEvent.Broadcast(DocumentState);
}

void FCEFWebBrowserWindow::NotifyDocumentLoadingStateChange(bool IsLoading)
{
	if (!IsLoading)
	{
		bIsInitialized = true;

		if (bRecoverFromRenderProcessCrash)
		{
			bRecoverFromRenderProcessCrash = false;
			// Toggle hidden/visible state to get OnPaint calls from CEF.
			SetIsHidden(true);
			SetIsHidden(false);
		}

		// Compatibility with Android script bindings: dispatch a custom ue:ready event when the document is fully loaded
		ExecuteJavascript(TEXT("document.dispatchEvent(new CustomEvent('ue:ready', {details: window.ue}));"));

		// CEF OSR doesn't maintain the focused state across page navigations due to the internal RenderWidgetHostView being replaced,
		// so focus needs to be restored explicitly on the host from our tracked state.
		if (bMainHasFocus && IsValid() && !InternalCefBrowser->GetHost()->GetWindowHandle())
		{
			InternalCefBrowser->GetHost()->SetFocus(true);
		}
	}

	// Ignore a load completed notification if there was an error.
	// For load started, reset any errors from previous page load.
	if (IsLoading || DocumentState != EWebBrowserDocumentState::Error)
	{
		ErrorCode = 0;
		DocumentState = IsLoading
			? EWebBrowserDocumentState::Loading
			: EWebBrowserDocumentState::Completed;
		DocumentStateChangedEvent.Broadcast(DocumentState);
	}
}

FSlateRenderer* const FCEFWebBrowserWindow::GetRenderer()
{
	if (FSlateApplication::IsInitialized())
	{
		if (FSlateRenderer* Renderer = FSlateApplication::Get().GetRenderer())
		{
			if (!Renderer->HasLostDevice())
			{
				return Renderer;
			}
		}
	}

	ReleaseTextures();
	return nullptr;
}

void FCEFWebBrowserWindow::HandleRenderingError()
{
	// GetRenderer handles errors already
	GetRenderer();
}

void FCEFWebBrowserWindow::OnPaint(CefRenderHandler::PaintElementType Type, const CefRenderHandler::RectList& DirtyRects, const void* Buffer, int Width, int Height)
{
	if (!Buffer || Width <= 0 || Height <= 0)
	{
		return;
	}

	const FIntRect DirtyRect = (DirtyRects.size() == 1) ? FIntRect(DirtyRects[0].x, DirtyRects[0].y, DirtyRects[0].x + DirtyRects[0].width, DirtyRects[0].y + DirtyRects[0].height) : FIntRect();
	const FIntRect ViewportRect = FIntRect(0, 0, ViewportSize.X, ViewportSize.Y).Scale(ViewportDPIScaleFactor);
	if (DirtyRect.IsEmpty() || ViewportRect.IsEmpty())
	{
		return;
	}

	++NumFramesSinceViewportResize;

#if UE_CEF_HAS_RESIZE_BUG
	if (Type == PET_VIEW)
	{
		const bool bWasCorrectNativeCefBuffer = bHasCorrectNativeCefBuffer;
		bHasCorrectNativeCefBuffer = (FIntPoint(Width, Height) == ViewportRect.Size());

		if (!bHasCorrectNativeCefBuffer)
		{
			UE_LOGF(LogWebBrowser, Verbose, "FCEFWebBrowserWindow::OnPaint: Frame size still wrong after %zu frames: %dx%d -> %dx%d (%d,%d %dx%d)", NumFramesSinceViewportResize, ViewportRect.Width(), ViewportRect.Height(), Width, Height, DirtyRect.Min.X, DirtyRect.Min.Y, DirtyRect.Width(), DirtyRect.Height());
		}
		else if (!bWasCorrectNativeCefBuffer)
		{
			UE_LOGF(LogWebBrowser, Verbose, "FCEFWebBrowserWindow::OnPaint: Frame size OK again after %zu frames: %dx%d -> %dx%d (%d,%d %dx%d)", NumFramesSinceViewportResize, ViewportRect.Width(), ViewportRect.Height(), Width, Height, DirtyRect.Min.X, DirtyRect.Min.Y, DirtyRect.Width(), DirtyRect.Height());
		}

		if (CapturedCefFrame.IsValid())
		{
			// Capture the frame to be used during resizing (even if it's not the right size)
			if (!CapturedCefFrame->SetBufferAsB8G8R8A8(Buffer, Width, Height, bHasCorrectNativeCefBuffer))
			{
				UE_LOGF(LogWebBrowser, Verbose, "FCEFWebBrowserWindow::OnPaint: Detected corrupted CEF frame!");
				// force a CEF resize even if we were below the threshold, as CEF considers those "good" frames and will stop sending more until requested
				NumFramesSinceViewportResize = MIN_FRAMES_FOR_CEF_RESIZE;
				return;
			}
		}
	}
#endif // UE_CEF_HAS_RESIZE_BUG

	bool bNeedsRedraw = false;
	if (bUsingAcceleratedPaint)
	{
		UE_LOGF(LogWebBrowser, Error, "Accelerated CEF rendering selected but OnPaint called. Disabling accelerated rendering for this browser window.");
		bUsingAcceleratedPaint = false;
		if (UpdatableTextures[Type] != nullptr)
		{
			if (RHIRenderHelper)
			{
				RHIRenderHelper->ReleaseSlateUpdatableTexture(UpdatableTextures[Type]);
				UpdatableTextures[Type] = nullptr;
			}
		}
	}

	if (UpdatableTextures[Type] == nullptr)
	{
		if (FSlateRenderer* const Renderer = GetRenderer())
		{
			UpdatableTextures[Type] = Renderer->CreateUpdatableTexture(Width, Height);
			HandleRenderingError();
		}
	}

	if (UpdatableTextures[Type] != nullptr)
	{
		if (Type == PET_VIEW && BufferedVideo.IsValid())
		{
			// If we're using bufferedVideo, submit the frame to it
			bNeedsRedraw = BufferedVideo->SubmitFrame(Width, Height, Buffer, DirtyRect);
		}
		else
		{
			if (Type == PET_VIEW && !bHasCorrectNativeCefBuffer)
			{
				if (CapturedCefFrame.IsValid() && !CapturedCefFrame->IsEmpty())
				{
					// will safely delete the FSlateTextureData pointer when done with it on the rendering thread
					UpdatableTextures[Type]->UpdateTextureThreadSafeWithTextureData(CapturedCefFrame->CreateSlateTextureData(ViewportRect.Size()));
					HandleRenderingError();
				}
			}
			else
			{
				UpdatableTextures[Type]->UpdateTextureThreadSafeRaw(Width, Height, Buffer, DirtyRect);
				HandleRenderingError();
				bNeedsRedraw = true;
			}

			if (Type == PET_POPUP && bShowPopupRequested)
			{
				bShowPopupRequested = false;
				bPopupHasFocus = true;
				OnShowPopup().Broadcast(PopupRect);
			}
		}
	}

	bIsInitialized = true;
	if (bNeedsRedraw)
	{
		NeedsRedrawEvent.Broadcast();
	}
}

void FCEFWebBrowserWindow::OnAcceleratedPaint(CefRenderHandler::PaintElementType Type, const CefRenderHandler::RectList& DirtyRects, const CefAcceleratedPaintInfo& Info)
{
	FIntRect DirtyRect = (DirtyRects.size() == 1) ? FIntRect(DirtyRects[0].x, DirtyRects[0].y, DirtyRects[0].x + DirtyRects[0].width, DirtyRects[0].y + DirtyRects[0].height) : FIntRect();
	FIntRect ViewportRect = FIntRect(0, 0, ViewportSize.X, ViewportSize.Y).Scale(ViewportDPIScaleFactor);
	if (DirtyRect.IsEmpty() || ViewportRect.IsEmpty())
	{
		return;
	}

	++NumFramesSinceViewportResize;

	bool bNeedsRedraw = false;
	if (!bUsingAcceleratedPaint)
	{
		UE_LOGF(LogWebBrowser, Error, "Accelerated CEF rendering NOT selected but OnAcceleratedPaint called. Enabling accelerated rendering for this browser window.");
		bUsingAcceleratedPaint = true;
		if (UpdatableTextures[Type] != nullptr)
		{
			if (FSlateRenderer* const Renderer = GetRenderer())
			{
				Renderer->ReleaseUpdatableTexture(UpdatableTextures[Type]);
				HandleRenderingError();
				UpdatableTextures[Type] = nullptr;
			}
		}
	}

#  if PLATFORM_WINDOWS
	void* SharedHandle = Info.shared_texture_handle;
#  elif PLATFORM_MAC
	void* SharedHandle = Info.shared_texture_io_surface;
#  else
	void* SharedHandle = nullptr;
#  endif

#if PLATFORM_MAC
	// an IOSurface backs the handle here and its texture is automatically updated if changed, so we only need to
	// update our texture if the backing handle itself changed
	if (LastPaintedSharedHandle == SharedHandle)
		return;
	LastPaintedSharedHandle = SharedHandle;
#endif

#if UE_CEF_HAS_RESIZE_BUG
	if (Type == PET_VIEW)
	{
		bool bWasCorrectNativeCefBuffer = bHasCorrectNativeCefBuffer;
#if CEF_VERSION_MAJOR >= 147
		const FIntRect SourceRect = FIntRect(Info.extra.content_rect.x, Info.extra.content_rect.y, Info.extra.content_rect.x + Info.extra.content_rect.width, Info.extra.content_rect.y + Info.extra.content_rect.height);
#else
		const FIntRect SourceRect = DirtyRect;
#endif
		bHasCorrectNativeCefBuffer = (SourceRect == ViewportRect);
		if (!bHasCorrectNativeCefBuffer)
		{
			UE_LOGF(LogWebBrowser, Verbose, "FCEFWebBrowserWindow::OnAcceleratedPaint: Frame size still wrong after %zu frames: %dx%d -> %dx%d (%d, %d)", NumFramesSinceViewportResize, ViewportRect.Width(), ViewportRect.Height(), SourceRect.Width(), SourceRect.Height(), SourceRect.Min.X, SourceRect.Min.Y);
		}
		else if (!bWasCorrectNativeCefBuffer)
		{
			UE_LOGF(LogWebBrowser, Verbose, "FCEFWebBrowserWindow::OnAcceleratedPaint: Frame size OK again after %zu frames: %dx%d -> %dx%d (%d, %d)", NumFramesSinceViewportResize, ViewportRect.Width(), ViewportRect.Height(), SourceRect.Width(), SourceRect.Height(), SourceRect.Min.X, SourceRect.Min.Y);
		}

		// During resizing, CEF occasionally sends textures with black bars either above/below or left/right and distorted content in between, we want to ignore these textures.
		// Fortunately, the SourceRect for these textures (and only these) has a non-zero X/Y depending on the black bars orientation, so we can detect them without inspecting the content.
		if (SourceRect.Min.X > 0 || SourceRect.Min.Y > 0)
		{
			// force a CEF resize even if we were below the threshold, as CEF considers those "good" frames and will stop sending more until requested
			NumFramesSinceViewportResize = MIN_FRAMES_FOR_CEF_RESIZE;
			return;
		}
	}
#endif // UE_CEF_HAS_RESIZE_BUG

	if (RHIRenderHelper && RHIRenderHelper->CopySharedTextureSync(UpdatableTextures[Type], SharedHandle, DirtyRect))
	{
		bNeedsRedraw = true;
		if (Type == PET_POPUP && bShowPopupRequested)
		{
			bShowPopupRequested = false;
			bPopupHasFocus = true;
			OnShowPopup().Broadcast(PopupRect);
		}
	}

	bIsInitialized = true;
	if (bNeedsRedraw)
	{
		NeedsRedrawEvent.Broadcast();
	}
}


void FCEFWebBrowserWindow::UpdateVideoBuffering()
{
	if (BufferedVideo.IsValid() && UpdatableTextures[PET_VIEW] != nullptr)
	{
		FSlateTextureData* SlateTextureData = BufferedVideo->GetNextFrameTextureData();
		if (SlateTextureData != nullptr)
		{
			UpdatableTextures[PET_VIEW]->UpdateTextureThreadSafeWithTextureData(SlateTextureData);
			HandleRenderingError();
		}
	}
}

#if PLATFORM_WINDOWS
bool FCEFWebBrowserWindow::LoadCustomCEF3Cursor(cef_cursor_type_t Type)
{
	// generated from the ui_unscaled_resources.h file in a CEF build
#define IDC_PAN_EAST 25846
#define IDC_PAN_MIDDLE 25847
#define IDC_PAN_MIDDLE_HORIZONTAL 25848
#define IDC_PAN_MIDDLE_VERTICAL 25849
#define IDC_PAN_NORTH 25850
#define IDC_PAN_NORTH_EAST 25851
#define IDC_PAN_NORTH_WEST 25852
#define IDC_PAN_SOUTH 25853
#define IDC_PAN_SOUTH_EAST 25854
#define IDC_PAN_SOUTH_WEST 25855
#define IDC_PAN_WEST 25856

	if (!FSlateApplication::IsInitialized())
	{
		return false;
	}

	HINSTANCE CEF3ModuleHandle = (HINSTANCE)CEF3Utils::GetCEF3ModuleHandle();
	if (CEF3ModuleHandle != nullptr)
	{
		HCURSOR customCursor = 0;
		switch (Type) {
		case CT_MIDDLE_PANNING_HORIZONTAL:
			customCursor = LoadCursor(CEF3ModuleHandle, MAKEINTRESOURCE(IDC_PAN_MIDDLE_HORIZONTAL));
			break;
		case CT_MIDDLE_PANNING_VERTICAL:
			customCursor = LoadCursor(CEF3ModuleHandle, MAKEINTRESOURCE(IDC_PAN_MIDDLE_VERTICAL));
			break;
		case CT_MIDDLEPANNING:
			customCursor = LoadCursor(CEF3ModuleHandle, MAKEINTRESOURCE(IDC_PAN_MIDDLE));
			break;
		case CT_SOUTHPANNING:
			customCursor = LoadCursor(CEF3ModuleHandle, MAKEINTRESOURCE(IDC_PAN_SOUTH));
			break;
		case CT_NORTHPANNING:
			customCursor = LoadCursor(CEF3ModuleHandle, MAKEINTRESOURCE(IDC_PAN_NORTH));
			break;
		case CT_EASTPANNING:
			customCursor = LoadCursor(CEF3ModuleHandle, MAKEINTRESOURCE(IDC_PAN_EAST));
			break;
		case CT_WESTPANNING:
			customCursor = LoadCursor(CEF3ModuleHandle, MAKEINTRESOURCE(IDC_PAN_WEST));
			break;
		case CT_NORTHEASTPANNING:
			customCursor = LoadCursor(CEF3ModuleHandle, MAKEINTRESOURCE(IDC_PAN_NORTH_EAST));
			break;
		case CT_NORTHWESTPANNING:
			customCursor = LoadCursor(CEF3ModuleHandle, MAKEINTRESOURCE(IDC_PAN_NORTH_WEST));
			break;
		case CT_SOUTHEASTPANNING:
			customCursor = LoadCursor(CEF3ModuleHandle, MAKEINTRESOURCE(IDC_PAN_SOUTH_EAST));
			break;
		case CT_SOUTHWESTPANNING:
			customCursor = LoadCursor(CEF3ModuleHandle, MAKEINTRESOURCE(IDC_PAN_SOUTH_WEST));
			break;
		}
			
		if (customCursor)
		{
			TSharedPtr<ICursor> PlatformCursor = FSlateApplication::Get().GetPlatformCursor();

			if (PlatformCursor.IsValid())
			{
				PlatformCursor->SetTypeShape(EMouseCursor::Custom, (void*)customCursor);
				Cursor = EMouseCursor::Custom;
				::SetCursor(customCursor);
			}
			return true;
		}
	}
	return false;
}
#endif

bool FCEFWebBrowserWindow::OnCursorChange(CefCursorHandle CefCursor, cef_cursor_type_t Type, const CefCursorInfo& CustomCursorInfo)
{
	if (!FSlateApplication::IsInitialized())
	{
		return false;
	}

	switch (Type) {
		// Map the basic 3 cursor types directly to Slate types on all platforms
		case CT_NONE:
			Cursor = EMouseCursor::None;
			break;
		case CT_POINTER:
			Cursor = EMouseCursor::Default;
			break;
		case CT_IBEAM:
			Cursor = EMouseCursor::TextEditBeam;
			break;
		#if PLATFORM_WINDOWS || PLATFORM_MAC
		default:
			// Platform specific support for native cursor types
			{
		#if PLATFORM_WINDOWS
				// check if we have a cursor in libcef.dll we can use
				if (LoadCustomCEF3Cursor(Type))
				{
					return true;
				}
		#endif
				TSharedPtr<ICursor> PlatformCursor = FSlateApplication::Get().GetPlatformCursor();

				if (PlatformCursor.IsValid())
				{
					PlatformCursor->SetTypeShape(EMouseCursor::Custom, (void*)CefCursor);
					Cursor = EMouseCursor::Custom;
				}
			}
			break;
		#else
		// Map to closest Slate equivalent on platforms where native cursors are not available.
		case CT_VERTICALTEXT:
			Cursor = EMouseCursor::TextEditBeam;
			break;
		case CT_EASTRESIZE:
		case CT_WESTRESIZE:
		case CT_EASTWESTRESIZE:
		case CT_COLUMNRESIZE:
			Cursor = EMouseCursor::ResizeLeftRight;
			break;
		case CT_NORTHRESIZE:
		case CT_SOUTHRESIZE:
		case CT_NORTHSOUTHRESIZE:
		case CT_ROWRESIZE:
			Cursor = EMouseCursor::ResizeUpDown;
			break;
		case CT_NORTHWESTRESIZE:
		case CT_SOUTHEASTRESIZE:
		case CT_NORTHWESTSOUTHEASTRESIZE:
			Cursor = EMouseCursor::ResizeSouthEast;
			break;
		case CT_NORTHEASTRESIZE:
		case CT_SOUTHWESTRESIZE:
		case CT_NORTHEASTSOUTHWESTRESIZE:
			Cursor = EMouseCursor::ResizeSouthWest;
			break;
		case CT_MOVE:
		case CT_MIDDLEPANNING:
		case CT_EASTPANNING:
		case CT_NORTHPANNING:
		case CT_NORTHEASTPANNING:
		case CT_NORTHWESTPANNING:
		case CT_SOUTHPANNING:
		case CT_SOUTHEASTPANNING:
		case CT_SOUTHWESTPANNING:
		case CT_WESTPANNING:
			Cursor = EMouseCursor::CardinalCross;
			break;
		case CT_CROSS:
			Cursor = EMouseCursor::Crosshairs;
			break;
		case CT_HAND:
			Cursor = EMouseCursor::Hand;
			break;
		case CT_GRAB:
			Cursor = EMouseCursor::GrabHand;
			break;
		case CT_GRABBING:
			Cursor = EMouseCursor::GrabHandClosed;
			break;
		case CT_NOTALLOWED:
		case CT_NODROP:
			Cursor = EMouseCursor::SlashedCircle;
			break;
		default:
			Cursor = EMouseCursor::Default;
			break;
		#endif
	}
	// Tell Slate to update the cursor now
	FSlateApplication::Get().QueryCursor();
	return false;
}

EWebTransitionSource TransitionTypeToSourceEnum(const CefRequest::TransitionType& Type)
{
	CefRequest::TransitionType TransitionSource = (CefRequest::TransitionType)(Type & TT_SOURCE_MASK);
	switch (TransitionSource)
	{
		case TT_LINK:			return EWebTransitionSource::Link;
		case TT_EXPLICIT:		return EWebTransitionSource::Explicit;
		case TT_AUTO_SUBFRAME:	return EWebTransitionSource::AutoSubframe;
		case TT_FORM_SUBMIT:	return EWebTransitionSource::FormSubmit;
		case TT_RELOAD:			return EWebTransitionSource::Reload;
		default:				return EWebTransitionSource::Unknown;
	}
}

EWebTransitionSourceQualifier TransitionTypeToSourceQualifierEnum(const CefRequest::TransitionType& Type)
{
	CefRequest::TransitionType TransitionSourceQualifier = (CefRequest::TransitionType)(Type & TT_QUALIFIER_MASK);
	switch (TransitionSourceQualifier)
	{
		case TT_BLOCKED_FLAG:			return EWebTransitionSourceQualifier::Blocked;
		case TT_FORWARD_BACK_FLAG:		return EWebTransitionSourceQualifier::ForwardBack;
		case TT_CHAIN_START_FLAG:		return EWebTransitionSourceQualifier::ChainStart;
		case TT_CHAIN_END_FLAG:			return EWebTransitionSourceQualifier::ChainEnd;
		case TT_CLIENT_REDIRECT_FLAG:	return EWebTransitionSourceQualifier::ClientRedirect;
		case TT_SERVER_REDIRECT_FLAG:	return EWebTransitionSourceQualifier::ServerRedirect;
		default:						return EWebTransitionSourceQualifier::Unknown;
	}
}

bool FCEFWebBrowserWindow::OnBeforeBrowse(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefRequest> Request, bool user_gesture, bool bIsRedirect)
{
	if (InternalCefBrowser != nullptr && InternalCefBrowser->IsSame(Browser))
	{
		CefRefPtr<CefFrame> MainFrame = InternalCefBrowser->GetMainFrame();
		if (MainFrame.get() != nullptr)
		{
			if (OnBeforeBrowse().IsBound())
			{
				FString Url = WCHAR_TO_TCHAR(Request->GetURL().ToWString().c_str());
				bool bIsMainFrame = Frame->IsMain();

				FWebNavigationRequest RequestDetails;
				RequestDetails.bIsRedirect = bIsRedirect;
				RequestDetails.bIsMainFrame = bIsMainFrame;

				const CefRequest::TransitionType RequestTransitionType = Request->GetTransitionType();
				RequestDetails.TransitionSource = TransitionTypeToSourceEnum(RequestTransitionType);
				RequestDetails.TransitionSourceQualifier = TransitionTypeToSourceQualifierEnum(RequestTransitionType);
				RequestDetails.bIsExplicitTransition = RequestDetails.TransitionSource == EWebTransitionSource::Explicit;
			
				if (bIsMainFrame)
				{
					// We need to defer all future navigations until we can determine if this current navigation is going to be handled or not
					bDeferNavigations = true;
				}

				bool bHandled = OnBeforeBrowse().Execute(Url, RequestDetails);
				if (bIsMainFrame)
				{
					// If the browse request is handled and this is the main frame we must defer LoadUrl() calls until the request is fully aborted in/after NotifyDocumentError
					bDeferNavigations = bHandled && !bIsRedirect;
					if (bDeferNavigations)
					{
						PendingAbortUrl = Url;
					}
					else if (HasPendingNavigation())
					{
						ProcessPendingNavigation();
					}
				}
				return bHandled;
			}
		}
	}
	return false;
}



FString URLRequestSTatusToString(const CefResourceRequestHandler::URLRequestStatus& Status)
{
	const static FString URLRequestStatus_Success("SUCCESS");
	const static FString URLRequestStatus_IoPending("IO_PENDING");
	const static FString URLRequestStatus_Canceled("CANCELED");
	const static FString URLRequestStatus_Failed("FAILED");
	const static FString URLRequestStatus_Unknown("UNKNOWN");

	FString StatusStr;
	switch (Status)
	{
	case CefResourceRequestHandler::URLRequestStatus::UR_SUCCESS:
		StatusStr = URLRequestStatus_Success;
		break;
	case CefResourceRequestHandler::URLRequestStatus::UR_IO_PENDING:
		StatusStr = URLRequestStatus_IoPending;
		break;
	case CefResourceRequestHandler::URLRequestStatus::UR_CANCELED:
		StatusStr = URLRequestStatus_Canceled;
		break;
	case CefResourceRequestHandler::URLRequestStatus::UR_FAILED:
		StatusStr = URLRequestStatus_Failed;
		break;
	case CefResourceRequestHandler::URLRequestStatus::UR_UNKNOWN:
		StatusStr = URLRequestStatus_Unknown;
		break;
	default:
		StatusStr = URLRequestStatus_Unknown;
		break;
	}
	return StatusStr;
}

void FCEFWebBrowserWindow::HandleOnBeforeResourceLoad(const CefString& URL, CefRequest::ResourceType Type, FRequestHeaders& AdditionalHeaders, const bool AllowUserCredentials)
{
	BeforeResourceLoadDelegate.ExecuteIfBound(WCHAR_TO_TCHAR(URL.ToWString().c_str()), ResourceTypeToString(Type), AdditionalHeaders, AllowUserCredentials);
}

void FCEFWebBrowserWindow::HandleOnResourceLoadComplete(const CefString& URL, CefRequest::ResourceType Type, CefResourceRequestHandler::URLRequestStatus Status, int64 ContentLength)
{
	ResourceLoadCompleteDelegate.ExecuteIfBound(WCHAR_TO_TCHAR(URL.ToWString().c_str()), ResourceTypeToString(Type), URLRequestSTatusToString(Status), ContentLength);
}

EWebBrowserConsoleLogSeverity CefLogSeverityToWebBrowser(cef_log_severity_t Level)
{
	switch (Level)
	{
	case LOGSEVERITY_VERBOSE:
		return EWebBrowserConsoleLogSeverity::Verbose;
	//case LOGSEVERITY_DEBUG: // same as LOGSEVERITY_VERBOSE
	//	return Verbose;
	case LOGSEVERITY_INFO:
		return EWebBrowserConsoleLogSeverity::Info;
	case LOGSEVERITY_WARNING:
		return EWebBrowserConsoleLogSeverity::Warning;
	case LOGSEVERITY_ERROR:
		return EWebBrowserConsoleLogSeverity::Error;
	case LOGSEVERITY_FATAL:
		return EWebBrowserConsoleLogSeverity::Fatal;
	case LOGSEVERITY_DEFAULT:
	default:
		return EWebBrowserConsoleLogSeverity::Default;
	}
}

void FCEFWebBrowserWindow::HandleOnConsoleMessage(CefRefPtr<CefBrowser> Browser, cef_log_severity_t Level, const CefString& Message, const CefString& Source, int32 Line)
{
	ConsoleMessageDelegate.ExecuteIfBound(WCHAR_TO_TCHAR(Message.ToWString().c_str()), WCHAR_TO_TCHAR(Source.ToWString().c_str()), Line, CefLogSeverityToWebBrowser(Level));
}

TOptional<FString> FCEFWebBrowserWindow::GetResourceContent(CefRefPtr< CefFrame > Frame, CefRefPtr< CefRequest > Request)
{
	if (ContentsToLoad.IsSet())
	{
		FString Contents = ContentsToLoad.GetValue();
		ContentsToLoad.Reset();
		return Contents;
	}
	if (OnLoadUrl().IsBound())
	{
		FString Method = WCHAR_TO_TCHAR(Request->GetMethod().ToWString().c_str());
		FString Url = WCHAR_TO_TCHAR(Request->GetURL().ToWString().c_str());
		FString Response;
		if (OnLoadUrl().Execute(Method, Url, Response))
		{
			return Response;
		}
	}

	return TOptional<FString>();
}


int32 FCEFWebBrowserWindow::GetCefKeyboardModifiers(const FKeyEvent& KeyEvent)
{
	int32 Modifiers = GetCefInputModifiers(KeyEvent);

	const FKey Key = KeyEvent.GetKey();
	if (Key == EKeys::LeftAlt ||
		Key == EKeys::LeftCommand ||
		Key == EKeys::LeftControl ||
		Key == EKeys::LeftShift)
	{
		Modifiers |= EVENTFLAG_IS_LEFT;
	}
	if (Key == EKeys::RightAlt ||
		Key == EKeys::RightCommand ||
		Key == EKeys::RightControl ||
		Key == EKeys::RightShift)
	{
		Modifiers |= EVENTFLAG_IS_RIGHT;
	}
	if (Key == EKeys::NumPadZero ||
		Key == EKeys::NumPadOne ||
		Key == EKeys::NumPadTwo ||
		Key == EKeys::NumPadThree ||
		Key == EKeys::NumPadFour ||
		Key == EKeys::NumPadFive ||
		Key == EKeys::NumPadSix ||
		Key == EKeys::NumPadSeven ||
		Key == EKeys::NumPadEight ||
		Key == EKeys::NumPadNine)
	{
		Modifiers |= EVENTFLAG_IS_KEY_PAD;
	}

	return Modifiers;
}

int32 FCEFWebBrowserWindow::GetCefMouseModifiers(const FPointerEvent& InMouseEvent)
{
	int32 Modifiers = GetCefInputModifiers(InMouseEvent);

	if (InMouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		Modifiers |= EVENTFLAG_LEFT_MOUSE_BUTTON;
	}
	if (InMouseEvent.IsMouseButtonDown(EKeys::MiddleMouseButton))
	{
		Modifiers |= EVENTFLAG_MIDDLE_MOUSE_BUTTON;
	}
	if (InMouseEvent.IsMouseButtonDown(EKeys::RightMouseButton))
	{
		Modifiers |= EVENTFLAG_RIGHT_MOUSE_BUTTON;
	}

	return Modifiers;
}

CefMouseEvent FCEFWebBrowserWindow::GetCefMouseEvent(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup)
{
	FVector2D LocalPos = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

	// convert from game viewport coords to webview coords
	if (bIsPopup)
	{
		LocalPos += FVector2D(PopupRect.Min) * ViewportScaleFactor;
	}
	else
	{
		LocalPos *= ViewportScaleFactor;
	}

	CefMouseEvent Event;
	Event.x = LocalPos.X;
	Event.y = LocalPos.Y;
	Event.modifiers = GetCefMouseModifiers(MouseEvent);
	return Event;
}

int32 FCEFWebBrowserWindow::GetCefInputModifiers(const FInputEvent& InputEvent)
{
	int32 Modifiers = 0;

	if (InputEvent.IsShiftDown())
	{
		Modifiers |= EVENTFLAG_SHIFT_DOWN;
	}
	if (InputEvent.IsControlDown())
	{
#if PLATFORM_MAC
		// Slate swaps the flags for Command and Control on OSX, so we need to swap them back for CEF
		Modifiers |= EVENTFLAG_COMMAND_DOWN;
#else
		Modifiers |= EVENTFLAG_CONTROL_DOWN;
#endif
	}
	if (InputEvent.IsAltDown())
	{
		Modifiers |= EVENTFLAG_ALT_DOWN;
	}
	if (InputEvent.IsCommandDown())
	{
#if PLATFORM_MAC
		// Slate swaps the flags for Command and Control on OSX, so we need to swap them back for CEF
		Modifiers |= EVENTFLAG_CONTROL_DOWN;
#else
		Modifiers |= EVENTFLAG_COMMAND_DOWN;
#endif
	}
	if (InputEvent.AreCapsLocked())
	{
		Modifiers |= EVENTFLAG_CAPS_LOCK_ON;
	}

	return Modifiers;
}

bool FCEFWebBrowserWindow::CanSupportAcceleratedPaint()
{
	static bool DisableAcceleratedPaint = FParse::Param(FCommandLine::Get(), TEXT("nocefaccelpaint"));
	if (DisableAcceleratedPaint)
	{
		return false;
	}

	// Use off screen rendering so we can integrate with our windows
#if PLATFORM_LINUX
	return false;
#elif PLATFORM_WINDOWS
	return FCEFWebBrowserWindowRHIHelper::BUseSupportedRHIRenderer();
#elif PLATFORM_MAC
	return false; // Needs RHI support for the CreateSharedHandleTexture call
#else
	return false;
#endif

}

void FCEFWebBrowserWindow::UpdateCachedGeometry(const FGeometry& AllottedGeometry)
{
	// Extract the game viewport scale factor from the global Scale, which also includes the DPI scale factor from the OS
	ViewportScaleFactor = AllottedGeometry.Scale / ViewportDPIScaleFactor;

#if !PLATFORM_LINUX
	// Forward along the geometry for use by IME
	Ime->UpdateCachedGeometry(AllottedGeometry);
#endif
}

void FCEFWebBrowserWindow::CheckTickActivity()
{
	// Early out if we're currently hidden, not initialized or currently loading.
	if (bIsHidden || !IsValid() || IsLoading() || ViewportSize == FIntPoint::ZeroValue)
	{
		return;
	}

	// We clear the bTickedLastFrame flag here and set it on every Slate tick.
	// If it's still clear when we come back it means we're not getting ticks from slate.
	// Note: The BrowserSingleton object will not invoke this method if Slate itself is sleeping.
	// Therefore we can safely assume the widget is hidden in that case.
	if (!bTickedLastFrame)
	{
		SetIsHidden(true);
	}

	bTickedLastFrame = false;
}

void FCEFWebBrowserWindow::RequestNavigationInternal(FString Url, FString Contents)
{
	if (!IsValid())
	{
		return;
	}

	CefRefPtr<CefFrame> MainFrame = InternalCefBrowser->GetMainFrame();
	if (MainFrame.get() != nullptr)
	{
		ContentsToLoad = Contents.IsEmpty() ? TOptional<FString>() : Contents;
		PendingLoadUrl = Url;

		if (!bDeferNavigations)
		{
			ProcessPendingNavigation();
		}
	}
}

bool FCEFWebBrowserWindow::HasPendingNavigation()
{
	return !PendingLoadUrl.IsEmpty();
}

void FCEFWebBrowserWindow::ProcessPendingNavigation()
{
	if (!IsValid() || bDeferNavigations || !HasPendingNavigation())
	{
		return;
	}

	CefRefPtr<CefFrame> MainFrame = InternalCefBrowser->GetMainFrame();
	if (MainFrame.get() != nullptr)
	{
		CefString Url = TCHAR_TO_WCHAR(*PendingLoadUrl);
		PendingLoadUrl.Empty();
#if PLATFORM_MAC
		if ([NSThread isMainThread])
		{
			MainFrame->LoadURL(Url);
		}
		else
		{
			MainThreadCall(^{
				MainFrame->LoadURL(Url);
			});
		}
#else
		MainFrame->LoadURL(Url);
#endif
	}
}

void FCEFWebBrowserWindow::SetIsHidden(bool bValue)
{
	if (bIsHidden == bValue)
	{
		return;
	}
	bIsHidden = bValue;
	if (IsValid())
	{
		CefRefPtr<CefBrowserHost> BrowserHost = InternalCefBrowser->GetHost();
#if PLATFORM_WINDOWS
		HWND NativeWindowHandle = BrowserHost->GetWindowHandle();
		if (NativeWindowHandle != nullptr)
		{
			// When rendering directly into a subwindow, we must hide the native window when fully obscured
			::ShowWindow(NativeWindowHandle, bIsHidden ? SW_HIDE : SW_SHOW);

			if (bIsHidden)
			{
				TSharedPtr<SWindow> ParentWindowPtr = ParentWindow.Pin();
				if (::IsWindowEnabled(NativeWindowHandle) && ParentWindowPtr.IsValid())
				{
					::SetFocus((HWND)ParentWindowPtr->GetNativeWindow()->GetOSWindowHandle());
				}
				// when hidden also resize the window to 0x0 to further reduce resource usage. This copies the
				// behavior of the CefClient code, see browser_window_std_win.cc in the ::Hide() function. This code
				// is also required for the HTML5 visibility API to work 
				SetWindowPos(NativeWindowHandle, NULL, 0, 0, 0, 0, SWP_NOZORDER | SWP_NOMOVE | SWP_NOACTIVATE);
			}
			else
			{
				// restore the window to its right size/position
				HWND Parent = ::GetParent(NativeWindowHandle);
				// Position is in screen coordinates, so we'll need to get the parent window location first.
				RECT ParentRect = { 0, 0, 0, 0 };
				if (Parent)
				{
					::GetWindowRect(Parent, &ParentRect);
				}

				FIntPoint WindowSizeScaled = (FVector2D(ViewportSize) * ViewportDPIScaleFactor).IntPoint();

				::SetWindowPos(NativeWindowHandle, 0, ViewportPos.X - ParentRect.left, ViewportPos.Y - ParentRect.top, WindowSizeScaled.X, WindowSizeScaled.Y, 0);
			}
		}
		else
		{
#elif PLATFORM_MAC
		CefWindowHandle NativeWindowHandle = BrowserHost->GetWindowHandle();
		if (NativeWindowHandle != nullptr)
		{
			NSView *browserView = CAST_CEF_WINDOW_HANDLE_TO_NSVIEW(NativeWindowHandle);
			if (bIsHidden)
			{
				[browserView setHidden:YES];
			}
			else
			{
				[browserView setHidden:NO];
			}
			
		}
		else
		{
#endif
			BrowserHost->WasHidden(bIsHidden);
#if PLATFORM_WINDOWS || PLATFORM_MAC
		}
#endif
	}
}

void FCEFWebBrowserWindow::SetIsDisabled(bool bValue)
{
	if (bIsDisabled == bValue)
	{
		return;
	}
	bIsDisabled = bValue;
	SetIsHidden(bIsDisabled);
}

TSharedPtr<SWindow> FCEFWebBrowserWindow::GetParentWindow() const
{
	TSharedPtr<SWindow> ParentWindowPtr = ParentWindow.Pin();
	return ParentWindowPtr;
}

void FCEFWebBrowserWindow::SetParentWindow(TSharedPtr<SWindow> Window)
{
	ParentWindow = Window;
#if PLATFORM_WINDOWS
	if (IsValid())
	{
		CefRefPtr<CefBrowserHost> BrowserHost = InternalCefBrowser->GetHost();

		HWND NativeWindowHandle = BrowserHost->GetWindowHandle();
		if (NativeWindowHandle != nullptr)
		{
			TSharedPtr<SWindow> ParentWindowPtr = ParentWindow.Pin();
			void* ParentWindowHandle = (ParentWindow.IsValid() && ParentWindowPtr->GetNativeWindow().IsValid()) ? ParentWindowPtr->GetNativeWindow()->GetOSWindowHandle() : nullptr;
			if (ParentWindowHandle != nullptr)
			{
				// When rendering directly to a HWND update its parent windown
				::SetParent(NativeWindowHandle, (HWND)ParentWindowHandle);
			}
		}
	}
#endif
}

CefRefPtr<CefDictionaryValue> FCEFWebBrowserWindow::GetProcessInfo()
{
	CefRefPtr<CefDictionaryValue> Retval = nullptr;
	if (IsValid())
	{
		Retval = CefDictionaryValue::Create();
		Retval->SetInt("browser", InternalCefBrowser->GetIdentifier());
		Retval->SetDictionary("bindings", Scripting->GetPermanentBindings());
	}
	return Retval;
}

void FCEFWebBrowserWindow::SetCaptureCefFrames(const bool bValue)
{
#if UE_CEF_HAS_RESIZE_BUG
	if (bValue)
	{
		if (!CapturedCefFrame.IsValid())
		{
			CapturedCefFrame = MakeShared<FCEFCapturedFrame>();
		}
	}
	else
	{
		CapturedCefFrame.Reset();
	}
#endif
}

bool FCEFWebBrowserWindow::HasResizeBug() const
{
#if UE_CEF_HAS_RESIZE_BUG
	return true;
#else
	return false;
#endif
}

bool FCEFWebBrowserWindow::IsCefResizeNeeded() const
{
	if (!bHasCorrectNativeCefBuffer && NumFramesSinceViewportResize >= MIN_FRAMES_FOR_CEF_RESIZE)
	{
		UE_LOGF(LogWebBrowser, Verbose, "FCEFWebBrowserWindow::IsCefResizeNeeded: Forcing CEF resize");
		return true;
	}
	return false;
}

void FCEFWebBrowserWindow::RequestCefResize()
{
	if (!IsValid())
	{
		return;
	}

	InternalCefBrowser->GetHost()->WasResized();
	// Recent CEF versions fail to trigger a repaint after WasResized() is called
	// so we must force one on the view (and the popup if visible):
	// (upstream issue: https://github.com/chromiumembedded/cef/issues/3826)
#if UE_CEF_HAS_RESIZE_BUG
	InternalCefBrowser->GetHost()->Invalidate(PET_VIEW);
	if (bPopupHasFocus)
	{
		InternalCefBrowser->GetHost()->Invalidate(PET_POPUP);
	}
#endif
}

bool FCEFWebBrowserWindow::OnProcessMessageReceived(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> frame, CefProcessId SourceProcess, CefRefPtr<CefProcessMessage> Message)
{
	if (IsClosing())
	{
		UE_LOGF(LogWebBrowser, Warning, "Received a process message for a browser window in closing state, ignoring.");
		return false;
	}
	if (!IsValid())
	{
		UE_LOGF(LogWebBrowser, Warning, "Received a process message for a browser window in invalid state, ignoring.");
		return false;
	}
	if (!Browser || !Browser->IsValid())
	{
		UE_LOGF(LogWebBrowser, Warning, "Received a process message with an invalid browser state, ignoring.");
		return false;
	}

	bool bHandled = Scripting->OnProcessMessageReceived(Browser, SourceProcess, Message);

	if (!bHandled)
	{
#if PLATFORM_MAC
		if (!bInDirectHwndMode) // IME is handled by the CEF control in direct render mode
		{
			bHandled = Ime->OnProcessMessageReceived(Browser, SourceProcess, Message);
		}
#elif PLATFORM_WINDOWS
		bHandled = Ime->OnProcessMessageReceived(Browser, SourceProcess, Message);
#endif
	}
	
	return bHandled;
}

void FCEFWebBrowserWindow::BindUObject(const FString& Name, UObject* Object, bool bIsPermanent)
{
	Scripting->BindUObject(Name, Object, bIsPermanent);
}

void FCEFWebBrowserWindow::UnbindUObject(const FString& Name, UObject* Object, bool bIsPermanent)
{
	Scripting->UnbindUObject(Name, Object, bIsPermanent);
}

void FCEFWebBrowserWindow::BindInputMethodSystem(ITextInputMethodSystem* TextInputMethodSystem)
{
#if !PLATFORM_LINUX
	Ime->BindInputMethodSystem(TextInputMethodSystem);
#endif
}

void FCEFWebBrowserWindow::UnbindInputMethodSystem()
{
#if !PLATFORM_LINUX
	Ime->UnbindInputMethodSystem();
#endif
}

void FCEFWebBrowserWindow::OnBrowserClosing()
{
	bIsClosing = true;
}

void FCEFWebBrowserWindow::OnBrowserClosed()
{
	if (OnCloseWindow().IsBound())
	{
		OnCloseWindow().Execute(TWeakPtr<IWebBrowserWindow>(SharedThis(this)));
	}

	Scripting->UnbindCefBrowser();
#if !PLATFORM_LINUX
	Ime->UnbindCefBrowser();
#endif
	InternalCefBrowser = nullptr;
}

void FCEFWebBrowserWindow::SetPopupMenuPosition(CefRect CefPopupSize)
{
	// The popup position is in webview coords, convert to game viewport coords so it shows up in the right place
	const FIntPoint PopupPosition = (FVector2D(CefPopupSize.x, CefPopupSize.y) / ViewportScaleFactor).IntPoint();
	PopupRect = FIntRect(PopupPosition, PopupPosition + FIntPoint(CefPopupSize.width, CefPopupSize.height));

	// Resize the popup texture if using accelerated rendering, taking OS DPI scaling into account
	if (bUsingAcceleratedPaint && RHIRenderHelper)
	{
		const FIntPoint TextureSize = (FVector2D(CefPopupSize.width, CefPopupSize.height) * ViewportDPIScaleFactor).IntPoint();
		UpdatableTextures[PET_POPUP] = RHIRenderHelper->ResizeSlateUpdatableTexture(UpdatableTextures[PET_POPUP], TextureSize);
	}
}

void FCEFWebBrowserWindow:: ShowPopupMenu(bool bShow)
{
	if (bShow)
	{
		bShowPopupRequested = true; // We have to delay showing the popup until we get the first OnPaint on it.
	}
	else
	{
		bPopupHasFocus = false;
		bShowPopupRequested = false;
		PopupRect = FIntRect();
		OnDismissPopup().Broadcast();
	}
}

void FCEFWebBrowserWindow::OnImeCompositionRangeChanged(
	CefRefPtr<CefBrowser> Browser, 
	const CefRange& SelectionRange, 
	const CefRenderHandler::RectList& CharacterBounds)
{
	if (InternalCefBrowser != nullptr && InternalCefBrowser->IsSame(Browser))
	{
#if !PLATFORM_LINUX
		Ime->CEFCompositionRangeChanged(SelectionRange, CharacterBounds);
#endif
	}
}

void FCEFWebBrowserWindow::OnTextSelectionChanged(
	CefRefPtr<CefBrowser> Browser,
	const CefString& SelectedText,
	const CefRange& SelectionRange)
{
	if (InternalCefBrowser != nullptr && InternalCefBrowser->IsSame(Browser))
	{
#if !PLATFORM_LINUX
		Ime->CEFTextSelectionChanged(SelectedText, SelectionRange);
#endif
	}
}

void FCEFWebBrowserWindow::UpdateDragRegions(const TArray<FWebBrowserDragRegion>& Regions)
{
	DragRegions = Regions;
}

bool FCEFWebBrowserWindow::IsInDragRegion(const FIntPoint& Point)
{
	// Here we traverse the array of drag regions backwards because we assume the drag regions are z ordered such that 
	//    the end of the list contains the drag regions of the top most elements of the web page.  We can stop checking
	//    once we hit a region that contains our point.
	for (int32 Idx = DragRegions.Num() - 1; Idx >= 0; --Idx)
	{
		if (DragRegions[Idx].Rect.Contains(Point))
		{
			return DragRegions[Idx].bDraggable;
		}
	}
	return false;
}

void FCEFWebBrowserWindow::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (!IsValid())
	{
		return;
	}

	TSharedPtr<FExternalDragOperation> ExternalDragOp = DragDropEvent.GetOperationAs<FExternalDragOperation>();
	if (!ExternalDragOp.IsValid())
	{
		return;
	}

	CefRefPtr<CefDragData> DragData = CefDragData::Create();

	if (ExternalDragOp->HasFiles())
	{
		const TArray<FString>& Files = ExternalDragOp->GetFiles();
		for (const FString& FilePath : Files)
		{
			DragData->AddFile(TCHAR_TO_WCHAR(*FilePath), CefString());
		}
	}

	if (ExternalDragOp->HasText())
	{
		const FString& Text = ExternalDragOp->GetText();
		if (Text.StartsWith(TEXT("file://")))
		{
			// Convert file URL to a local file path and add as a file:
			// file://[/]C:/path/to/file -> C:/path/to/file (Windows)
			// file://[/]path/to/file -> /path/to/file (Unix)
			FString FilePath = Text.RightChop(7);
			// Remove any left '/' in front of the Windows drive
			if (FilePath.Len() >= 3 && FilePath[2] == TEXT(':'))
			{
				FilePath = FilePath.RightChop(1);
			}
			// Add any missing '/' in front of the Unix path
			if (FilePath.Len() >= 2 && FilePath[1] != TEXT(':') && FilePath[0] != TEXT('/'))
			{
				FilePath.InsertAt(0, TEXT('/'));
			}

			// URL decode the path (handle %20 for spaces, etc.)
			FilePath = FPlatformHttp::UrlDecode(FilePath);
			FPaths::NormalizeFilename(FilePath);

			DragData->AddFile(TCHAR_TO_WCHAR(*FilePath), CefString());
		}
		else if (Text.StartsWith(TEXT("http://")) || Text.StartsWith(TEXT("https://")))
		{
			DragData->SetLinkURL(TCHAR_TO_WCHAR(*Text));
			DragData->SetLinkTitle(TCHAR_TO_WCHAR(*Text));
		}
		else
		{
			DragData->SetFragmentText(TCHAR_TO_WCHAR(*Text));
		}
	}

	// Convert the mouse position to CEF coordinates
	CefMouseEvent MouseEvent = GetCefMouseEvent(MyGeometry, DragDropEvent, false);

	InternalCefBrowser->GetHost()->DragTargetDragEnter(DragData, MouseEvent, DRAG_OPERATION_COPY);
}

FReply FCEFWebBrowserWindow::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (!IsValid())
	{
		return FReply::Unhandled();
	}

	TSharedPtr<FExternalDragOperation> ExternalDragOp = DragDropEvent.GetOperationAs<FExternalDragOperation>();
	if (!ExternalDragOp.IsValid())
	{
		return FReply::Unhandled();
	}

	// Check if we have any supported drag data
	if (!ExternalDragOp->HasFiles() && !ExternalDragOp->HasText())
	{
		return FReply::Unhandled();
	}

	// Convert the mouse position to CEF coordinates
	CefMouseEvent MouseEvent = GetCefMouseEvent(MyGeometry, DragDropEvent, false);

	// Allow all common operations - CEF will determine what's actually allowed based on the drag data
	CefBrowserHost::DragOperationsMask AllowedOps = static_cast<CefBrowserHost::DragOperationsMask>(
		DRAG_OPERATION_COPY | DRAG_OPERATION_MOVE | DRAG_OPERATION_LINK);

	InternalCefBrowser->GetHost()->DragTargetDragOver(MouseEvent, AllowedOps);

	return FReply::Handled();
}

void FCEFWebBrowserWindow::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	if (!IsValid())
	{
		return;
	}

	TSharedPtr<FExternalDragOperation> ExternalDragOp = DragDropEvent.GetOperationAs<FExternalDragOperation>();
	if (!ExternalDragOp.IsValid())
	{
		return;
	}

	InternalCefBrowser->GetHost()->DragTargetDragLeave();
}

FReply FCEFWebBrowserWindow::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (!IsValid())
	{
		return FReply::Unhandled();
	}

	TSharedPtr<FExternalDragOperation> ExternalDragOp = DragDropEvent.GetOperationAs<FExternalDragOperation>();
	if (!ExternalDragOp.IsValid())
	{
		return FReply::Unhandled();
	}

	// Check if we have any supported drag data
	if (!ExternalDragOp->HasFiles() && !ExternalDragOp->HasText())
	{
		return FReply::Unhandled();
	}

	// Convert the mouse position to CEF coordinates
	CefMouseEvent MouseEvent = GetCefMouseEvent(MyGeometry, DragDropEvent, false);

	InternalCefBrowser->GetHost()->DragTargetDrop(MouseEvent);

	return FReply::Handled();
}

#endif
