// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/CursorReply.h"
#include "Input/Events.h"
#include "Input/Reply.h"
#include "Widgets/SViewport.h"
#include "WebBrowserSingleton.h"

#if WITH_CEF3

#include "IWebBrowserWindow.h"
#include "CEFBrowserHandler.h"

#include "CEFLibCefIncludes.h"

#include "CEFCapturedFrame.h"

#endif

class FBrowserBufferedVideo;
class FCEFBrowserHandler;
class FCEFJSScripting;
class FSlateUpdatableTexture;
class IWebBrowserPopupFeatures;
class IWebBrowserWindow;
struct Rect;
class FSlateShaderResource;
enum class EWebBrowserDocumentState;
struct FGeometry;
struct FPointerEvent;
class UObject;
struct FInputEvent;
class FWebJSScripting;
class FCEFImeHandler;
class ITextInputMethodSystem;
class FCEFWebBrowserWindowRHIHelper;

#if WITH_CEF3

/**
 * Helper for containing items required for CEF browser window creation.
 */
struct FWebBrowserWindowInfo
{
	FWebBrowserWindowInfo(CefRefPtr<CefBrowser> InBrowser, CefRefPtr<FCEFBrowserHandler> InHandler)
		: Browser(InBrowser)
		, Handler(InHandler)
	{}
	CefRefPtr<CefBrowser> Browser;
	CefRefPtr<FCEFBrowserHandler> Handler;
};

/**
* Representation of a window drag region.
*/
struct FWebBrowserDragRegion
{
	FWebBrowserDragRegion(const FIntRect& InRect, bool bInDraggable)
		: Rect(InRect)
		, bDraggable(bInDraggable)
	{}

	FIntRect Rect;
	bool bDraggable;
};

/**
 * Implementation of interface for dealing with a Web Browser window.
 */
class FCEFWebBrowserWindow
	: public IWebBrowserWindow
	, public TSharedFromThis<FCEFWebBrowserWindow>
{
	// Allow the Handler to access functions only it needs
	friend class FCEFBrowserHandler;

	// The WebBrowserSingleton should be the only one creating instances of this class
	friend class FWebBrowserSingleton;

	// CreateWidget should only be called by the WebBrowserView
	friend class SWebBrowserView;

private:
	/**
	 * Creates and initializes a new instance.
	 *
	 * @param Browser The CefBrowser object representing this browser window.
	 * @param Handler Pointer to the CEF handler for this window.
	 * @param Url The Initial URL that will be loaded.
	 * @param ContentsToLoad Optional string to load as a web page.
	 * @param UserAgentProduct the product part of the user agent string for HTTP requests sent by this window.
	 * @param bShowErrorMessage Whether to show an error message in case of loading errors.
	 * @param bThumbMouseButtonNavigation Whether to allow forward and back navigation via the mouse thumb buttons.
	 * @param bUseTransparency Whether to enable transparency.
	 * @param bJSBindingToLoweringEnabled Whether we ToLower all JavaScript member names.
	 */
	FCEFWebBrowserWindow(CefRefPtr<CefBrowser> Browser, CefRefPtr<FCEFBrowserHandler> Handler, FString Url, TOptional<FString> ContentsToLoad, FString UserAgentProduct, bool bShowErrorMessage, bool bThumbMouseButtonNavigation, bool bUseTransparency, bool bJSBindingToLoweringEnabled, bool bUsingAcceleratedPaint);

	/**
	 * Create the SWidget for this WebBrowserWindow
	 */
	TSharedRef<SViewport> CreateWidget();

public:
	/** Virtual Destructor. */
	virtual ~FCEFWebBrowserWindow();

	bool IsShowingErrorMessages() const { return bShowErrorMessage; }
	bool IsThumbMouseButtonNavigationEnabled() const { return bThumbMouseButtonNavigation; }
	bool UseTransparency() const { return bUseTransparency; }
	bool UsingAcceleratedPaint() const { return bUsingAcceleratedPaint; }
	FString GetUserAgentProduct() const { return UserAgentProduct; }
	
public:

	// IWebBrowserWindow Interface

	virtual void LoadURL(FString NewURL) override;
	virtual void LoadString(FString Contents, FString DummyURL) override;
	virtual void SetViewportSize(FIntPoint WindowSize, FIntPoint WindowPos) override;
	virtual FIntPoint GetViewportSize() const override { return FIntPoint::NoneValue; }
	virtual FSlateShaderResource* GetTexture(bool bIsPopup = false) override;
	virtual bool IsValid() const override;
	virtual bool IsInitialized() const override;
	virtual bool IsClosing() const override;
	virtual EWebBrowserDocumentState GetDocumentLoadingState() const override;
	virtual FString GetTitle() const override;
	virtual FString GetUrl() const override;
	virtual void GetSource(TFunction<void (const FString&)> Callback) const override;
	virtual bool OnKeyDown(const FKeyEvent& InKeyEvent) override;
	virtual bool OnKeyUp(const FKeyEvent& InKeyEvent) override;
	virtual bool OnKeyChar(const FCharacterEvent& InCharacterEvent) override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup) override;
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup) override;
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	virtual void SetSupportsMouseWheel(bool bValue) override;
	virtual bool GetSupportsMouseWheel() const override;
	virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup) override;
	virtual FReply OnTouchGesture(const FGeometry& MyGeometry, const FPointerEvent& GestureEvent, bool bIsPopup) override;
	virtual void OnFocus(bool SetFocus, bool bIsPopup) override;
	virtual void OnCaptureLost() override;
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	virtual bool CanGoBack() const override;
	virtual void GoBack() override;
	virtual bool CanGoForward() const override;
	virtual void GoForward() override;
	virtual bool IsLoading() const override;
	virtual void Reload() override;
	virtual void StopLoad() override;
	virtual void ExecuteJavascript(const FString& Script) override;
	virtual void CloseBrowser(bool bForce, bool bBlockTillClosed) override;
	virtual void BindUObject(const FString& Name, UObject* Object, bool bIsPermanent = true) override;
	virtual void UnbindUObject(const FString& Name, UObject* Object = nullptr, bool bIsPermanent = true) override;
	virtual void BindInputMethodSystem(ITextInputMethodSystem* TextInputMethodSystem) override;
	virtual void UnbindInputMethodSystem() override;
	virtual int GetLoadError() override;
	virtual void SetIsDisabled(bool bValue) override;
	virtual TSharedPtr<SWindow> GetParentWindow() const override;
	virtual void SetParentWindow(TSharedPtr<SWindow> Window) override;
	virtual void ShowFloatingCloseButton(bool bShow, bool bDraggable) override
	{		
	}

	DECLARE_DERIVED_EVENT(FCEFWebBrowserWindow, IWebBrowserWindow::FOnDocumentStateChanged, FOnDocumentStateChanged);
	virtual FOnDocumentStateChanged& OnDocumentStateChanged() override
	{
		return DocumentStateChangedEvent;
	}

	DECLARE_DERIVED_EVENT(FCEFWebBrowserWindow, IWebBrowserWindow::FOnTitleChanged, FOnTitleChanged);
	virtual FOnTitleChanged& OnTitleChanged() override
	{
		return TitleChangedEvent;
	}

	DECLARE_DERIVED_EVENT(FCEFWebBrowserWindow, IWebBrowserWindow::FOnUrlChanged, FOnUrlChanged);
	virtual FOnUrlChanged& OnUrlChanged() override
	{
		return UrlChangedEvent;
	}

	DECLARE_DERIVED_EVENT(FCEFWebBrowserWindow, IWebBrowserWindow::FOnToolTip, FOnToolTip);
	virtual FOnToolTip& OnToolTip() override
	{
		return ToolTipEvent;
	}

	DECLARE_DERIVED_EVENT(FCEFWebBrowserWindow, IWebBrowserWindow::FOnNeedsRedraw, FOnNeedsRedraw);
	virtual FOnNeedsRedraw& OnNeedsRedraw() override
	{
		return NeedsRedrawEvent;
	}

	virtual FOnBeforeBrowse& OnBeforeBrowse() override
	{
		return BeforeBrowseDelegate;
	}

	virtual FOnLoadUrl& OnLoadUrl() override
	{
		return LoadUrlDelegate;
	}

	virtual FOnCreateWindow& OnCreateWindow() override
	{
		return WebBrowserHandler->OnCreateWindow();
	}

	virtual FOnCloseWindow& OnCloseWindow() override
	{
		return CloseWindowDelegate;
	}

	virtual FOnFloatingCloseButtonPressed& OnFloatingCloseButtonPressed() override
	{
		return FloatingCloseButtonPressedDelegate;
	}

	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) override
	{
		return Cursor == EMouseCursor::Default ? FCursorReply::Unhandled() : FCursorReply::Cursor(Cursor);
	}

	virtual FOnBeforePopupDelegate& OnBeforePopup() override
	{
		return WebBrowserHandler->OnBeforePopup();
	}

	virtual FOnBeforeResourceLoadDelegate& OnBeforeResourceLoad() override
	{
		if (!WebBrowserHandler->OnBeforeResourceLoad().IsBoundToObject(this))
		{
			WebBrowserHandler->OnBeforeResourceLoad().BindSP(this, &FCEFWebBrowserWindow::HandleOnBeforeResourceLoad);
		}

		return BeforeResourceLoadDelegate;
	}

	virtual FOnResourceLoadCompleteDelegate& OnResourceLoadComplete() override
	{
		if (!WebBrowserHandler->OnResourceLoadComplete().IsBoundToObject(this))
		{
			WebBrowserHandler->OnResourceLoadComplete().BindSP(this, &FCEFWebBrowserWindow::HandleOnResourceLoadComplete);
		}

		return ResourceLoadCompleteDelegate;
	}

	virtual FOnConsoleMessageDelegate& OnConsoleMessage() override
	{
		if (!WebBrowserHandler->OnConsoleMessage().IsBoundToObject(this))
		{
			WebBrowserHandler->OnConsoleMessage().BindSP(this, &FCEFWebBrowserWindow::HandleOnConsoleMessage);
		}

		return ConsoleMessageDelegate;
	}

	DECLARE_DERIVED_EVENT(FCEFWebBrowserWindow, IWebBrowserWindow::FOnShowPopup, FOnShowPopup);
	virtual FOnShowPopup& OnShowPopup() override
	{
		return ShowPopupEvent;
	}

	DECLARE_DERIVED_EVENT(FCEFWebBrowserWindow, IWebBrowserWindow::FOnDismissPopup, FOnDismissPopup);
	virtual FOnDismissPopup& OnDismissPopup() override
	{
		return DismissPopupEvent;
	}

	virtual FOnShowDialog& OnShowDialog() override
	{
		return ShowDialogDelegate;
	}

	virtual FOnDismissAllDialogs& OnDismissAllDialogs() override
	{
		return DismissAllDialogsDelegate;
	}

	virtual FOnSuppressContextMenu& OnSuppressContextMenu() override
	{
		return SuppressContextMenuDelgate;
	}

	virtual FOnDragWindow& OnDragWindow() override
	{
		return DragWindowDelegate;
	}
		
	virtual FOnUnhandledKeyDown& OnUnhandledKeyDown() override
	{
		return UnhandledKeyDownDelegate;
	}

	virtual FOnUnhandledKeyUp& OnUnhandledKeyUp() override
	{
		return UnhandledKeyUpDelegate;
	}

	virtual FOnUnhandledKeyChar& OnUnhandledKeyChar() override
	{
		return UnhandledKeyCharDelegate;
	}

private:

	/**
	 * Used to obtain the internal CEF browser.
	 *
	 * @return The bound CEF browser.
	 */
	CefRefPtr<CefBrowser> GetCefBrowser();

	/**
	 * Sets the Title of this window.
	 *
	 * @param InTitle The new title of this window.
	 */
	void SetTitle(const CefString& InTitle);

	/**
	 * Sets the url of this window.
	 *
	 * @param InUrl The new url of this window.
	 */
	void SetUrl(const CefString& InUrl);

	/**
	 * Sets the tool tip for this window.
	 *
	 * @param InToolTip The text to show in the ToolTip. Empty string for no tool tip.
	 */
	void SetToolTip(const CefString& InToolTip);

	/**
	 * Get the current proportions of this window.
	 *
	 * @param Rect Reference to CefRect to store sizes.
	 * @return Whether Rect was set up correctly.
	 */
	void GetViewRect(CefRect& Rect);

	/** Notifies when document loading has failed. */
	void NotifyDocumentError(CefLoadHandler::ErrorCode InErrorCode, const CefString& ErrorText, const CefString& FailedUrl);
	
	/** Notifies clients that document loading has failed. */
	void NotifyDocumentError(int ErrorCode);

	/**
	 * Notifies clients that the loading state of the document has changed.
	 *
	 * @param IsLoading Whether the document is loading (false = completed).
	 */
	void NotifyDocumentLoadingStateChange(bool IsLoading);

	/**
	 * Called when there is an update to the rendered web page.
	 *
	 * @param Type Paint type.
	 * @param DirtyRects List of image areas that have been changed.
	 * @param Buffer Pointer to the raw texture data.
	 * @param Width Width of the texture.
	 * @param Height Height of the texture.
	 */
	void OnPaint(CefRenderHandler::PaintElementType Type, const CefRenderHandler::RectList& DirtyRects, const void* Buffer, int Width, int Height);

	/**
	 * Called when there is an update to the rendered web page.
	 *
	 * @param Type Paint type.
	 * @param DirtyRects List of image areas that have been changed.
	 * @param SharedHandle the handle for a D3D11 Texture2D that can be accessed via ID3D11Device using the OpenSharedResource method
	 */
	void OnAcceleratedPaint(CefRenderHandler::PaintElementType type, const CefRenderHandler::RectList& DirtyRects, const CefAcceleratedPaintInfo& Info);

	/**
	 * Called when cursor would change due to web browser interaction.
	 *
	 * @param Cursor Handle to CEF mouse cursor.
	 */
	bool OnCursorChange(CefCursorHandle Cursor, cef_cursor_type_t Type, const CefCursorInfo& CustomCursorInfo);

	/**
	 * Called when a message was received from the renderer process.
	 *
	 * @param Browser The CefBrowser for this window.
	 * @param SourceProcess The process id of the sender of the message. Currently always PID_RENDERER.
	 * @param Message The actual message.
	 * @return true if the message was handled, else false.
	 */
	bool OnProcessMessageReceived(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> frame, CefProcessId SourceProcess, CefRefPtr<CefProcessMessage> Message);

	/**
	 * Called before browser navigation.
	 *
	 * @param Browser The CefBrowser for this window.
	 * @param Frame The CefFrame the request came from.
	 * @param Request The CefRequest containing web request info.
	 * @param user_gesture true if the navigation was a result of a gesture, false otherwise.
	 * @param bIsRedirect true if the navigation was a result of redirection, false otherwise.
	 * @return true if the navigation was handled and no further processing of the navigation request should be disabled, false if the navigation should be handled by the default CEF implementation.
	 */
	bool OnBeforeBrowse(CefRefPtr<CefBrowser> Browser, CefRefPtr<CefFrame> Frame, CefRefPtr<CefRequest> Request, bool user_gesture, bool bIsRedirect);

	void HandleOnBeforeResourceLoad(const CefString& URL, CefRequest::ResourceType Type, FRequestHeaders& AdditionalHeaders, const bool AllowUserCredentials);
	void HandleOnResourceLoadComplete(const CefString& URL, CefRequest::ResourceType Type, CefResourceRequestHandler::URLRequestStatus Status, int64 ContentLength);
	void HandleOnConsoleMessage(CefRefPtr<CefBrowser> Browser, cef_log_severity_t Level, const CefString& Message, const CefString& Source, int32 Line);

	/**
	 * Called before loading a resource to allow overriding the content for a request.
	 *
	 * @return string content representing the content to show for the URL or an unset value to fetch the URL normally.
	 */
	TOptional<FString> GetResourceContent(CefRefPtr< CefFrame > Frame, CefRefPtr< CefRequest > Request);

	/**
	* Convenience function to translate modifier keys from Cef keyboard event
	*/
	FModifierKeysState SlateModifiersFromCefModifiers(const CefKeyEvent& CefEvent);

	/**
	 * Called when browser reports a key event that was not handled by it
	 */
	bool OnUnhandledKeyEvent(const CefKeyEvent& CefEvent);

	/**
	 * Handle showing javascript dialogs
	 */
	bool OnJSDialog(CefJSDialogHandler::JSDialogType DialogType, const CefString& MessageText, const CefString& DefaultPromptText, CefRefPtr<CefJSDialogCallback> Callback, bool& OutSuppressMessage);

	/**
	 * Handle showing the file select / upload dialogs
	 */
	bool OnFileDialog(CefDialogHandler::FileDialogMode Mode, const CefString& DialogTitle, const CefString& DefaultFilePath, const std::vector<CefString>& AcceptFilters, const std::vector<CefString>& AcceptExtensions, const std::vector<CefString>& AcceptDescriptions, CefRefPtr<CefFileDialogCallback> Callback);

	/**
	 * Handle showing unload confirmation dialogs
	 */
	bool OnBeforeUnloadDialog(const CefString& MessageText, bool IsReload, CefRefPtr<CefJSDialogCallback> Callback);

	/**
	 * Notify when any and all pending dialogs should be canceled
	 */
	void OnResetDialogState();

	/**
	 * Called when render process was terminated abnormally.
	 */
	void OnRenderProcessTerminated(CefRequestHandler::TerminationStatus Status);


	/** Called when the browser requests a new UI window
	 *
	 * @param NewBrowserWindow The web browser window to display in the new UI window.
	 * @param BrowserPopupFeatures The popup features and settings for the browser window.
	 * @return true if the UI window was created, false otherwise.
	 */
	bool RequestCreateWindow(const TSharedRef<IWebBrowserWindow>& NewBrowserWindow, const TSharedPtr<IWebBrowserPopupFeatures>& BrowserPopupFeatures);

	//bool SupportsCloseWindows();
	//bool RequestCloseWindow(const TSharedRef<IWebBrowserWindow>& BrowserWindow);


	/**
	 * Called once the browser begins closing procedures.
	 */
	void OnBrowserClosing();

	/**
	 * Called once the browser is closed.
	 */
	void OnBrowserClosed();

	/**
	 * Called to set the popup menu position and size. For the accelerated paint path,
	 * the size is used to pre-create the popup texture at the correct dimensions.
	 *
 	 * @param PopupSize The location and size of the popup widget.
	 */
	void SetPopupMenuPosition(CefRect PopupSize);

	/**
	 * Called to request that the popup widget is shown or hidden.
	 *
 	 * @param bShow true for showing the popup, false for hiding it.
	 */
	void ShowPopupMenu(bool bShow);

	/**
	 * Called when the IME composition DOM node has changed.
	 * @param Browser The CefBrowser for this window.
	 * @param SelectionRange The range of characters that have been selected.
	 * @param CharacterBounds The bounds of each character in view coordinates.
	 */
	void OnImeCompositionRangeChanged(
		CefRefPtr<CefBrowser> Browser,
		const CefRange& SelectionRange,
		const CefRenderHandler::RectList& CharacterBounds);

	/**
	 * Called when the text selection has changed for a text field in a web page, used for IME.
	 * @param Browser The CefBrowser for this window.
	 * @param SelectedText The selected text.
	 * @param SelectionRange The range of characters that have been selected.
	 */
	void OnTextSelectionChanged(
		CefRefPtr<CefBrowser> Browser,
		const CefString& SelectedText,
		const CefRange& SelectionRange);

	void UpdateDragRegions(const TArray<FWebBrowserDragRegion>& Regions);

public:

	/**
	 * Gets the Cef Keyboard Modifiers based on a Key Event.
	 *
	 * @param KeyEvent The Key event.
	 * @return Bits representing keyboard modifiers.
	 */
	static int32 GetCefKeyboardModifiers(const FKeyEvent& KeyEvent);

	/**
	 * Gets the Cef Mouse Modifiers based on a Mouse Event.
	 *
	 * @param InMouseEvent The Mouse event.
	 * @return Bits representing mouse modifiers.
	 */
	static int32 GetCefMouseModifiers(const FPointerEvent& InMouseEvent);

	/**
	 * Gets the Cef Input Modifiers based on an Input Event.
	 *
	 * @param InputEvent The Input event.
	 * @return Bits representing input modifiers.
	 */
	static int32 GetCefInputModifiers(const FInputEvent& InputEvent);


	/**
	 * Is this platform able to support the accelerated paint path for CEF. 
	 *
	 * @return true if supported AND enabled on this platform, false otherwise.
	 */
	static bool CanSupportAcceleratedPaint();
public:

	/**
	 * Called from the WebBrowserViewport tick event. Allows us to cache the geometry and use it for coordinate transformations.
	 */
	void UpdateCachedGeometry(const FGeometry& AllottedGeometry);

	/**
	 * Called from the WebBrowserSingleton tick event. Should test wether the widget got a tick from Slate last frame and set the state to hidden if not.
	 */
	void CheckTickActivity() override;

	/**
	* Called from the engine tick.
	*/
	void UpdateVideoBuffering();

	/**
	 * Called on every browser window when CEF launches a new render process.
	 * Used to ensure global JS objects are registered as soon as possible.
	 */
	CefRefPtr<CefDictionaryValue> GetProcessInfo();

	/**
	* Return true if this URL will support adding an Authorization header to it
	*/
	bool URLRequestAllowsCredentials(const FString& URL) const { return WebBrowserHandler->URLRequestAllowsCredentials(URL); }
	
	/**
	 * Whether we should use a workaround for this bug -
	 * https://github.com/chromiumembedded/cef/issues/3826
	 */
	void SetCaptureCefFrames(const bool bValue);
	
	/**
	 * Whether the CEF version we're using has a resize bug that sometimes requires requesting new frames after a resize.
	 */
	bool HasResizeBug() const;
	
	/**
	 * Indicates a CEF resize is needed, using heuristics to avoid too many resize requests during an active drag-resize.
	 */
	bool IsCefResizeNeeded() const;

	/**
	 * Requests a resize of the CEF webview to ViewportSize
	 */
	void RequestCefResize();
	
private:

	/** @return the currently valid renderer, if available */
	FSlateRenderer* const GetRenderer();

	/**Checks whether an error with the renderer occurred and handles it if one has */
	void HandleRenderingError();

	/** Releases the updatable textures */
	void ReleaseTextures();

	/** Creates the initial updatable textures */
	bool CreateInitialTextures();

	/** Executes or defers a LoadUrl navigation */
	void RequestNavigationInternal(FString Url, FString Contents);

	/** Specifies whether or not we have a pending deferred navigation */
	bool HasPendingNavigation();

	/** Executes navigation on a pending deferred navigation */
	void ProcessPendingNavigation();

	/** Helper that calls WasHidden on the CEF host object when the value changes */
	void SetIsHidden(bool bValue);

	/** Used by the key down and up handlers to convert Slate key events to the CEF equivalent. */
	void PopulateCefKeyEvent(const FKeyEvent& InKeyEvent, CefKeyEvent& OutKeyEvent);

	/** Used to convert a FPointerEvent to a CefMouseEvent */
	CefMouseEvent GetCefMouseEvent(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup);

	/** Specifies whether or not a point falls within any tagged drag regions that are draggable. */
	bool IsInDragRegion(const FIntPoint& Point);

	bool BlockInputInDirectHwndMode() const;

#if PLATFORM_WINDOWS
	/** manually load cursor icons from the CEF3 dll if needed, working around a CEF3 bug */
	bool LoadCustomCEF3Cursor(cef_cursor_type_t Type);
#endif
private:

	/** Current state of the document being loaded. */
	EWebBrowserDocumentState DocumentState;

	/** Interface to the texture we are rendering to. */
	FSlateUpdatableTexture* UpdatableTextures[2];

	/** Pointer to the CEF Browser for this window. */
	CefRefPtr<CefBrowser> InternalCefBrowser;

	/** Pointer to the CEF handler for this window. */
	CefRefPtr<FCEFBrowserHandler> WebBrowserHandler;

	/** Current title of this window. */
	FString Title;

	/** Current Url of this window. */
	FString CurrentUrl;

	/** Current tool tip. */
	FString ToolTipText;

	/** The product part for the user agent string of this window. */
	FString UserAgentProduct;

	/** Current size of this window. */
	FIntPoint ViewportSize;

	/** Current position of this window. */
	FIntPoint ViewportPos;

	/** Current DPI scale factor of this window (from the OS). */
	float ViewportDPIScaleFactor;

	/** Current scale factor of this window (from the game viewport). */
	float ViewportScaleFactor;

	/** Whether this window is closing. */
	bool bIsClosing;

	/** Whether this window has been painted at least once. */
	bool bIsInitialized;

	/** Optional text to load as a web page. */
	TOptional<FString> ContentsToLoad;

	/** Delegate for broadcasting load state changes. */
	FOnDocumentStateChanged DocumentStateChangedEvent;

	/** Whether to show an error message in case of loading errors. */
	bool bShowErrorMessage;

	/** Whether to allow forward and back navigation via the mouse thumb buttons. */
	bool bThumbMouseButtonNavigation;

	/** Whether transparency is enabled. */
	bool bUseTransparency;

	/** Whether the accelerated paint path is enabled (i.e shared texture handles) */
	bool bUsingAcceleratedPaint;

	/** Delegate for broadcasting title changes. */
	FOnTitleChanged TitleChangedEvent;

	/** Delegate for broadcasting address changes. */
	FOnUrlChanged UrlChangedEvent;

	/** Delegate for showing or hiding tool tips. */
	FOnToolTip ToolTipEvent;

	/** Delegate for notifying that the window needs refreshing. */
	FOnNeedsRedraw NeedsRedrawEvent;

	/** Delegate that is executed prior to browser navigation. */
	FOnBeforeBrowse BeforeBrowseDelegate;

	/** Delegate for overriding Url contents. */
	FOnLoadUrl LoadUrlDelegate;

	/** Delegate for handling requests to close new windows that were created. */
	FOnCloseWindow CloseWindowDelegate;

	/** Delegate for handling requests to close from the webview floating close button. */
	FOnFloatingCloseButtonPressed FloatingCloseButtonPressedDelegate;

	/** Delegate for handling resource load requests */
	FOnBeforeResourceLoadDelegate BeforeResourceLoadDelegate;

	/** Delegate that allows for responses to resource loads */
	FOnResourceLoadCompleteDelegate ResourceLoadCompleteDelegate;

	/** Delegate that allows for response to console logs.  Typically used to capture and mirror web logs in client application logs. */
	FOnConsoleMessageDelegate ConsoleMessageDelegate;

	/** Delegate for handling requests to show the popup menu. */
	FOnShowPopup ShowPopupEvent;

	/** Delegate for handling requests to dismiss the current popup menu. */
	FOnDismissPopup DismissPopupEvent;

	/** Delegate for showing dialogs. */
	FOnShowDialog ShowDialogDelegate;

	/** Delegate for dismissing all dialogs. */
	FOnDismissAllDialogs DismissAllDialogsDelegate;

	/** Delegate for suppressing context menu */
	FOnSuppressContextMenu SuppressContextMenuDelgate;

	/** Delegate that is executed when a drag event is detected in an area of the web page tagged as a drag region. */
	FOnDragWindow DragWindowDelegate;
	
	/** Delegate for handling key down events not handled by the browser. */
	FOnUnhandledKeyDown UnhandledKeyDownDelegate;

	/** Delegate for handling key up events not handled by the browser. */
	FOnUnhandledKeyUp UnhandledKeyUpDelegate;

	/** Delegate for handling key char events not handled by the browser. */
	FOnUnhandledKeyChar UnhandledKeyCharDelegate;

	/** Tracks the current mouse cursor */
	EMouseCursor::Type Cursor;

	/** Tracks whether the widget is currently disabled or not*/
	bool bIsDisabled;

	/** Tracks whether the widget is currently hidden or not*/
	bool bIsHidden;

	/** Used to detect when the widget is hidden*/
	bool bTickedLastFrame;
	
	/** Tracks whether or not the user initiated a window drag by clicking on a page's drag region. */
	bool bDraggingWindow;

	/** Used for unhandled key events forwarding*/
	TOptional<FKeyEvent> PreviousKeyDownEvent;
	TOptional<FKeyEvent> PreviousKeyUpEvent;
	TOptional<FCharacterEvent> PreviousCharacterEvent;
	bool bIgnoreKeyDownEvent;
	bool bIgnoreKeyUpEvent;
	bool bIgnoreCharacterEvent;

	/** Used to ignore any popup menus when forwarding focus gained/lost events*/
	bool bMainHasFocus;
	bool bPopupHasFocus;

	bool bSupportsMouseWheel;

	FIntRect PopupRect;
	bool bShowPopupRequested;

	/** This is set to true when reloading after render process crash. */
	bool bRecoverFromRenderProcessCrash;

	int ErrorCode;

	/** Used to defer navigations */
	bool bDeferNavigations;

	/** Used to identify a navigation that needs to fully abort before we can stop deferring navigations. */
	FString PendingAbortUrl;

	/** Used to store the url of pending navigation requests while we need to defer navigations. */
	FString PendingLoadUrl;

	TUniquePtr<FBrowserBufferedVideo> BufferedVideo;
#if PLATFORM_MAC
	void *LastPaintedSharedHandle;
#endif
	
	/** Handling of passing and marshalling messages for JS integration is delegated to a helper class*/
	TSharedPtr<FCEFJSScripting> Scripting;

#if !PLATFORM_LINUX
	/** Handling of foreign language character input is delegated to a helper class */
	TSharedPtr<FCEFImeHandler> Ime;
#endif

	TArray<FWebBrowserDragRegion> DragRegions;

	TWeakPtr<SWindow> ParentWindow;

	TUniquePtr<FCEFWebBrowserWindowRHIHelper> RHIRenderHelper;

#if PLATFORM_WINDOWS || PLATFORM_MAC
	bool bInDirectHwndMode;
#endif

	bool bHasCorrectNativeCefBuffer;
	TSharedPtr<FCEFCapturedFrame> CapturedCefFrame;

	size_t NumFramesSinceViewportResize;
};

typedef FCEFWebBrowserWindow FWebBrowserWindow;

#endif
