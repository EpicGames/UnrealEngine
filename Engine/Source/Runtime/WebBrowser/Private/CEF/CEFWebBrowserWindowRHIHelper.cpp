// Copyright Epic Games, Inc. All Rights Reserved.

#include "CEFWebBrowserWindowRHIHelper.h"
#include "WebBrowserLog.h"

#if WITH_CEF3

#include "CEF/CEFWebBrowserWindow.h"

#if WITH_ENGINE
	#include "RHI.h"

	#if PLATFORM_WINDOWS
		#include "Slate/SlateTextures.h"
		#include "RenderingThread.h"
		#include "ID3D11DynamicRHI.h"
		#include "ID3D12DynamicRHI.h"
	#endif

#endif

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#include "d3d11_1.h"
#include "d3d12.h"
#include "dxgi1_4.h"
#include "Windows/HideWindowsPlatformTypes.h"

static TComPtr<IDXGIAdapter> GetDXGIAdapterFromRHI()
{
	TComPtr<IDXGIAdapter> DXGIAdapter;

#if WITH_ENGINE
	const ERHIInterfaceType RHIType = GDynamicRHI ? GDynamicRHI->GetInterfaceType() : ERHIInterfaceType::Hidden;
	if (RHIType == ERHIInterfaceType::D3D11)
	{
		TComPtr<IDXGIDevice> DXGIDevice;
		if (HRESULT Hr = GetID3D11DynamicRHI()->RHIGetDevice()->QueryInterface(IID_PPV_ARGS(&DXGIDevice)); FAILED(Hr))
		{
			UE_LOGF(LogWebBrowser, Error, "GetDXGIAdapter() - - ID3D11Device::QueryInterface 0x%x", Hr);
			return DXGIAdapter;
		}
		if (HRESULT Hr = DXGIDevice->GetAdapter(&DXGIAdapter); FAILED(Hr))
		{
			UE_LOGF(LogWebBrowser, Error, "GetDXGIAdapter() - - IDXGIDevice::GetAdapter 0x%x", Hr);
			return DXGIAdapter;
		}
	}
	else if (RHIType == ERHIInterfaceType::D3D12)
	{
		const LUID Luid = GetID3D12DynamicRHI()->RHIGetDevice_NoMGPU()->GetAdapterLuid();

		TComPtr<IDXGIFactory4> DXGIFactory;
		if (HRESULT Hr = CreateDXGIFactory1(IID_PPV_ARGS(&DXGIFactory)); FAILED(Hr))
		{
			UE_LOGF(LogWebBrowser, Error, "GetDXGIAdapter() - - CreateDXGIFactory1 0x%x", Hr);
			return DXGIAdapter;
		}
		if (HRESULT Hr = DXGIFactory->EnumAdapterByLuid(Luid, IID_PPV_ARGS(&DXGIAdapter)); FAILED(Hr))
		{
			UE_LOGF(LogWebBrowser, Error, "GetDXGIAdapter() - - IDXGIFactory4::EnumAdapterByLuid 0x%x", Hr);
			return DXGIAdapter;
		}
	}
#endif

	return DXGIAdapter;
}
#endif // PLATFORM_WINDOWS


FCEFWebBrowserWindowRHIHelper::FCEFWebBrowserWindowRHIHelper()
{
#if WITH_ENGINE
#if PLATFORM_WINDOWS
	// We need to get access to the underlying D3D device from RHI to create a new device with the same adapter
	TComPtr<IDXGIAdapter> DXGIAdapter = GetDXGIAdapterFromRHI();

	const D3D_FEATURE_LEVEL FeatureLevels[] =
	{
		D3D_FEATURE_LEVEL_11_1,
		D3D_FEATURE_LEVEL_11_0,
	};

	if (HRESULT Hr = D3D11CreateDevice(DXGIAdapter.Get(), DXGIAdapter.Get() ? D3D_DRIVER_TYPE_UNKNOWN : D3D_DRIVER_TYPE_HARDWARE, nullptr, 0, FeatureLevels, UE_ARRAY_COUNT(FeatureLevels), D3D11_SDK_VERSION,
		&D3D11Device, nullptr, &D3D11DeviceContext); FAILED(Hr))
	{
		UE_LOGF(LogWebBrowser, Error, "FCEFWebBrowserWindowRHIHelper::FCEFWebBrowserWindowRHIHelper() - - D3D11CreateDevice 0x%x", Hr);
		return;
	}
#endif // PLATFORM_WINDOWS
#endif // WITH_ENGINE
}

bool FCEFWebBrowserWindowRHIHelper::BUseSupportedRHIRenderer()
{
#if WITH_ENGINE
	checkf(GDynamicRHI, TEXT("WebBrowserWindow is instantiated before GDynamicRHI is initialized! CEF requires RHI resources to be available for accelerated rendering, please delay the instantiation."));
	const ERHIInterfaceType RHIType = GDynamicRHI ? GDynamicRHI->GetInterfaceType() : ERHIInterfaceType::Hidden;
	// We only support D3D RHIs as we rely on the OpenSharedResource1 interface
	return (RHIType == ERHIInterfaceType::D3D11 || RHIType == ERHIInterfaceType::D3D12)
		// Disable during automation as the GPU process often fails to initialize on build machines
		&& !GIsAutomationTesting && !GIsBuildMachine;
#else
	return false;
#endif // WITH_ENGINE
}

uint64_t FCEFWebBrowserWindowRHIHelper::GetRHIAdapterLuid()
{
	uint64_t AdapterLuid = 0;

#if WITH_ENGINE
#if PLATFORM_WINDOWS
	LUID Luid = {};
	checkf(GDynamicRHI, TEXT("WebBrowserModule is instantiated before GDynamicRHI is initialized! CEF requires RHI resources to be available for accelerated rendering, please delay the instantiation."));
	const ERHIInterfaceType RHIType = GDynamicRHI ? GDynamicRHI->GetInterfaceType() : ERHIInterfaceType::Hidden;
	if (RHIType == ERHIInterfaceType::D3D11)
	{
		TComPtr<IDXGIAdapter> DXGIAdapter = GetDXGIAdapterFromRHI();
		DXGI_ADAPTER_DESC Desc = {};
		if (DXGIAdapter.IsValid() && SUCCEEDED(DXGIAdapter->GetDesc(&Desc)))
		{
			Luid = Desc.AdapterLuid;
		}
	}
	else if (RHIType == ERHIInterfaceType::D3D12)
	{
		Luid = GetID3D12DynamicRHI()->RHIGetDevice_NoMGPU()->GetAdapterLuid();
	}

	AdapterLuid = (static_cast<uint64_t>(Luid.HighPart) << 32) | static_cast<uint64_t>(Luid.LowPart);
#endif // PLATFORM_WINDOWS
#endif // WITH_ENGINE

	return AdapterLuid;
}

FSlateUpdatableTexture* FCEFWebBrowserWindowRHIHelper::CreateSlateUpdatableTexture(const FIntPoint& TextureSize)
{
#if WITH_ENGINE
#if PLATFORM_WINDOWS
	// On Windows we need the slate texture to be shared so we can access it from the D3D device that lives in the OnAcceleratedPaint thread
	FSlateTexture2DRHIRef* SlateTexture = new FSlateTexture2DRHIRef(TextureSize.X, TextureSize.Y, PF_B8G8R8A8, nullptr, ETextureCreateFlags::Shared | ETextureCreateFlags::RenderTargetable, true);
	BeginInitResource(SlateTexture);
	return SlateTexture;
#else
	return nullptr;
#endif // PLATFORM_WINDOWS
#else
	return nullptr;
#endif // WITH_ENGINE
}

void FCEFWebBrowserWindowRHIHelper::ReleaseSlateUpdatableTexture(FSlateUpdatableTexture* SlateTexture)
{
#if WITH_ENGINE
#if PLATFORM_WINDOWS
	if (!SlateTexture)
	{
		return;
	}

	SlateTextureHandles.Remove(SlateTexture);
	SlateTexture->Cleanup(); // calls BeginReleaseResource and handles deletion at the end
#endif // PLATFORM_WINDOWS
#endif // WITH_ENGINE
}

FSlateUpdatableTexture* FCEFWebBrowserWindowRHIHelper::ResizeSlateUpdatableTexture(FSlateUpdatableTexture* SlateTexture, const FIntPoint& TextureSize)
{
#if WITH_ENGINE
#if PLATFORM_WINDOWS
	if (!SlateTexture)
	{
		UE_LOGF(LogWebBrowser, Error, "FCEFWebBrowserWindowRHIHelper::ResizeSlateUpdatableTexture() - invalid SlateTexture!");
		return nullptr;
	}

	if (TextureSize.X <= 0 || TextureSize.Y <= 0)
	{
		UE_LOGF(LogWebBrowser, Error, "FCEFWebBrowserWindowRHIHelper::ResizeSlateUpdatableTexture() - invalid SlateTexture!");
		return nullptr;
	}

	if (!EnsureShareable(SlateTexture))
	{
		UE_LOGF(LogWebBrowser, Error, "FCEFWebBrowserWindowRHIHelper::ResizeSlateUpdatableTexture() - SlateTexture not shareable!");
		ReleaseSlateUpdatableTexture(SlateTexture);
		return nullptr;
	}

	if (SlateTexture->GetSlateResource()->GetWidth() == TextureSize.X && SlateTexture->GetSlateResource()->GetHeight() == TextureSize.Y)
	{
		return SlateTexture;
	}

	FSlateUpdatableTexture* NewSlateTexture = CreateSlateUpdatableTexture(TextureSize);
	// Needed so we can copy the existing contents in the new texture now
	FlushRenderingCommands();

	FIntRect DirtyRect = {
		0,
		0,
		std::min(static_cast<int>(SlateTexture->GetSlateResource()->GetWidth()), TextureSize.X),
		std::min(static_cast<int>(SlateTexture->GetSlateResource()->GetHeight()), TextureSize.Y)
	};
	
	if (!CopySharedTextureSync(NewSlateTexture, (HANDLE)SlateTextureHandles.FindRef(SlateTexture).Get(), DirtyRect))
	{
		UE_LOGF(LogWebBrowser, Error, "FCEFWebBrowserWindowRHIHelper::ResizeSlateUpdatableTexture() - Failed to copy SlateTexture into NewSlateTexture!");
	}

	ReleaseSlateUpdatableTexture(SlateTexture);

	return NewSlateTexture;
#else
	return nullptr;
#endif // PLATFORM_WINDOWS
#else
	return nullptr;
#endif // WITH_ENGINE
}

bool FCEFWebBrowserWindowRHIHelper::EnsureShareable(FSlateUpdatableTexture* SlateTexture)
{
#if WITH_ENGINE
	if (!SlateTexture)
	{
		UE_LOGF(LogWebBrowser, Error, "FCEFWebBrowserWindowRHIHelper::EnsureShareable() - Invalid SlateTexture");
		return false;
	}

	if (TSharedPtr<void>* HandlePtr = SlateTextureHandles.Find(SlateTexture); HandlePtr)
	{
		// Null Handle means that extraction/initialization failed
		return HandlePtr->Get() != nullptr;
	}

#if PLATFORM_WINDOWS
	FTextureRHIRef Slate2DRef = static_cast<FSlateTexture2DRHIRef*>(SlateTexture)->GetRHIRef();
	if (!Slate2DRef.IsValid() || !Slate2DRef->GetNativeResource())
	{
		// If this happens frequently then we need to make the resource creation sync in CreateSlateUpdatableTexture
		UE_LOGF(LogWebBrowser, Error, "FCEFWebBrowserWindowRHIHelper::EnsureShareable() - SlateTexture is not ready!");
		return false;
	}

	HANDLE SharedHandle = nullptr;

	// We add SharedHandle to the map even if we couldn't extract it, so we don't continuously try and report failures
	const ERHIInterfaceType RHIType = GDynamicRHI ? GDynamicRHI->GetInterfaceType() : ERHIInterfaceType::Hidden;
	if (RHIType == ERHIInterfaceType::D3D11)
	{
		TComPtr<IDXGIResource> DXGIResource;
		if (HRESULT Hr = static_cast<ID3D11Texture2D*>(Slate2DRef->GetNativeResource())->QueryInterface(IID_PPV_ARGS(&DXGIResource)); FAILED(Hr))
		{
			UE_LOGF(LogWebBrowser, Error, "FCEFWebBrowserWindowRHIHelper::EnsureShareable() - - ID3D11Texture2D::QueryInterface 0x%x", Hr);
		}
		else if (Hr = DXGIResource->GetSharedHandle(&SharedHandle); FAILED(Hr))
		{
			UE_LOGF(LogWebBrowser, Error, "FCEFWebBrowserWindowRHIHelper::EnsureShareable() - - IDXGIResource::GetSharedHandle 0x%x", Hr);
		}

		// GetSharedHandle returns a legacy non-NT handle, which must NOT be closed when we're done with it
		SlateTextureHandles.Add(SlateTexture, MakeShareable(SharedHandle, [](HANDLE _LegacySharedHandle) {}));
	}
	else if (RHIType == ERHIInterfaceType::D3D12)
	{
		if (HRESULT Hr = GetID3D12DynamicRHI()->RHIGetDevice_NoMGPU()->CreateSharedHandle(static_cast<ID3D12Resource*>(Slate2DRef->GetNativeResource()), NULL, GENERIC_ALL, NULL, &SharedHandle); FAILED(Hr))
		{
			UE_LOGF(LogWebBrowser, Error, "FCEFWebBrowserWindowRHIHelper::EnsureShareable() - - ID3D12Device::CreateSharedHandle 0x%x", Hr);
		}

		// CreateSharedHandle returns a NT handle,. which must be closed when we're done with it
		SlateTextureHandles.Add(SlateTexture, MakeShareable(SharedHandle, [](HANDLE _NTSharedHandle)
			{
				if (_NTSharedHandle)
				{
					CloseHandle(_NTSharedHandle);
				}
			}));
	}

	return SharedHandle != nullptr;
#else
	return false;
#endif // PLATFORM_WINDOWS
#else
	return false;
#endif // WITH_ENGINE

}

bool FCEFWebBrowserWindowRHIHelper::CopySharedTextureSync(FSlateUpdatableTexture* SlateTexture, void* SharedHandle, const FIntRect& DirtyIn)
{
#if WITH_ENGINE
	if (!EnsureShareable(SlateTexture))
	{
		return false;
	}

#if PLATFORM_WINDOWS
	if (!D3D11Device.IsValid() || !D3D11DeviceContext.IsValid())
	{
		return false;
	}

	TComPtr<ID3D11Device1> D3D11Device1;
	if (HRESULT Hr = D3D11Device->QueryInterface(IID_PPV_ARGS(&D3D11Device1)); FAILED(Hr))
	{
		UE_LOGF(LogWebBrowser, Error, "FCEFWebBrowserWindowRHIHelper::CopySharedTextureSync() - - ID3D11Device::QueryInterface 0x%x", Hr);
		return false;
	}

	TComPtr<ID3D11Texture2D> SourceTexture;
	if (HRESULT Hr = D3D11Device1->OpenSharedResource1((HANDLE)SharedHandle, IID_PPV_ARGS(&SourceTexture)); FAILED(Hr))
	{
		// Try to open as a D3D11 shared handle instead
		if (HRESULT Hr2 = D3D11Device1->OpenSharedResource((HANDLE)SharedHandle, IID_PPV_ARGS(&SourceTexture)); FAILED(Hr2))
		{
			UE_LOGF(LogWebBrowser, Error, "FCEFWebBrowserWindowRHIHelper::CopySharedTextureSync() - - ID3D11Device1::OpenSharedResource[1] 0x%x [0x%x]", Hr2, Hr);
			return false;
		}
	}

	TComPtr<ID3D11Texture2D> DestTexture;
	const ERHIInterfaceType RHIType = GDynamicRHI ? GDynamicRHI->GetInterfaceType() : ERHIInterfaceType::Hidden;
	if (RHIType == ERHIInterfaceType::D3D11)
	{
		if (HRESULT Hr = D3D11Device->OpenSharedResource((HANDLE)SlateTextureHandles.FindRef(SlateTexture).Get(), IID_PPV_ARGS(&DestTexture)); FAILED(Hr))
		{
			UE_LOGF(LogWebBrowser, Error, "FCEFWebBrowserWindowRHIHelper::CopySharedTextureSync() - - ID3D11Device::OpenSharedResource 0x%x", Hr);
			return false;
		}
	}
	else if (RHIType == ERHIInterfaceType::D3D12)
	{
		if (HRESULT Hr = D3D11Device1->OpenSharedResource1((HANDLE)SlateTextureHandles.FindRef(SlateTexture).Get(), IID_PPV_ARGS(&DestTexture)); FAILED(Hr))
		{
			UE_LOGF(LogWebBrowser, Error, "FCEFWebBrowserWindowRHIHelper::CopySharedTextureSync() - - ID3D11Device1::OpenSharedResource1 0x%x", Hr);
			return false;
		}
	}

	// Copy the texture's dirty subregion on the GPU, limited to DestTexture's size
	D3D11_TEXTURE2D_DESC SourceTextureDesc = {};
	SourceTexture->GetDesc(&SourceTextureDesc);
	D3D11_TEXTURE2D_DESC DestTextureDesc = {};
	DestTexture->GetDesc(&DestTextureDesc);

	D3D11_BOX SourceDirtyBox = {};
	SourceDirtyBox.left = std::max(0, DirtyIn.Min.X);
	SourceDirtyBox.right = std::min({ static_cast<int>(SourceTextureDesc.Width), static_cast<int>(DestTextureDesc.Width), std::max(0, DirtyIn.Max.X) });
	SourceDirtyBox.top = std::max(0, DirtyIn.Min.Y);
	SourceDirtyBox.bottom = std::min({ static_cast<int>(SourceTextureDesc.Height), static_cast<int>(DestTextureDesc.Height), std::max(0, DirtyIn.Max.Y) });
	SourceDirtyBox.front = 0;
	SourceDirtyBox.back = 1;

	if (SourceDirtyBox.right <= SourceDirtyBox.left || SourceDirtyBox.bottom <= SourceDirtyBox.top)
	{
		return false;
	}

	D3D11DeviceContext->CopySubresourceRegion(
		DestTexture.Get(), 0,				// pDstResource, DstSubresource
		SourceDirtyBox.left, SourceDirtyBox.top, 0,	// DstX, DstY, DstZ
		SourceTexture.Get(), 0,				// pSrcResource, SrcSubresource
		&SourceDirtyBox						// pSrcBox
	);

	// The CopySubresourceRegion call is asynchronous by default, so we need to force a sync flush to ensure it completes before we exit this method
	D3D11DeviceContext->Flush();

	// Now wait on the GPU to complete the copy and flush
	D3D11_QUERY_DESC QueryDesc = {};
	QueryDesc.Query = D3D11_QUERY_EVENT;
	TComPtr<ID3D11Query> Query;
	if (HRESULT Hr = D3D11Device1->CreateQuery(&QueryDesc, &Query); FAILED(Hr))
	{
		UE_LOGF(LogWebBrowser, Error, "FCEFWebBrowserWindowRHIHelper::CopySharedTextureSync() - ID3D11Device1::CreateQuery 0x%x", Hr);
		return false;
	}
	D3D11DeviceContext->End(Query.Get());
	BOOL bIsDone = false;
	HRESULT Hr = S_OK;
	for (;;)
	{
		Hr = D3D11DeviceContext->GetData(Query.Get(), &bIsDone, sizeof(bIsDone), 0);
		if (FAILED(Hr))
		{
			UE_LOGF(LogWebBrowser, Error, "FCEFWebBrowserWindowRHIHelper::CopySharedTextureSync() - ID3D11DeviceContext::GetData  0x%x", Hr);
			return false;
		}

		// We need to check for S_OK specifically as S_FALSE is also considered a success return code
		if (Hr == S_OK && bIsDone)
		{
			break;
		}

		// Idle wait before retrying
		if (!SwitchToThread())
		{
			Sleep(0);
		}
	}

	return true;
#else
	UE_LOGF(LogWebBrowser, Error, "FCEFWebBrowserWindowRHIHelper::CopySharedTextureSync() - missing implementation");
	return false;
#endif // PLATFORM_WINDOWS
#else
	UE_LOGF(LogWebBrowser, Error, "FCEFWebBrowserWindowRHIHelper::CopySharedTextureSync() - unsupported usage, RHI renderer but missing engine");
	return false;
#endif // WITH_ENGINE
}

#endif
