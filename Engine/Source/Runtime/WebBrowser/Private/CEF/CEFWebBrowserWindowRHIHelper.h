// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_CEF3

#include "Layout/Geometry.h"

class FSlateUpdatableTexture;

#if PLATFORM_WINDOWS
#include "Windows/WindowsHWrapper.h"
#include "Microsoft/COMPointer.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include "d3d11.h"
#include "Windows/HideWindowsPlatformTypes.h"
#endif // PLATFORM_WINDOWS

/**
 * Implementation of RHI renderer details for the CEF accelerated rendering path
 */
class FCEFWebBrowserWindowRHIHelper
{
public:
	FCEFWebBrowserWindowRHIHelper();
	virtual ~FCEFWebBrowserWindowRHIHelper() = default;

	static bool BUseSupportedRHIRenderer();
	static uint64_t GetRHIAdapterLuid();

public:
	FSlateUpdatableTexture* CreateSlateUpdatableTexture(const FIntPoint& TextureSize);
	void ReleaseSlateUpdatableTexture(FSlateUpdatableTexture* SlateTexture);
	FSlateUpdatableTexture* ResizeSlateUpdatableTexture(FSlateUpdatableTexture* SlateTexture, const FIntPoint& TextureSize);

	bool CopySharedTextureSync(FSlateUpdatableTexture* SlateTexture, void* SharedHandle, const FIntRect& DirtyIn);

private:
	bool EnsureShareable(FSlateUpdatableTexture* SlateTexture);

	TMap<FSlateUpdatableTexture*, TSharedPtr<void>> SlateTextureHandles;

#if PLATFORM_WINDOWS
	TComPtr<ID3D11Device> D3D11Device;
	TComPtr<ID3D11DeviceContext> D3D11DeviceContext;
#endif // PLATFORM_WINDOWS
};

#endif
