// Copyright Epic Games, Inc. All Rights Reserved.

#include "MobileJSScripting.h"

#if PLATFORM_ANDROID  || PLATFORM_IOS  || PLATFORM_MAC

#include "IWebBrowserWindow.h"
#include "MobileJSStructSerializerBackend.h"
#include "MobileJSStructDeserializerBackend.h"
#include "StructSerializer.h"
#include "StructDeserializer.h"
#include "UObject/UnrealType.h"
#include "Async/Async.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "JsonObjectConverter.h"

// For UrlDecode/Encode
#include "Http.h"

DEFINE_LOG_CATEGORY(LogMobileJSScripting);

// Inseterted as a part of an URL to send a message to the front end.
// Note, we can't use a custom protocol due to cross-domain issues.
const FString FMobileJSScripting::JSMessageTag = TEXT("/!!com.epicgames.unreal.message/");
const FString FMobileJSScripting::JSMessageHandler = TEXT("com_epicgames_unreal_message");


namespace
{
	const FString ExecuteMethodCommand = "ExecuteUObjectMethod";
	const FString ScriptingInit =
		TEXT("(function() {")
			TEXT("var util = Object.create({")

			// Simple random-based (RFC-4122 version 4) UUID generator.
			// Version 4 UUIDs have the form xxxxxxxx-xxxx-4xxx-yxxx-xxxxxxxxxxxx where x is any hexadecimal digit and y is one of 8, 9, a, or b
			// This function returns the UUID as a hex string without the dashes
			TEXT("uuid: function()")
			TEXT("{")
			TEXT("	var b = new Uint8Array(16); window.crypto.getRandomValues(b);")
			TEXT("	b[6] = b[6]&0xf|0x40; b[8]=b[8]&0x3f|0x80;") // Set the reserved bits to the correct values
			TEXT("	return Array.prototype.reduce.call(b, function(a,i){return a+((0x100|i).toString(16).substring(1))},'').toUpperCase();")
			TEXT("}, ")

			// save a callback function in the callback registry
			// returns the uuid of the callback for passing to the host application
			// ensures that each function object is only stored once.
			// (Closures executed multiple times are considered separate objects.)
			TEXT("registerCallback: function(callback)")
			TEXT("{")
			TEXT("	var key;")
			TEXT("	for(key in this.callbacks)")
			TEXT("	{")
			TEXT("		if (!this.callbacks[key].isOneShot && this.callbacks[key].accept === callback)")
			TEXT("		{")
			TEXT("			return key;")
			TEXT("		}")
			TEXT("	}")
			TEXT("	key = this.uuid();")
			TEXT("	this.callbacks[key] = {accept:callback, reject:callback, bIsOneShot:false};")
			TEXT("	return key;")
			TEXT("}, ")

			TEXT("registerPromise: function(accept, reject, name)")
			TEXT("{")
			TEXT("	var key = this.uuid();")
			TEXT("	this.callbacks[key] = {accept:accept, reject:reject, bIsOneShot:true, name:name};")
			TEXT("	return key;")
			TEXT("}, ")

			// invoke a callback method or promise by uuid
			TEXT("invokeCallback: function(key, bIsError, args)")
			TEXT("{")
			TEXT("	var callback = this.callbacks[key];")
			TEXT("	if (typeof callback === 'undefined')")
			TEXT("	{")
			TEXT("		console.error('Unknown callback id', key);")
			TEXT("		return;")
			TEXT("	}")
			TEXT("	if (callback.bIsOneShot)")
			TEXT("	{")
			TEXT("		callback.iwanttodeletethis=true;")
			TEXT("		delete this.callbacks[key];")
			TEXT("	}")
			TEXT("	callback[bIsError?'reject':'accept'].apply(window, args);")
			TEXT("}, ")

			// convert an argument list to a dictionary of arguments.
			// The args argument must be an argument object as it uses the callee member to deduce the argument names
			TEXT("argsToDict: function(args)")
			TEXT("{")
			TEXT("	var res = {};")
			TEXT("	args.callee.toString().match(/\\((.+?)\\)/)[1].split(/\\s*,\\s*/).forEach(function(name, idx){res[name]=args[idx]});")
			TEXT("	return res;")
			TEXT("}, ")

			// encodes and sends a message to the host application
			TEXT("sendMessage: function()")
			TEXT("{")
#if PLATFORM_IOS || PLATFORM_MAC
			TEXT("	window.webkit.messageHandlers.") + FMobileJSScripting::JSMessageHandler + TEXT(".postMessage(Array.prototype.map.call(arguments,function(e){return encodeURIComponent(e)}).join('/'));")
#else
			TEXT("	var req=new XMLHttpRequest();")
			TEXT("	req.open('GET', '") + FMobileJSScripting::JSMessageTag + TEXT("' + Array.prototype.map.call(arguments,function(e){return encodeURIComponent(e)}).join('/'), true);")
			TEXT("	req.send(null);")
#endif
			TEXT("}, ")

			// uses the above helper methods to execute a method on a uobject instance.
			// the method set as callee on args needs to be a named function, as the name of the method to invoke is taken from it
			TEXT("executeMethod: function(id, args)")
			TEXT("{")
			TEXT("	var self = this;") // the closures need access to the outer this object

			// In case there are function objects in the argument list, temporarily override Function.toJSON to be able to pass them as callbacks
			TEXT("	var functionJSON = Function.prototype.toJSON;")
			TEXT("	Function.prototype.toJSON = function(){ return self.registerCallback(this) };")

			// Create a promise object to return back to the caller and create a callback function to handle the response
			TEXT("	var promiseID;")
			TEXT("	var promise = new Promise(function (accept, reject) ")
			TEXT("	{")
			TEXT("		promiseID = self.registerPromise(accept, reject, args.callee.name)")
			TEXT("	});")

			// Actually invoke the method by sending a message to the host app
			TEXT("	this.sendMessage('") + ExecuteMethodCommand + TEXT("', id, promiseID, args.callee.name, JSON.stringify(this.argsToDict(args)));")

			// Restore Function.toJSON back to its old value (usually undefined) and return the promise object to the caller
			TEXT("	Function.prototype.toJSON = functionJSON;")
			TEXT("	return promise;")
			TEXT("}")
			TEXT("},{callbacks: {value:{}}});")

			// Create the global window.ue variable
			TEXT("window.ue = Object.create({}, {'$': {writable: false, configurable:false, enumerable: false, value:util}});")
		TEXT("})();")
	;
	const FString ScriptingPostInit =
		TEXT("(function() {")
		TEXT("	document.dispatchEvent(new CustomEvent('ue:ready', {details: window.ue}));")
		TEXT("})();")
	;


	typedef TSharedRef<TJsonWriter<>> FJsonWriterRef;

	template<typename ValueType> void WriteValue(FJsonWriterRef Writer, const FString& Key, const ValueType& Value)
	{
		Writer->WriteValue(Key, Value);
	}
	void WriteNull(FJsonWriterRef Writer, const FString& Key)
	{
		Writer->WriteNull(Key);
	}
	void WriteArrayStart(FJsonWriterRef Writer, const FString& Key)
	{
		Writer->WriteArrayStart(Key);
	}
	void WriteObjectStart(FJsonWriterRef Writer, const FString& Key)
	{
		Writer->WriteObjectStart(Key);
	}
	void WriteRaw(FJsonWriterRef Writer, const FString& Key, const FString& Value)
	{
		Writer->WriteRawJSONValue(Key, Value);
	}
	template<typename ValueType> void WriteValue(FJsonWriterRef Writer, const int, const ValueType& Value)
	{
		Writer->WriteValue(Value);
	}
	void WriteNull(FJsonWriterRef Writer, int)
	{
		Writer->WriteNull();
	}
	void WriteArrayStart(FJsonWriterRef Writer, int)
	{
		Writer->WriteArrayStart();
	}
	void WriteObjectStart(FJsonWriterRef Writer, int)
	{
		Writer->WriteObjectStart();
	}
	void WriteRaw(FJsonWriterRef Writer, int, const FString& Value)
	{
		Writer->WriteRawJSONValue(Value);
	}

	template<typename KeyType>
	bool WriteJsParam(FMobileJSScriptingRef Scripting, FJsonWriterRef Writer, const KeyType& Key, FWebJSParam& Param)
	{
		switch (Param.Tag)
		{
			case FWebJSParam::PTYPE_NULL:
				WriteNull(Writer, Key);
				break;
			case FWebJSParam::PTYPE_BOOL:
				WriteValue(Writer, Key, Param.BoolValue);
				break;
			case FWebJSParam::PTYPE_DOUBLE:
				WriteValue(Writer, Key, Param.DoubleValue);
				break;
			case FWebJSParam::PTYPE_INT:
				WriteValue(Writer, Key, Param.IntValue);
				break;
			case FWebJSParam::PTYPE_STRING:
				WriteValue(Writer, Key, *Param.StringValue);
				break;
			case FWebJSParam::PTYPE_OBJECT:
			{
				if (Param.ObjectValue == nullptr)
				{
					WriteNull(Writer, Key);
				}
				else
				{
					FString ConvertedObject = Scripting->ConvertObject(Param.ObjectValue);
					WriteRaw(Writer, Key, ConvertedObject);
				}
				break;
			}
			case FWebJSParam::PTYPE_STRUCT:
			{
				FString ConvertedStruct = Scripting->ConvertStruct(Param.StructValue->GetTypeInfo(), Param.StructValue->GetData());
				WriteRaw(Writer, Key, ConvertedStruct);
				break;
			}
			case FWebJSParam::PTYPE_ARRAY:
			{
				WriteArrayStart(Writer, Key);
				for(int i=0; i < Param.ArrayValue->Num(); ++i)
				{
					WriteJsParam(Scripting, Writer, i, (*Param.ArrayValue)[i]);
				}
				Writer->WriteArrayEnd();
				break;
			}
			case FWebJSParam::PTYPE_MAP:
			{
				WriteObjectStart(Writer, Key);
				for(auto& Pair : *Param.MapValue)
				{
					WriteJsParam(Scripting, Writer, *Pair.Key, Pair.Value);
				}
				Writer->WriteObjectEnd();
				break;
			}
			default:
				return false;
		}
		return true;
	}
}

void FMobileJSScripting::AddPermanentBind(const FString& Name, UObject* Object)
{
	const FString ExposedName = GetBindingName(Name, Object);

	// Each object can only have one permanent binding
	if (BoundObjects[Object].bIsPermanent)
	{
		return;
	}
	// Existing permanent objects must be removed first
	if (PermanentUObjectsByName.Contains(ExposedName))
	{
		return;
	}
	BoundObjects[Object] = { true, -1 };
	PermanentUObjectsByName.Add(ExposedName, Object);
}

void FMobileJSScripting::RemovePermanentBind(const FString& Name, UObject* Object)
{
	const FString ExposedName = GetBindingName(Name, Object);

	// If overriding an existing permanent object, make it non-permanent
	if (PermanentUObjectsByName.Contains(ExposedName) && (Object == nullptr || PermanentUObjectsByName[ExposedName] == Object))
	{
		Object = PermanentUObjectsByName.FindAndRemoveChecked(ExposedName);
		BoundObjects.Remove(Object);
		return;
	}
	else
	{
		return;
	}
}

void FMobileJSScripting::BindUObject(const FString& Name, UObject* Object, bool bIsPermanent)
{
	TSharedPtr<IWebBrowserWindow> Window = WindowPtr.Pin();
	if (Window.IsValid())
	{
		BindUObject(Window.ToSharedRef(), Name, Object, bIsPermanent);
	}
	else if (bIsPermanent)
	{
		AddPermanentBind(Name, Object);
	}
}

void FMobileJSScripting::UnbindUObject(const FString& Name, UObject* Object, bool bIsPermanent)
{
	TSharedPtr<IWebBrowserWindow> Window = WindowPtr.Pin();
	if (Window.IsValid())
	{
		UnbindUObject(Window.ToSharedRef(), Name, Object, bIsPermanent);
	}
	else if (bIsPermanent)
	{
		RemovePermanentBind(Name, Object);
	}
}

void FMobileJSScripting::BindUObject(const TSharedRef<class IWebBrowserWindow>& InWindow, const FString& Name, UObject* Object, bool bIsPermanent)
{
	WindowPtr = InWindow;

	const FString ExposedName = GetBindingName(Name, Object);
	FString Converted = ConvertObject(Object);
	if (bIsPermanent)
	{
		AddPermanentBind(Name, Object);
	}

	InitializeScript(InWindow);
	FString SetValueScript = FString::Printf(TEXT("window.ue['%s'] = %s;"), *ExposedName.ReplaceCharWithEscapedChar(), *Converted);
	SetValueScript.Append(ScriptingPostInit);
	
	InWindow->ExecuteJavascript(SetValueScript);
	InWindow->ExecuteJavascript(ScriptingPostInit);
}

void FMobileJSScripting::UnbindUObject(const TSharedRef<class IWebBrowserWindow>& InWindow, const FString& Name, UObject* Object, bool bIsPermanent)
{
	WindowPtr = InWindow;

	const FString ExposedName = GetBindingName(Name, Object);
	if (bIsPermanent)
	{
		RemovePermanentBind(Name, Object);
	}

	FString DeleteValueScript = FString::Printf(TEXT("delete window.ue['%s'];"), *ExposedName.ReplaceCharWithEscapedChar());
	InWindow->ExecuteJavascript(DeleteValueScript);
}

bool FMobileJSScripting::OnJsMessageReceived(const FString& Command, const TArray<FString>& Params, const FString& Origin)
{
	bool Result = false;
	if (Command == ExecuteMethodCommand)
	{
		Result = HandleExecuteUObjectMethodMessage(Params);
	} 
	else 
	{
		UE_LOGF(LogMobileJSScripting, Error, "Unknown JS message received: %ls", *Command);
	}

	return Result;
}

FString FMobileJSScripting::ConvertStruct(UStruct* TypeInfo, const void* StructPtr)
{
	TSharedRef<FJsonObject> OutJson(new FJsonObject());
	if (FJsonObjectConverter::UStructToJsonObject(TypeInfo, StructPtr, OutJson))
	{
		FString StringToFill;
		auto Writer = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&StringToFill);
		if (FJsonSerializer::Serialize(OutJson, Writer))
		{
			return StringToFill;
		}
	}
	return TEXT("undefined");
}

FString FMobileJSScripting::ConvertObject(UObject* Object)
{
	RetainBinding(Object);
	UClass* Class = Object->GetClass();

	FString ObjectGuidString = *PtrToGuid(Object).ToString(EGuidFormats::Digits);
	bool first = true;
	FString Result = "(function(){ return Object.create({";
	for (TFieldIterator<UFunction> FunctionIt(Class, EFieldIteratorFlags::IncludeSuper); FunctionIt; ++FunctionIt)
	{
		UFunction* Function = *FunctionIt;
		if(!first)
		{
			Result.Append(TEXT(","));
		}
		else
		{
			first = false;
		}
		Result.Append(*GetBindingName(Function));
		Result.Append(TEXT(": function "));
		Result.Append(*GetBindingName(Function));
		Result.Append(TEXT(" ("));

		bool firstArg = true;
		for ( TFieldIterator<FProperty> It(Function); It; ++It )
		{
			FProperty* Param = *It;
			if (Param->PropertyFlags & CPF_Parm && ! (Param->PropertyFlags & CPF_ReturnParm) )
			{
				FStructProperty *StructProperty = CastField<FStructProperty>(Param);
				if (!StructProperty || !StructProperty->Struct->IsChildOf(FWebJSResponse::StaticStruct()))
				{
					if(!firstArg)
					{
						Result.Append(TEXT(", "));
					}
					else
					{
						firstArg = false;
					}
					Result.Append(*GetBindingName(Param));
				}
			}
		}

		Result.Append(TEXT(")"));
		Result.Append(TEXT(" {return window.ue.$.executeMethod(\""));
		Result.Append(ObjectGuidString);
		Result.Append(TEXT("\", arguments)}"));
	}
	Result.Append(TEXT("},{"));
	Result.Append(TEXT("$id: {writable: false, configurable:false, enumerable: false, value: '"));
	Result.Append(ObjectGuidString);
	Result.Append(TEXT("'}})})()"));
	return Result;
}

void FMobileJSScripting::InvokeJSFunction(FGuid FunctionId, int32 ArgCount, FWebJSParam Arguments[], bool bIsError)
{
	TSharedPtr<IWebBrowserWindow> Window = WindowPtr.Pin();
	if (Window.IsValid())
	{
		FString CallbackScript = FString::Printf(TEXT("window.ue.$.invokeCallback('%s', %s, "), *FunctionId.ToString(EGuidFormats::Digits), (bIsError)?TEXT("true"):TEXT("false"));
		{
			TArray<uint8> Buffer;
			FMemoryWriter MemoryWriter(Buffer);
			FJsonWriterRef JsonWriter = TJsonWriter<>::Create(&MemoryWriter);
			JsonWriter->WriteArrayStart();
			for(int i=0; i<ArgCount; i++)
			{
				WriteJsParam(SharedThis(this), JsonWriter, i, Arguments[i]);
			}
			JsonWriter->WriteArrayEnd();
			CallbackScript.Append((TCHAR*)Buffer.GetData(), Buffer.Num()/sizeof(TCHAR));
		}
		CallbackScript.Append(TEXT(")"));
		Window->ExecuteJavascript(CallbackScript);
	}
}

void FMobileJSScripting::InvokeJSFunctionRaw(FGuid FunctionId, const FString& RawJSValue, bool bIsError)
{
	TSharedPtr<IWebBrowserWindow> Window = WindowPtr.Pin();
	if (Window.IsValid())
	{
		FString CallbackScript = FString::Printf(TEXT("window.ue.$.invokeCallback('%s', %s, [%s])"),
			*FunctionId.ToString(EGuidFormats::Digits), (bIsError)?TEXT("true"):TEXT("false"), *RawJSValue);
		Window->ExecuteJavascript(CallbackScript);
	}
}

void FMobileJSScripting::InvokeJSErrorResult(FGuid FunctionId, const FString& Error)
{
	FWebJSParam Args[1] = {FWebJSParam(Error)};
	UE_LOGF(LogMobileJSScripting, Error, "JS Error %ls", *Error);
	InvokeJSFunction(FunctionId, 1, Args, true);
}

bool FMobileJSScripting::HandleExecuteUObjectMethodMessage(const TArray<FString>& MessageArgs)
{
	if (MessageArgs.Num() != 4)
	{
		UE_LOGF(LogMobileJSScripting, Error, "insufficent num of args expected 4 got %d", MessageArgs.Num());
		return false;
	}

	FGuid ObjectKey;
	if (!FGuid::Parse(MessageArgs[0], ObjectKey))
	{
		// Invalid GUID
		UE_LOGF(LogMobileJSScripting, Error, "JS GUID %ls was invalid", *MessageArgs[0]);
		return false;
	}
	// Get the promise callback and use that to report any results from executing this function.
	FGuid ResultCallbackId;
	if (!FGuid::Parse(MessageArgs[1], ResultCallbackId))
	{
		// Invalid GUID
		UE_LOGF(LogMobileJSScripting, Error, "JS GUID %ls was invalid", *MessageArgs[1]);
		return false;
	}

	UObject* Object = GuidToPtr(ObjectKey);
	if (Object == nullptr)
	{
		// Unknown uobject id
		InvokeJSErrorResult(ResultCallbackId, TEXT("Unknown UObject ID"));
		return true;
	}

	FName MethodName = FName(*MessageArgs[2]);
	UFunction* Function = Object->FindFunction(MethodName);
	if (!Function)
	{
		InvokeJSErrorResult(ResultCallbackId, TEXT("Unknown UObject Function"));
		return true;
	}

	// Coerce arguments to function arguments.
	uint16 ParamsSize = Function->ParmsSize;
	TArray<uint8> Params;
	FProperty* ReturnParam = nullptr;
	FProperty* PromiseParam = nullptr;

	if (ParamsSize > 0)
	{
		// Find return parameter and a promise argument if present, as we need to handle them differently
		for ( TFieldIterator<FProperty> It(Function); It; ++It )
		{
			FProperty* Param = *It;
			if (Param->PropertyFlags & CPF_Parm)
			{
				if (Param->PropertyFlags & CPF_ReturnParm)
				{
					ReturnParam = Param;
				}
				else
				{
					FStructProperty *StructProperty = CastField<FStructProperty>(Param);
					if (StructProperty && StructProperty->Struct->IsChildOf(FWebJSResponse::StaticStruct()))
					{
						PromiseParam = Param;
					}
				}
				if (ReturnParam && PromiseParam)
				{
					break;
				}
			}
		}

		// UFunction is a subclass of UStruct, so we can treat the arguments as a struct for deserialization
		Params.AddUninitialized(Function->GetStructureSize());
		Function->InitializeStruct(Params.GetData());

		FMobileJSStructDeserializerBackend Backend = FMobileJSStructDeserializerBackend(SharedThis(this), MessageArgs[3]);
		FStructDeserializer::Deserialize(Params.GetData(), *Function, Backend);
	}

	if (PromiseParam)
	{
		FWebJSResponse* PromisePtr = PromiseParam->ContainerPtrToValuePtr<FWebJSResponse>(Params.GetData());
		if (PromisePtr)
		{
			*PromisePtr = FWebJSResponse(SharedThis(this), ResultCallbackId);
		}
	}

	Object->ProcessEvent(Function, Params.GetData());
	if ( ! PromiseParam ) // If PromiseParam is set, we assume that the UFunction will ensure it is called with the result
	{
		if ( ReturnParam )
		{
			FStructSerializerPolicies ReturnPolicies;
			ReturnPolicies.PropertyFilter = [&](const FProperty* CandidateProperty, const FProperty* ParentProperty)
			{
				return ParentProperty != nullptr || CandidateProperty == ReturnParam;
			};
			FMobileJSStructSerializerBackend ReturnBackend = FMobileJSStructSerializerBackend(SharedThis(this));
			FStructSerializer::Serialize(Params.GetData(), *Function, ReturnBackend, ReturnPolicies);

			// Extract the result value from the serialized JSON object:
			FString ResultJS = ReturnBackend.ToString();
			
			if(!bDefaultJSReturnInDict) 
			{
				FString ParsedJS;
				TSharedPtr<FJsonObject> JsonObj = MakeShareable(new FJsonObject);
							
				TSharedPtr<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&ParsedJS); 
				TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(ResultJS);

				if (FJsonSerializer::Deserialize(Reader, JsonObj))
				{
					FString ReturnValueFieldName = "ReturnValue";
					
					TArray<TSharedPtr<FJsonValue>> ValuesArray;
					TSharedPtr<FJsonValue> ReturnValueField = JsonObj->TryGetField(ReturnValueFieldName);
					
					ValuesArray.Add(ReturnValueField);
					
					FJsonSerializer::Serialize(ValuesArray, *Writer, true);
				}
				
				ParsedJS.RemoveFromStart("[\n", ESearchCase::Type::CaseSensitive);
				ParsedJS.RemoveFromEnd("\n]", ESearchCase::Type::CaseSensitive);
				
				ParsedJS.TrimStartInline();
				ParsedJS.TrimEndInline();
				
				InvokeJSFunctionRaw(ResultCallbackId, ParsedJS, false);
			} else {
				InvokeJSFunctionRaw(ResultCallbackId, ResultJS, false);
			}
		}
		else
		{
			InvokeJSFunction(ResultCallbackId, 0, nullptr, false);
		}
	}

	if (Params.Num() > 0)
	{
		// Invoke the parameters destructors as needed before Params itself gets released
		for (TFieldIterator<FProperty> It(Function); It && It->HasAnyPropertyFlags(CPF_Parm); ++It)
		{
			if (It->HasAnyPropertyFlags(CPF_NoDestructor) || !It->IsInContainer(Function->GetStructureSize()))
			{
				continue;
			}

			It->DestroyValue_InContainer(Params.GetData());
		}
	}
	
	return true;
}

void FMobileJSScripting::InitializeScript(const TSharedRef<class IWebBrowserWindow>& InWindow)
{
	WindowPtr = InWindow;

	FString Script = ScriptingInit;

	FIntPoint Viewport = InWindow->GetViewportSize();
	int32 ScreenWidth = Viewport.X;
	int32 ScreenHeight = Viewport.Y;
#if PLATFORM_ANDROID
	if (const FString* ScreenWidthVar = FAndroidMisc::GetConfigRulesVariable(TEXT("ScreenWidth")))
	{
		ScreenWidth = FCString::Atoi(**ScreenWidthVar);
	}
	if (const FString* ScreenHeightVar = FAndroidMisc::GetConfigRulesVariable(TEXT("ScreenHeight")))
	{
		ScreenHeight = FCString::Atoi(**ScreenHeightVar);
	}
	Script.Append(TEXT("window.uePlatform = \"Android\";\n"));
#elif PLATFORM_IOS
	Script.Append(TEXT("window.uePlatform = \"iOS\";\n"));
#elif PLATFORM_MAC
	Script.Append(TEXT("window.uePlatform = \"macOS\";\n"));
#endif
	Script.Append(*FString::Printf(TEXT(
		"window.ueDeviceWidth = %d;\n"
		"window.ueDeviceHeight= %d;\n"
		"window.ueWindowWidth = %d;\n"
		"window.ueWindowHeight = %d;\n"),
		ScreenWidth, ScreenHeight,
		Viewport.X, Viewport.Y));
	Script.Append(ScriptingPostInit);
	InWindow->ExecuteJavascript(Script);
}

void FMobileJSScripting::PageStarted(const TSharedRef<class IWebBrowserWindow>& InWindow)
{
	SetWindow(InWindow);

	if (bInjectJSOnPageStarted )
	{
		InjectJavascript(InWindow);
	}
}

void FMobileJSScripting::PageLoaded(const TSharedRef<class IWebBrowserWindow>& InWindow)
{
	SetWindow(InWindow);

	if (!bInjectJSOnPageStarted)
	{
		InjectJavascript(InWindow);
	}
}

void FMobileJSScripting::InjectJavascript(const TSharedRef<class IWebBrowserWindow>& Window)
{
	// Expunge temporary objects.
	for (TMap<TObjectPtr<UObject>, ObjectBinding>::TIterator It(BoundObjects); It; ++It)
	{
		if (!It->Value.bIsPermanent)
		{
			It.RemoveCurrent();
		}
	}

	InitializeScript(Window);
	FString Script;
	
	if(GIsRunning)
	{
		// we can't try and use the ConvertObject function which uses UObject APIs 
		// while we are shutting down, since we can't rely on the UObject API existing 
		// in an unbroken state.
		for (auto& Item : PermanentUObjectsByName)
		{
			Script.Append(*FString::Printf(TEXT("window.ue['%s'] = %s;"), *Item.Key.ReplaceCharWithEscapedChar(), *ConvertObject(Item.Value)));
		}
	}
	
	Script.Append(ScriptingPostInit);
	Window->ExecuteJavascript(Script);
}

FMobileJSScripting::FMobileJSScripting(bool bJSBindingToLoweringEnabled, bool bInjectJSOnPageStarted, bool bDefaultJSReturnInDict)
	: FWebJSScripting(bJSBindingToLoweringEnabled)
	, bInjectJSOnPageStarted(bInjectJSOnPageStarted)
	, bDefaultJSReturnInDict(bDefaultJSReturnInDict)
{
}

void FMobileJSScripting::SetWindow(const TSharedRef<class IWebBrowserWindow>& InWindow)
{
	WindowPtr = InWindow;
}

#endif // PLATFORM_ANDROID  || PLATFORM_IOS  || PLATFORM_MAC
