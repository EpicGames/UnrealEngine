// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#if PLATFORM_ANDROID  || PLATFORM_IOS  ||  PLATFORM_MAC

#include "WebJSFunction.h"
#include "WebJSScripting.h"

DECLARE_LOG_CATEGORY_EXTERN(LogMobileJSScripting, Log, All);

typedef TSharedRef<class FMobileJSScripting> FMobileJSScriptingRef;
typedef TSharedPtr<class FMobileJSScripting> FMobileJSScriptingPtr;

/**
 * Implements handling of bridging UObjects client side with JavaScript renderer side.
 */
class FMobileJSScripting
	: public FWebJSScripting
	, public TSharedFromThis<FMobileJSScripting>
{
public:
	static const FString JSMessageTag;
	static const FString JSMessageHandler;

	FMobileJSScripting(bool bJSBindingToLoweringEnabled, bool bInjectJSOnPageStarted, bool bDefaultJSReturnInDict);

	virtual void BindUObject(const FString& Name, UObject* Object, bool bIsPermanent = true) override;
	virtual void UnbindUObject(const FString& Name, UObject* Object = nullptr, bool bIsPermanent = true) override;
	void BindUObject(const TSharedRef<class IWebBrowserWindow>& InWindow, const FString& Name, UObject* Object, bool bIsPermanent = true);
	void UnbindUObject(const TSharedRef<class IWebBrowserWindow>& InWindow, const FString& Name, UObject* Object = nullptr, bool bIsPermanent = true);

	/**
	 * Called when a message was received from the browser process.
	 *
	 * @param Command The command sent from the browser.
	 * @param Params Command-specific data.
	 * @return true if the message was handled, else false.
	 */
	bool OnJsMessageReceived(const FString& Command, const TArray<FString>& Params, const FString& Origin);

	FString ConvertStruct(UStruct* TypeInfo, const void* StructPtr);
	FString ConvertObject(UObject* Object);

	virtual void InvokeJSFunction(FGuid FunctionId, int32 ArgCount, FWebJSParam Arguments[], bool bIsError=false) override;
	virtual void InvokeJSErrorResult(FGuid FunctionId, const FString& Error) override;
	void PageStarted(const TSharedRef<class IWebBrowserWindow>& InWindow);  // Called when page load started
	void PageLoaded(const TSharedRef<class IWebBrowserWindow>& InWindow);  // Called on page completely loaded
	void SetWindow(const TSharedRef<class IWebBrowserWindow>& InWindow);

private:
	void InitializeScript(const TSharedRef<class IWebBrowserWindow>& InWindow);
	void InjectJavascript(const TSharedRef<class IWebBrowserWindow>& InWindow);
	void InvokeJSFunctionRaw(FGuid FunctionId, const FString& JSValue, bool bIsError=false);
	bool IsValid()
	{
		return WindowPtr.Pin().IsValid();
	}
	void AddPermanentBind(const FString& Name, UObject* Object);
	void RemovePermanentBind(const FString& Name, UObject* Object);

	/** Message handling helpers */

	bool HandleExecuteUObjectMethodMessage(const TArray<FString>& Params);

	/** Pointer to the Mobile Browser for this window. */
	TWeakPtr<class IWebBrowserWindow> WindowPtr;
	
	/** When to inject JS code */
	bool bInjectJSOnPageStarted;
	bool bDefaultJSReturnInDict;
};

#endif // PLATFORM_ANDROID  || PLATFORM_IOS  || PLATFORM_MAC
