// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if PLATFORM_ANDROID  || PLATFORM_IOS  || PLATFORM_MAC

#include "MobileJSScripting.h"
#include "Backends/JsonStructDeserializerBackend.h"
#include "Serialization/MemoryReader.h"

class FMobileJSStructDeserializerBackend
	: public FJsonStructDeserializerBackend
{
public:
	FMobileJSStructDeserializerBackend(FMobileJSScriptingRef InScripting, const FString& JsonString);

	virtual bool ReadProperty( FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex ) override;

private:
	FMobileJSScriptingRef Scripting;
	TArray<uint8> JsonData;
	FMemoryReader Reader;
};

#endif // USE_ANDROID_JNI || PLATFORM_IOS
