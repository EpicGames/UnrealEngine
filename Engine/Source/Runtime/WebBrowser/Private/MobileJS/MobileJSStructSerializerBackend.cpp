// Copyright Epic Games, Inc. All Rights Reserved.

#include "MobileJSStructSerializerBackend.h"

#if PLATFORM_ANDROID || PLATFORM_IOS || PLATFORM_MAC

#include "MobileJSScripting.h"
#include "UObject/UnrealType.h"
#include "UObject/PropertyPortFlags.h"
#include "Templates/Casts.h"

void FMobileJSStructSerializerBackend::WriteProperty(const FStructSerializerState& State, int32 ArrayIndex)
{
	// The parent class serialzes UObjects as NULLs
	if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(State.ValueProperty))
	{
		WriteUObject(State, ObjectProperty->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}
	// basic property type (json serializable)
	else
	{
		FJsonStructSerializerBackend::WriteProperty(State, ArrayIndex);
	}
}

void FMobileJSStructSerializerBackend::WriteUObject(const FStructSerializerState& State, UObject* Value)
{
	// Note this function uses WriteRawJSONValue to append non-json data to the output stream.
	FString RawValue = Scripting->ConvertObject(Value);
	if ((State.ValueProperty == nullptr) || (State.ValueProperty->ArrayDim > 1) || (State.ValueProperty->GetOwner<FArrayProperty>() != nullptr))
	{
		GetWriter()->WriteRawJSONValue(RawValue);
	}
	else if (State.KeyProperty != nullptr)
	{
		FString KeyString;
		State.KeyProperty->ExportTextItem_Direct(KeyString, State.KeyData, nullptr, nullptr, PPF_None);
		GetWriter()->WriteRawJSONValue(KeyString, RawValue);
	}
	else
	{
		GetWriter()->WriteRawJSONValue(Scripting->GetBindingName(State.ValueProperty), RawValue);
	}
}

FString FMobileJSStructSerializerBackend::ToString()
{
	ReturnBuffer.Add(0);
	ReturnBuffer.Add(0); // Add two as we're dealing with UTF-16, so 2 bytes
	return UTF16_TO_TCHAR((UTF16CHAR*)ReturnBuffer.GetData());
}

FMobileJSStructSerializerBackend::FMobileJSStructSerializerBackend(TSharedRef<class FMobileJSScripting> InScripting)
	: FJsonStructSerializerBackend(Writer, EStructSerializerBackendFlags::None)
	, Scripting(InScripting)
	, ReturnBuffer()
	, Writer(ReturnBuffer)
{
}

#endif // PLATFORM_ANDROID || PLATFORM_IOS || PLATFORM_MAC
