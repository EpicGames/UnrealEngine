// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if PLATFORM_ANDROID || PLATFORM_IOS || PLATFORM_MAC

#include "MobileJSScripting.h"
#include "Backends/JsonStructSerializerBackend.h"

class UObject;

/**
 * Implements a writer for UStruct serialization using JavaScript.
 *
 * Based on FJsonStructSerializerBackend, it adds support for certain object types not representable in pure JSON
 *
 */
class FMobileJSStructSerializerBackend
	: public FJsonStructSerializerBackend
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InScripting An instance of a web browser scripting obnject.
	 */
	FMobileJSStructSerializerBackend(FMobileJSScriptingRef InScripting);

public:
	virtual void WriteProperty(const FStructSerializerState& State, int32 ArrayIndex = 0) override;

	FString ToString();

private:
	void WriteUObject(const FStructSerializerState& State, UObject* Value);

	FMobileJSScriptingRef Scripting;
	TArray<uint8> ReturnBuffer;
	FMemoryWriter Writer;
};

#endif // PLATFORM_ANDROID || PLATFORM_IOS || PLATFORM_MAC
