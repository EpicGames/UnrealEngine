// Copyright Epic Games, Inc. All Rights Reserved.

#include "NativeJSScripting.h"


#include "NativeJSStructSerializerBackend.h"
#include "NativeJSStructDeserializerBackend.h"
#include "StructSerializer.h"
#include "StructDeserializer.h"
#include "UObject/UnrealType.h"
#include "NativeWebBrowserProxy.h"

namespace NativeFuncs
{
	const FString ExecuteMethodCommand = "ExecuteUObjectMethod";

	typedef TSharedRef<TJsonWriter<>> FJsonWriterRef;

	template<typename ValueType> void WriteValue(FJsonWriterRef Writer, const FString& Key, const ValueType& Value)
	{
		Writer->WriteValue(Key, Value);
	}
	void WriteNull(FJsonWriterRef Writer, const FString& Key)
	{
		Writer->WriteNull(Key);
	}
	void WriteArrayStart(FJsonWriterRef Writer, const FString& Key)
	{
		Writer->WriteArrayStart(Key);
	}
	void WriteObjectStart(FJsonWriterRef Writer, const FString& Key)
	{
		Writer->WriteObjectStart(Key);
	}
	void WriteRaw(FJsonWriterRef Writer, const FString& Key, const FString& Value)
	{
		Writer->WriteRawJSONValue(Key, Value);
	}
	template<typename ValueType> void WriteValue(FJsonWriterRef Writer, const int, const ValueType& Value)
	{
		Writer->WriteValue(Value);
	}
	void WriteNull(FJsonWriterRef Writer, int)
	{
		Writer->WriteNull();
	}
	void WriteArrayStart(FJsonWriterRef Writer, int)
	{
		Writer->WriteArrayStart();
	}
	void WriteObjectStart(FJsonWriterRef Writer, int)
	{
		Writer->WriteObjectStart();
	}
	void WriteRaw(FJsonWriterRef Writer, int, const FString& Value)
	{
		Writer->WriteRawJSONValue(Value);
	}

	template<typename KeyType>
	bool WriteJsParam(FNativeJSScriptingRef Scripting, FJsonWriterRef Writer, const KeyType& Key, FWebJSParam& Param)
	{
		switch (Param.Tag)
		{
			case FWebJSParam::PTYPE_NULL:
				WriteNull(Writer, Key);
				break;
			case FWebJSParam::PTYPE_BOOL:
				WriteValue(Writer, Key, Param.BoolValue);
				break;
			case FWebJSParam::PTYPE_DOUBLE:
				WriteValue(Writer, Key, Param.DoubleValue);
				break;
			case FWebJSParam::PTYPE_INT:
				WriteValue(Writer, Key, Param.IntValue);
				break;
			case FWebJSParam::PTYPE_STRING:
				WriteValue(Writer, Key, *Param.StringValue);
				break;
			case FWebJSParam::PTYPE_OBJECT:
			{
				if (Param.ObjectValue == nullptr)
				{
					WriteNull(Writer, Key);
				}
				else
				{
					FString ConvertedObject = Scripting->ConvertObject(Param.ObjectValue);
					WriteRaw(Writer, Key, ConvertedObject);
				}
				break;
			}
			case FWebJSParam::PTYPE_STRUCT:
			{
				FString ConvertedStruct = Scripting->ConvertStruct(Param.StructValue->GetTypeInfo(), Param.StructValue->GetData());
				WriteRaw(Writer, Key, ConvertedStruct);
				break;
			}
			case FWebJSParam::PTYPE_ARRAY:
			{
				WriteArrayStart(Writer, Key);
				for(int i=0; i < Param.ArrayValue->Num(); ++i)
				{
					WriteJsParam(Scripting, Writer, i, (*Param.ArrayValue)[i]);
				}
				Writer->WriteArrayEnd();
				break;
			}
			case FWebJSParam::PTYPE_MAP:
			{
				WriteObjectStart(Writer, Key);
				for(auto& Pair : *Param.MapValue)
				{
					WriteJsParam(Scripting, Writer, *Pair.Key, Pair.Value);
				}
				Writer->WriteObjectEnd();
				break;
			}
			default:
				return false;
		}
		return true;
	}
}


FString GetObjectPostInitScript(const FString& Name, const FString& FullyQualifiedName)
{
	return FString::Printf(TEXT("(function(){document.dispatchEvent(new CustomEvent('%s:ready', {details: %s}));})();"), *Name, *FullyQualifiedName);
}

void FNativeJSScripting::BindUObject(const FString& Name, UObject* Object, bool bIsPermanent )
{
	const FString ExposedName = GetBindingName(Name, Object);

	FString Converted = ConvertObject(Object);
	if (bIsPermanent)
	{
		// Existing permanent objects must be removed first and each object can only have one permanent binding
		if (PermanentUObjectsByName.Contains(ExposedName) || BoundObjects[Object].bIsPermanent)
		{
			return;
		}

		BoundObjects[Object]={true, -1};
		PermanentUObjectsByName.Add(ExposedName, Object);
	}
	
	if(!bLoaded)
	{
		PageLoaded();
	}
	else
	{
		const FString& EscapedName = ExposedName.ReplaceCharWithEscapedChar();
		FString SetValueScript = FString::Printf(TEXT("window.ue['%s'] = %s;"), *EscapedName, *Converted);
		SetValueScript.Append(GetObjectPostInitScript(EscapedName, FString::Printf(TEXT("window.ue['%s']"), *EscapedName)));
		ExecuteJavascript(SetValueScript);
	}
}

void FNativeJSScripting::ExecuteJavascript(const FString& Javascript)
{
	TSharedPtr<FNativeWebBrowserProxy> Window = WindowPtr.Pin();
	if (Window.IsValid())
	{
		Window->ExecuteJavascript(Javascript);
	}
}

void FNativeJSScripting::UnbindUObject(const FString& Name, UObject* Object, bool bIsPermanent)
{
	const FString ExposedName = GetBindingName(Name, Object);
	if (bIsPermanent)
	{
		// If overriding an existing permanent object, make it non-permanent
		if (PermanentUObjectsByName.Contains(ExposedName) && (Object == nullptr || PermanentUObjectsByName[ExposedName] == Object))
		{
			Object = PermanentUObjectsByName.FindAndRemoveChecked(ExposedName);
			BoundObjects.Remove(Object);
			return;
		}
		else
		{
			return;
		}
	}

	FString DeleteValueScript = FString::Printf(TEXT("delete window.ue['%s'];"), *ExposedName.ReplaceCharWithEscapedChar());
	ExecuteJavascript(DeleteValueScript);
}

int32 ParseParams(const FString& ParamStr, TArray<FString>& OutArray)
{
	OutArray.Reset();
	const TCHAR *Start = *ParamStr;
	if (Start && *Start != TEXT('\0'))
	{
		int32 DelimLimit = 4;
		while (const TCHAR *At = FCString::Strstr(Start, TEXT("/")))
		{
			OutArray.Emplace(FString::ConstructFromPtrSize(Start, At - Start));
			Start = At + 1;
			if (--DelimLimit == 0)
			{
				break;
			}
		}
		if (*Start)
		{
			OutArray.Emplace(Start);
		}
	}
	return OutArray.Num();
}

bool FNativeJSScripting::OnJsMessageReceived(const FString& Message)
{
	check(IsInGameThread());

	bool Result = false;
	TArray<FString> Params;
	if (ParseParams(Message, Params))
	{
		FString Command = Params[0];
		Params.RemoveAt(0, 1);

		if (Command == NativeFuncs::ExecuteMethodCommand)
		{
			Result = HandleExecuteUObjectMethodMessage(Params);
		}
	}
	return Result;
}

FString FNativeJSScripting::ConvertStruct(UStruct* TypeInfo, const void* StructPtr)
{
	TArray<uint8> ReturnBuffer;
	FMemoryWriter Writer(ReturnBuffer);

	FNativeJSStructSerializerBackend ReturnBackend = FNativeJSStructSerializerBackend(SharedThis(this), Writer);
	FStructSerializer::Serialize(StructPtr, *TypeInfo, ReturnBackend);

	// Extract the result value from the serialized JSON object:
	ReturnBuffer.Add(0);
	ReturnBuffer.Add(0); // Add two as we're dealing with UTF-16, so 2 bytes
	return UTF16_TO_TCHAR((UTF16CHAR*)ReturnBuffer.GetData());
}

FString FNativeJSScripting::ConvertObject(UObject* Object)
{
	RetainBinding(Object);
	UClass* Class = Object->GetClass();

	bool first = true;
	FString Result = "(function(){ return Object.create({";
	for (TFieldIterator<UFunction> FunctionIt(Class, EFieldIteratorFlags::IncludeSuper); FunctionIt; ++FunctionIt)
	{
		UFunction* Function = *FunctionIt;
		if(!first)
		{
			Result.Append(TEXT(","));
		}
		else
		{
			first = false;
		}
		Result.Append(*GetBindingName(Function));
		Result.Append(TEXT(": function "));
		Result.Append(*GetBindingName(Function));
		Result.Append(TEXT(" ("));

		bool firstArg = true;
		for ( TFieldIterator<FProperty> It(Function); It; ++It )
		{
			FProperty* Param = *It;
			if (Param->PropertyFlags & CPF_Parm && ! (Param->PropertyFlags & CPF_ReturnParm) )
			{
				FStructProperty *StructProperty = CastField<FStructProperty>(Param);
				if (!StructProperty || !StructProperty->Struct->IsChildOf(FWebJSResponse::StaticStruct()))
				{
					if(!firstArg)
					{
						Result.Append(TEXT(", "));
					}
					else
					{
						firstArg = false;
					}
					Result.Append(*GetBindingName(Param));
				}
			}
		}

		Result.Append(TEXT(")"));

		// We hijack the RPCResponseId and use it for our priority value.  0 means it has not been assigned and we default to 2.  1-5 is high-low priority which we map to the 0-4 range used by EmbeddedCommunication.
		int32 Priority = Function->RPCResponseId == 0 ? 2 : FMath::Clamp((int32)Function->RPCResponseId, 1, 5) - 1;

		Result.Append(TEXT(" {return window.ue.$.executeMethod('"));
		Result.Append(FString::FromInt(Priority));
		Result.Append(TEXT("',this.$id, arguments)}"));
	}
	Result.Append(TEXT("},{"));
	Result.Append(TEXT("$id: {writable: false, configurable:false, enumerable: false, value: '"));
 	Result.Append(*PtrToGuid(Object).ToString(EGuidFormats::Digits));
	Result.Append(TEXT("'}})})()"));
	return Result;
}

void FNativeJSScripting::InvokeJSFunction(FGuid FunctionId, int32 ArgCount, FWebJSParam Arguments[], bool bIsError)
{
	if (!IsValid())
	{
		return;
	}

	FString CallbackScript = FString::Printf(TEXT("window.ue.$.invokeCallback('%s', %s, "), *FunctionId.ToString(EGuidFormats::Digits), (bIsError) ? TEXT("true") : TEXT("false"));
	{
		TArray<uint8> Buffer;
		FMemoryWriter MemoryWriter(Buffer);
		NativeFuncs::FJsonWriterRef JsonWriter = TJsonWriter<>::Create(&MemoryWriter);
		JsonWriter->WriteArrayStart();
		for (int i = 0; i < ArgCount; i++)
		{
			NativeFuncs::WriteJsParam(SharedThis(this), JsonWriter, i, Arguments[i]);
		}
		JsonWriter->WriteArrayEnd();
		CallbackScript.Append((TCHAR*)Buffer.GetData(), Buffer.Num() / sizeof(TCHAR));
	}
	CallbackScript.Append(TEXT(")"));

	ExecuteJavascript(CallbackScript);

}

void FNativeJSScripting::InvokeJSFunctionRaw(FGuid FunctionId, const FString& RawJSValue, bool bIsError)
{
	if (!IsValid())
	{
		return;
	}

	FString CallbackScript = FString::Printf(TEXT("window.ue.$.invokeCallback('%s', %s, [%s])"),
		*FunctionId.ToString(EGuidFormats::Digits), (bIsError)?TEXT("true"):TEXT("false"), *RawJSValue);
	ExecuteJavascript(CallbackScript);
}

void FNativeJSScripting::InvokeJSErrorResult(FGuid FunctionId, const FString& Error)
{
	FWebJSParam Args[1] = {FWebJSParam(Error)};
	InvokeJSFunction(FunctionId, 1, Args, true);
}

bool FNativeJSScripting::HandleExecuteUObjectMethodMessage(const TArray<FString>& MessageArgs)
{
	if (MessageArgs.Num() != 4)
	{
		return false;
	}

	const FString& ObjectIdStr = MessageArgs[0];
	FGuid ObjectKey;
	UObject* Object = nullptr;
	if (FGuid::Parse(ObjectIdStr, ObjectKey))
	{
		Object = GuidToPtr(ObjectKey);
	}
	else if(PermanentUObjectsByName.Contains(ObjectIdStr))
	{
		Object = PermanentUObjectsByName[ObjectIdStr];
	}

	if(Object == nullptr)
	{
		// Unknown uobject id/name
		return false;
	}
	
	// Get the promise callback and use that to report any results from executing this function.
	FGuid ResultCallbackId;
	if (!FGuid::Parse(MessageArgs[1], ResultCallbackId))
	{
		// Invalid GUID
		return false;
	}

	FName MethodName = FName(*MessageArgs[2]);
	UFunction* Function = Object->FindFunction(MethodName);
	if (!Function)
	{
		InvokeJSErrorResult(ResultCallbackId, TEXT("Unknown UObject Function"));
		return true;
	}

	// Coerce arguments to function arguments.
	uint16 ParamsSize = Function->ParmsSize;
	TArray<uint8> Params;
	FProperty* ReturnParam = nullptr;
	FProperty* PromiseParam = nullptr;

	if (ParamsSize > 0)
	{
		// Find return parameter and a promise argument if present, as we need to handle them differently
		for ( TFieldIterator<FProperty> It(Function); It; ++It )
		{
			FProperty* Param = *It;
			if (Param->PropertyFlags & CPF_Parm)
			{
				if (Param->PropertyFlags & CPF_ReturnParm)
				{
					ReturnParam = Param;
				}
				else
				{
					FStructProperty *StructProperty = CastField<FStructProperty>(Param);
					if (StructProperty && StructProperty->Struct->IsChildOf(FWebJSResponse::StaticStruct()))
					{
						PromiseParam = Param;
					}
				}
				if (ReturnParam && PromiseParam)
				{
					break;
				}
			}
		}

		// UFunction is a subclass of UStruct, so we can treat the arguments as a struct for deserialization
		Params.AddUninitialized(ParamsSize);
		Function->InitializeStruct(Params.GetData());

		// Note: This is a no-op on platforms that are using a 16-bit TCHAR
		FTCHARToUTF16 UTF16String(*MessageArgs[3], MessageArgs[3].Len());

		TArray<uint8> JsonData;
		JsonData.Append((uint8*)UTF16String.Get(), UTF16String.Length() * sizeof(UTF16CHAR));

		FMemoryReader Reader(JsonData);
		FNativeJSStructDeserializerBackend Backend = FNativeJSStructDeserializerBackend(SharedThis(this), Reader);
		FStructDeserializer::Deserialize(Params.GetData(), *Function, Backend);
	}

	if (PromiseParam)
	{
		FWebJSResponse* PromisePtr = PromiseParam->ContainerPtrToValuePtr<FWebJSResponse>(Params.GetData());
		if (PromisePtr)
		{
			*PromisePtr = FWebJSResponse(SharedThis(this), ResultCallbackId);
		}
	}

	Object->ProcessEvent(Function, Params.GetData());
	if ( ! PromiseParam ) // If PromiseParam is set, we assume that the UFunction will ensure it is called with the result
	{
		if ( ReturnParam )
		{
			FStructSerializerPolicies ReturnPolicies;
			ReturnPolicies.PropertyFilter = [&ReturnParam](const FProperty* CandidateProperty, const FProperty* ParentProperty)
			{
				return ParentProperty != nullptr || CandidateProperty == ReturnParam;
			};
			TArray<uint8> ReturnBuffer;
			FMemoryWriter Writer(ReturnBuffer);

			FNativeJSStructSerializerBackend ReturnBackend = FNativeJSStructSerializerBackend(SharedThis(this), Writer);
			FStructSerializer::Serialize(Params.GetData(), *Function, ReturnBackend, ReturnPolicies);

			// Extract the result value from the serialized JSON object:
			ReturnBuffer.Add(0);
			ReturnBuffer.Add(0); // Add two as we're dealing with UTF-16, so 2 bytes
			const FString ResultJS = UTF16_TO_TCHAR((UTF16CHAR*)ReturnBuffer.GetData());

			InvokeJSFunctionRaw(ResultCallbackId, ResultJS, false);
		}
		else
		{
			InvokeJSFunction(ResultCallbackId, 0, nullptr, false);
		}
	}

	if (Params.Num() > 0)
	{
		// Invoke the parameters destructors as needed before Params itself gets released
		for (TFieldIterator<FProperty> It(Function); It && It->HasAnyPropertyFlags(CPF_Parm); ++It)
		{
			if (It->HasAnyPropertyFlags(CPF_NoDestructor) || !It->IsInContainer(Function->GetStructureSize()))
			{
				continue;
			}

			It->DestroyValue_InContainer(Params.GetData());
		}
	}
	
	return true;
}

FString FNativeJSScripting::GetInitializeScript()
{
	const FString NativeScriptingInit =
		TEXT("(function() {")
			TEXT("var util = Object.create({")

			// Simple random-based (RFC-4122 version 4) UUID generator.
			// Version 4 UUIDs have the form xxxxxxxx-xxxx-4xxx-yxxx-xxxxxxxxxxxx where x is any hexadecimal digit and y is one of 8, 9, a, or b
			// This function returns the UUID as a hex string without the dashes
			TEXT("uuid: function()")
			TEXT("{")
			TEXT("	var b = new Uint8Array(16); window.crypto.getRandomValues(b);")
			TEXT("	b[6] = b[6]&0xf|0x40; b[8]=b[8]&0x3f|0x80;") // Set the reserved bits to the correct values
			TEXT("	return Array.prototype.reduce.call(b, function(a,i){return a+((0x100|i).toString(16).substring(1))},'').toUpperCase();")
			TEXT("}, ")

			// save a callback function in the callback registry
			// returns the uuid of the callback for passing to the host application
			// ensures that each function object is only stored once.
			// (Closures executed multiple times are considered separate objects.)
			TEXT("registerCallback: function(callback)")
			TEXT("{")
			TEXT("	var key;")
			TEXT("	for(key in this.callbacks)")
			TEXT("	{")
			TEXT("		if (!this.callbacks[key].isOneShot && this.callbacks[key].accept === callback)")
			TEXT("		{")
			TEXT("			return key;")
			TEXT("		}")
			TEXT("	}")
			TEXT("	key = this.uuid();")
			TEXT("	this.callbacks[key] = {accept:callback, reject:callback, bIsOneShot:false};")
			TEXT("	return key;")
			TEXT("}, ")

			TEXT("registerPromise: function(accept, reject, name)")
			TEXT("{")
			TEXT("	var key = this.uuid();")
			TEXT("	this.callbacks[key] = {accept:accept, reject:reject, bIsOneShot:true, name:name};")
			TEXT("	return key;")
			TEXT("}, ")

			// strip ReturnValue object wrapper if present
			TEXT("returnValToObj: function(args)")
			TEXT("{")
			TEXT("	return Array.prototype.map.call(args, function(item){return item.ReturnValue || item});")
			TEXT("}, ")

			// invoke a callback method or promise by uuid
			TEXT("invokeCallback: function(key, bIsError, args)")
			TEXT("{")
			TEXT("	var callback = this.callbacks[key];")
			TEXT("	if (typeof callback === 'undefined')")
			TEXT("	{")
			TEXT("		console.error('Unknown callback id', key);")
			TEXT("		return;")
			TEXT("	}")
			TEXT("	if (callback.bIsOneShot)")
			TEXT("	{")
			TEXT("		callback.iwanttodeletethis=true;")
			TEXT("		delete this.callbacks[key];")
			TEXT("	}")
			TEXT("	callback[bIsError?'reject':'accept'].apply(window, this.returnValToObj(args));")
			TEXT("}, ")

			// convert an argument list to a dictionary of arguments.
			// The args argument must be an argument object as it uses the callee member to deduce the argument names
			TEXT("argsToDict: function(args)")
			TEXT("{")
			TEXT("	var res = {};")
			TEXT("	args.callee.toString().match(/\\((.+?)\\)/)[1].split(/\\s*,\\s*/).forEach(function(name, idx){res[name]=args[idx]});")
			TEXT("	return res;")
			TEXT("}, ")

			// encodes and sends a message to the host application
			TEXT("sendMessage: function()")
			TEXT("{")
			// @todo: Each kairos native browser will have a different way of passing a message out, here we use webkit postmessage but we'll need
			//    to be aware of our target platform when generating this script and adjust accordingly
			TEXT("  var delimiter = '/';")

#if PLATFORM_ANDROID
			TEXT("  if(window.JSBridge){")
			TEXT("    window.JSBridge.postMessage('', 'browserProxy', 'handlejs', Array.prototype.slice.call(arguments).join(delimiter));")
			TEXT("  }")
#else
			TEXT("  if(window.webkit && window.webkit.messageHandlers && window.webkit.messageHandlers.browserProxy){")
			TEXT("    window.webkit.messageHandlers.browserProxy.postMessage(Array.prototype.slice.call(arguments).join(delimiter));")
			TEXT("  }")
#endif
			TEXT("}, ")

			// custom replacer function passed into JSON.stringify to handle cases where there are function objects in the argument list
			// of the executeMethod call.  In those cases we want to be able to pass them as callbacks.
			TEXT("customReplacer: function(key, value)")
			TEXT("{")
			TEXT("	if (typeof value === 'function')")
			TEXT("	{")
			TEXT("		return window.ue.$.registerCallback(value);")
			TEXT("	}")
			TEXT("	return value;")
			TEXT("},")

			// uses the above helper methods to execute a method on a uobject instance.
			// the method set as callee on args needs to be a named function, as the name of the method to invoke is taken from it
			TEXT("executeMethod: function(priority, id, args)")
			TEXT("{")
			TEXT("	var self = this;") // the closures need access to the outer this object

			// Create a promise object to return back to the caller and create a callback function to handle the response
			TEXT("	var promiseID;")
			TEXT("	var promise = new Promise(function (accept, reject) ")
			TEXT("	{")
			TEXT("		promiseID = self.registerPromise(accept, reject, args.callee.name)")
			TEXT("	});")

			// Actually invoke the method by sending a message to the host app
			TEXT("	this.sendMessage(priority, '") + NativeFuncs::ExecuteMethodCommand + TEXT("', id, promiseID, args.callee.name, JSON.stringify(this.argsToDict(args), this.customReplacer));")

			// Return the promise object to the caller
			TEXT("	return promise;")
			TEXT("}")
			TEXT("},{callbacks: {value:{}}});")

			// Create the global window.ue variable
			TEXT("window.ue = Object.create({}, {'$': {writable: false, configurable:false, enumerable: false, value:util}});")
		TEXT("})();")
	;

	return NativeScriptingInit;
}

void FNativeJSScripting::PageLoaded()
{
	// Expunge temporary objects.
	for (decltype(BoundObjects)::TIterator It(BoundObjects); It; ++It)
	{
		if (!It->Value.bIsPermanent)
		{
			It.RemoveCurrent();
		}
	}

	FString Script = GetInitializeScript();
	
	for(auto& Item : PermanentUObjectsByName)
	{
		Script.Append(*FString::Printf(TEXT("window.ue['%s'] = %s;"), *Item.Key.ReplaceCharWithEscapedChar(), *ConvertObject(Item.Value)));
	}

	// Append postinit for each object we added.
	for (auto& Item : PermanentUObjectsByName)
	{
		const FString& Name = Item.Key.ReplaceCharWithEscapedChar();
		Script.Append(GetObjectPostInitScript(Name, FString::Printf(TEXT("window.ue['%s']"), *Name)));
	}

	// Append postinit for window.ue
	Script.Append(GetObjectPostInitScript(TEXT("ue"), TEXT("window.ue")));

	bLoaded = true;
	ExecuteJavascript(Script);
}

FNativeJSScripting::FNativeJSScripting(bool bJSBindingToLoweringEnabled, TSharedRef<FNativeWebBrowserProxy> Window)
	: FWebJSScripting(bJSBindingToLoweringEnabled)
	, bLoaded(false)
{
	WindowPtr = Window;
}

