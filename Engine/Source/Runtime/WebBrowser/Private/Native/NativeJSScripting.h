// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "WebJSFunction.h"
#include "WebJSScripting.h"

typedef TSharedRef<class FNativeJSScripting> FNativeJSScriptingRef;
typedef TSharedPtr<class FNativeJSScripting> FNativeJSScriptingPtr;

class FNativeWebBrowserProxy;

/**
 * Implements handling of bridging UObjects client side with JavaScript renderer side.
 */
class FNativeJSScripting
	: public FWebJSScripting
	, public TSharedFromThis<FNativeJSScripting>
{
public:
	//static const FString JSMessageTag;

	FNativeJSScripting(bool bJSBindingToLoweringEnabled, TSharedRef<FNativeWebBrowserProxy> Window);

	virtual void BindUObject(const FString& Name, UObject* Object, bool bIsPermanent = true) override;
	virtual void UnbindUObject(const FString& Name, UObject* Object = nullptr, bool bIsPermanent = true) override;

	bool OnJsMessageReceived(const FString& Message);

	FString ConvertStruct(UStruct* TypeInfo, const void* StructPtr);
	FString ConvertObject(UObject* Object);

	virtual void InvokeJSFunction(FGuid FunctionId, int32 ArgCount, FWebJSParam Arguments[], bool bIsError=false) override;
	virtual void InvokeJSErrorResult(FGuid FunctionId, const FString& Error) override;
	void PageLoaded();

private:
	FString GetInitializeScript();
	void InvokeJSFunctionRaw(FGuid FunctionId, const FString& JSValue, bool bIsError=false);
	bool IsValid()
	{
		return WindowPtr.Pin().IsValid();
	}

	/** Message handling helpers */
	bool HandleExecuteUObjectMethodMessage(const TArray<FString>& Params);
	void ExecuteJavascript(const FString& Javascript);

	TWeakPtr<FNativeWebBrowserProxy> WindowPtr;
	bool bLoaded;
};
