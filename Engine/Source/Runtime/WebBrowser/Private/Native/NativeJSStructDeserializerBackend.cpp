// Copyright Epic Games, Inc. All Rights Reserved.

#include "NativeJSStructDeserializerBackend.h"
#include "NativeJSScripting.h"
#include "UObject/UnrealType.h"
#include "Templates/Casts.h"

namespace NativeFuncs
{
	// @todo: this function is copied from CEFJSStructDeserializerBackend.cpp. Move shared utility code to a common header file
	/**
	 * Sets the value of the given property.
	 *
	 * @param Property The property to set.
	 * @param Outer The property that contains the property to be set, if any.
	 * @param Data A pointer to the memory holding the property's data.
	 * @param ArrayIndex The index of the element to set (if the property is an array).
	 * @return true on success, false otherwise.
	 * @see ClearPropertyValue
	 */
	template<typename FPropertyType, typename PropertyType>
	bool SetPropertyValue( FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex, const PropertyType& Value )
	{
		PropertyType* ValuePtr = nullptr;
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Outer);

		if (ArrayProperty != nullptr)
		{
			if (ArrayProperty->Inner != Property)
			{
				return false;
			}

			FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->template ContainerPtrToValuePtr<void>(Data));
			int32 Index = ArrayHelper.AddValue();

			ValuePtr = (PropertyType*)ArrayHelper.GetRawPtr(Index);
		}
		else
		{
			FPropertyType* TypedProperty = CastField<FPropertyType>(Property);

			if (TypedProperty == nullptr || ArrayIndex >= TypedProperty->ArrayDim)
			{
				return false;
			}

			ValuePtr = TypedProperty->template ContainerPtrToValuePtr<PropertyType>(Data, ArrayIndex);
		}

		if (ValuePtr == nullptr)
		{
			return false;
		}

		*ValuePtr = Value;

		return true;
	}
}


bool FNativeJSStructDeserializerBackend::ReadProperty( FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex )
{
	switch (GetLastNotation())
	{
		case EJsonNotation::String:
		{
			if (Property->IsA<FStructProperty>())
			{
				FStructProperty* StructProperty = CastField<FStructProperty>(Property);

				if ( StructProperty->Struct == FWebJSFunction::StaticStruct())
				{

					FGuid CallbackID;
					if (!FGuid::Parse(GetReader()->GetValueAsString(), CallbackID))
					{
						return false;
					}

					FWebJSFunction CallbackObject(Scripting, CallbackID);
					return NativeFuncs::SetPropertyValue<FStructProperty, FWebJSFunction>(Property, Outer, Data, ArrayIndex, CallbackObject);
				}
			}
		}
		break;
	}

	// If we reach this, default to parent class behavior
	return FJsonStructDeserializerBackend::ReadProperty(Property, Outer, Data, ArrayIndex);
}

FNativeJSStructDeserializerBackend::FNativeJSStructDeserializerBackend(FNativeJSScriptingRef InScripting, FMemoryReader& Reader)
	: FJsonStructDeserializerBackend(Reader)
	, Scripting(InScripting)
{
}
