// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NativeJSScripting.h"
#include "Backends/JsonStructDeserializerBackend.h"
#include "Serialization/MemoryReader.h"

class FNativeJSStructDeserializerBackend
	: public FJsonStructDeserializerBackend
{
public:
	FNativeJSStructDeserializerBackend(FNativeJSScriptingRef InScripting, FMemoryReader& Reader);

	virtual bool ReadProperty( FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex ) override;

private:
	FNativeJSScriptingRef Scripting;

};
