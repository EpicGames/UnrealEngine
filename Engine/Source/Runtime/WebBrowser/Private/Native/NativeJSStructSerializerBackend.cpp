// Copyright Epic Games, Inc. All Rights Reserved.

#include "Native/NativeJSStructSerializerBackend.h"

#include "NativeJSScripting.h"
#include "UObject/UnrealType.h"
#include "UObject/PropertyPortFlags.h"
#include "Templates/Casts.h"

void FNativeJSStructSerializerBackend::WriteProperty(const FStructSerializerState& State, int32 ArrayIndex)
{
	// The parent class serialzes UObjects as NULLs
	if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(State.ValueProperty))
	{
		WriteUObject(State, CastFieldChecked<FObjectProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}
	// basic property type (json serializable)
	else
	{
		FJsonStructSerializerBackend::WriteProperty(State, ArrayIndex);
	}
}

void FNativeJSStructSerializerBackend::WriteUObject(const FStructSerializerState& State, UObject* Value)
{
	// Note this function uses WriteRawJSONValue to append non-json data to the output stream.
	FString RawValue = Scripting->ConvertObject(Value);
	if ((State.ValueProperty == nullptr) || (State.ValueProperty->ArrayDim > 1) || State.ValueProperty->GetOwner< FArrayProperty>())
	{
		GetWriter()->WriteRawJSONValue(RawValue);
	}
	else if (State.KeyProperty != nullptr)
	{
		FString KeyString;
		State.KeyProperty->ExportTextItem_Direct(KeyString, State.KeyData, nullptr, nullptr, PPF_None);
		GetWriter()->WriteRawJSONValue(KeyString, RawValue);
	}
	else
	{
		GetWriter()->WriteRawJSONValue(Scripting->GetBindingName(State.ValueProperty), RawValue);
	}
}

FNativeJSStructSerializerBackend::FNativeJSStructSerializerBackend(TSharedRef<class FNativeJSScripting> InScripting, FMemoryWriter& Writer)
	: FJsonStructSerializerBackend(Writer, EStructSerializerBackendFlags::Default)
	, Scripting(InScripting)
{
}