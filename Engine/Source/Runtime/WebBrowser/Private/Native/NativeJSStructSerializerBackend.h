// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NativeJSScripting.h"
#include "Backends/JsonStructSerializerBackend.h"

class UObject;

/**
 * Implements a writer for UStruct serialization using JavaScript.
 *
 * Based on FJsonStructSerializerBackend, it adds support for certain object types not representable in pure JSON
 *
 */
class FNativeJSStructSerializerBackend
	: public FJsonStructSerializerBackend
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InScripting An instance of a web browser scripting obnject.
	 */
	FNativeJSStructSerializerBackend(FNativeJSScriptingRef InScripting, FMemoryWriter& Writer);

public:
	virtual void WriteProperty(const FStructSerializerState& State, int32 ArrayIndex = 0) override;

private:
	void WriteUObject(const FStructSerializerState& State, UObject* Value);

	FNativeJSScriptingRef Scripting;
};
