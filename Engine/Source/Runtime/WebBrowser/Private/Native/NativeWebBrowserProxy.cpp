// Copyright Epic Games, Inc. All Rights Reserved.


#include "NativeWebBrowserProxy.h"
#include "NativeJSScripting.h"
#include "Misc/EmbeddedCommunication.h"


FNativeWebBrowserProxy::FNativeWebBrowserProxy(bool bInJSBindingToLoweringEnabled)
	: bJSBindingToLoweringEnabled(bInJSBindingToLoweringEnabled)
{

}

void FNativeWebBrowserProxy::Initialize()
{
	Scripting = MakeShareable(new FNativeJSScripting(bJSBindingToLoweringEnabled, SharedThis(this)));
	FEmbeddedDelegates::GetNativeToEmbeddedParamsDelegateForSubsystem(TEXT("browserProxy")).AddRaw(this, &FNativeWebBrowserProxy::HandleEmbeddedCommunication);
}

FNativeWebBrowserProxy::~FNativeWebBrowserProxy()
{
	FEmbeddedDelegates::GetNativeToEmbeddedParamsDelegateForSubsystem(TEXT("browserProxy")).RemoveAll(this);
}

bool FNativeWebBrowserProxy::OnJsMessageReceived(const FString& Message)
{
	return Scripting->OnJsMessageReceived(Message);
}

void FNativeWebBrowserProxy::HandleEmbeddedCommunication(const FEmbeddedCallParamsHelper& Params)
{
	FString Error;
	if (Params.Command == "handlejs")
	{
		FString Message = Params.Parameters.FindRef(TEXT("script"));
		if (!Message.IsEmpty())
		{
			if (!OnJsMessageReceived(Message))
			{
				Error = TEXT("Command failed");
			}
		}
	}
	else if (Params.Command == "pageload")
	{
		Scripting->PageLoaded();
	}

	Params.OnCompleteDelegate(FEmbeddedCommunicationMap(), Error);
}

void FNativeWebBrowserProxy::LoadURL(FString NewURL)
{
}

void FNativeWebBrowserProxy::LoadString(FString Contents, FString DummyURL)
{
}

void FNativeWebBrowserProxy::SetViewportSize(FIntPoint WindowSize, FIntPoint WindowPos)
{
}

FIntPoint FNativeWebBrowserProxy::GetViewportSize() const
{
	return FIntPoint(ForceInitToZero);
}

FSlateShaderResource* FNativeWebBrowserProxy::GetTexture(bool bIsPopup /*= false*/)
{
	return nullptr;
}

bool FNativeWebBrowserProxy::IsValid() const
{
	return false;
}

bool FNativeWebBrowserProxy::IsInitialized() const
{
	return true;
}

bool FNativeWebBrowserProxy::IsClosing() const
{
	return false;
}

EWebBrowserDocumentState FNativeWebBrowserProxy::GetDocumentLoadingState() const
{
	return EWebBrowserDocumentState::Loading;
}

FString FNativeWebBrowserProxy::GetTitle() const
{
	return TEXT("");
}

FString FNativeWebBrowserProxy::GetUrl() const
{
	return TEXT("");
}

void FNativeWebBrowserProxy::GetSource(TFunction<void(const FString&)> Callback) const
{
	Callback(FString());
}

void FNativeWebBrowserProxy::SetSupportsMouseWheel(bool bValue)
{

}

bool FNativeWebBrowserProxy::GetSupportsMouseWheel() const
{
	return false;
}

bool FNativeWebBrowserProxy::OnKeyDown(const FKeyEvent& InKeyEvent)
{
	return false;
}

bool FNativeWebBrowserProxy::OnKeyUp(const FKeyEvent& InKeyEvent)
{
	return false;
}

bool FNativeWebBrowserProxy::OnKeyChar(const FCharacterEvent& InCharacterEvent)
{
	return false;
}

FReply FNativeWebBrowserProxy::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup)
{
	return FReply::Unhandled();
}

FReply FNativeWebBrowserProxy::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup)
{
	return FReply::Unhandled();
}

FReply FNativeWebBrowserProxy::OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup)
{
	return FReply::Unhandled();
}

FReply FNativeWebBrowserProxy::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup)
{
	return FReply::Unhandled();
}

void FNativeWebBrowserProxy::OnMouseLeave(const FPointerEvent& MouseEvent)
{
}

FReply FNativeWebBrowserProxy::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup)
{
	return FReply::Unhandled();
}


void FNativeWebBrowserProxy::OnFocus(bool SetFocus, bool bIsPopup)
{
}

void FNativeWebBrowserProxy::OnCaptureLost()
{
}

bool FNativeWebBrowserProxy::CanGoBack() const
{
	return false;
}

void FNativeWebBrowserProxy::GoBack()
{
}

bool FNativeWebBrowserProxy::CanGoForward() const
{
	return false;
}

void FNativeWebBrowserProxy::GoForward()
{
}

bool FNativeWebBrowserProxy::IsLoading() const
{
	return false;
}

void FNativeWebBrowserProxy::Reload()
{
}

void FNativeWebBrowserProxy::StopLoad()
{
}

void FNativeWebBrowserProxy::ExecuteJavascript(const FString& Script)
{
	FEmbeddedCallParamsHelper CallHelper;
	CallHelper.Command = TEXT("execjs");
	CallHelper.Parameters = { { TEXT("script"), Script } };

	FEmbeddedDelegates::GetEmbeddedToNativeParamsDelegateForSubsystem(TEXT("webview")).Broadcast(CallHelper);
}

void FNativeWebBrowserProxy::CloseBrowser(bool bForce, bool bBlockTillClosed /* ignored */)
{
}

void FNativeWebBrowserProxy::BindUObject(const FString& Name, UObject* Object, bool bIsPermanent /*= true*/)
{
	Scripting->BindUObject(Name, Object, bIsPermanent);
}

void FNativeWebBrowserProxy::UnbindUObject(const FString& Name, UObject* Object /*= nullptr*/, bool bIsPermanent /*= true*/)
{
	Scripting->UnbindUObject(Name, Object, bIsPermanent);
}

int FNativeWebBrowserProxy::GetLoadError()
{
	return 0;
}

void FNativeWebBrowserProxy::SetIsDisabled(bool bValue)
{
}

TSharedPtr<SWindow> FNativeWebBrowserProxy::GetParentWindow() const
{
	return nullptr;
}

void FNativeWebBrowserProxy::SetParentWindow(TSharedPtr<SWindow> Window)
{
}

void FNativeWebBrowserProxy::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	// Not supported on this platform
}

FReply FNativeWebBrowserProxy::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	// Not supported on this platform
	return FReply::Unhandled();
}

void FNativeWebBrowserProxy::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	// Not supported on this platform
}

FReply FNativeWebBrowserProxy::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	// Not supported on this platform
	return FReply::Unhandled();
}
