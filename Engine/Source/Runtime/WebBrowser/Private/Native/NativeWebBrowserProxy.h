// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IWebBrowserWindow.h"
#include "NativeJSScripting.h"

class FNativeWebBrowserProxy
	: public IWebBrowserWindow
	, public TSharedFromThis<FNativeWebBrowserProxy>
{
	// For creating instances of this class
	friend class FWebBrowserSingleton;

private:
	FNativeWebBrowserProxy(bool bJSBindingToLoweringEnabled);
	void Initialize();
	void HandleEmbeddedCommunication(const struct FEmbeddedCallParamsHelper& Params);
	bool OnJsMessageReceived(const FString& Message);

public:
	virtual ~FNativeWebBrowserProxy();

public:
	// IWebBrowserWindow Interface

	virtual void LoadURL(FString NewURL) override;
	virtual void LoadString(FString Contents, FString DummyURL) override;
	virtual void SetViewportSize(FIntPoint WindowSize, FIntPoint WindowPos) override;
	virtual FIntPoint GetViewportSize() const override;

	virtual class FSlateShaderResource* GetTexture(bool bIsPopup = false) override;
	virtual bool IsValid() const override;
	virtual bool IsInitialized() const override;
	virtual bool IsClosing() const override;
	virtual EWebBrowserDocumentState GetDocumentLoadingState() const override;
	virtual FString GetTitle() const override;
	virtual FString GetUrl() const override;
	virtual void GetSource(TFunction<void(const FString&)> Callback) const override;
	virtual bool OnKeyDown(const FKeyEvent& InKeyEvent) override;
	virtual bool OnKeyUp(const FKeyEvent& InKeyEvent) override;
	virtual bool OnKeyChar(const FCharacterEvent& InCharacterEvent) override;
	virtual void SetSupportsMouseWheel(bool bValue) override;
	virtual bool GetSupportsMouseWheel() const override;

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup) override;
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup) override;
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup) override;
	virtual FOnDragWindow& OnDragWindow() override
	{
		return DragWindowDelegate;
	}
	virtual void OnFocus(bool SetFocus, bool bIsPopup) override;
	virtual void OnCaptureLost() override;
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual bool CanGoBack() const override;
	virtual void GoBack() override;
	virtual bool CanGoForward() const override;
	virtual void GoForward() override;
	virtual bool IsLoading() const override;
	virtual void Reload() override;
	virtual void StopLoad() override;
	virtual void ExecuteJavascript(const FString& Script) override;
	virtual void CloseBrowser(bool bForce, bool bBlockTillClosed) override;
	virtual void BindUObject(const FString& Name, UObject* Object, bool bIsPermanent = true) override;
	virtual void UnbindUObject(const FString& Name, UObject* Object = nullptr, bool bIsPermanent = true) override;
	virtual int GetLoadError() override;
	virtual void SetIsDisabled(bool bValue) override;
	virtual TSharedPtr<SWindow> GetParentWindow() const override;
	virtual void SetParentWindow(TSharedPtr<SWindow> Window) override;
	virtual void ShowFloatingCloseButton(bool bShow, bool bDraggable) override
	{		
	}

	// @todo: None of these are actually called at the moment.
	DECLARE_DERIVED_EVENT(FNativeWebBrowserProxy, IWebBrowserWindow::FOnDocumentStateChanged, FOnDocumentStateChanged);
	virtual FOnDocumentStateChanged& OnDocumentStateChanged() override
	{
		return DocumentStateChangedEvent;
	}

	DECLARE_DERIVED_EVENT(FNativeWebBrowserProxy, IWebBrowserWindow::FOnTitleChanged, FOnTitleChanged);
	virtual FOnTitleChanged& OnTitleChanged() override
	{
		return TitleChangedEvent;
	}

	DECLARE_DERIVED_EVENT(FNativeWebBrowserProxy, IWebBrowserWindow::FOnUrlChanged, FOnUrlChanged);
	virtual FOnUrlChanged& OnUrlChanged() override
	{
		return UrlChangedEvent;
	}

	DECLARE_DERIVED_EVENT(FNativeWebBrowserProxy, IWebBrowserWindow::FOnToolTip, FOnToolTip);
	virtual FOnToolTip& OnToolTip() override
	{
		return ToolTipEvent;
	}

	DECLARE_DERIVED_EVENT(FNativeWebBrowserProxy, IWebBrowserWindow::FOnNeedsRedraw, FOnNeedsRedraw);
	virtual FOnNeedsRedraw& OnNeedsRedraw() override
	{
		return NeedsRedrawEvent;
	}

	virtual FOnBeforeBrowse& OnBeforeBrowse() override
	{
		return BeforeBrowseDelegate;
	}

	virtual FOnLoadUrl& OnLoadUrl() override
	{
		return LoadUrlDelegate;
	}

	virtual FOnCreateWindow& OnCreateWindow() override
	{
		return CreateWindowDelegate;
	}

	virtual FOnCloseWindow& OnCloseWindow() override
	{
		return CloseWindowDelegate;
	}

	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) override
	{
		return FCursorReply::Unhandled();
	}

	virtual FOnBeforePopupDelegate& OnBeforePopup() override
	{
		return BeforePopupDelegate;
	}

	virtual FOnBeforeResourceLoadDelegate& OnBeforeResourceLoad() override
	{
		return BeforeResourceLoadDelegate;
	}

	virtual FOnResourceLoadCompleteDelegate& OnResourceLoadComplete() override
	{
		return ResourceLoadCompleteDelegate;
	}

	virtual FOnConsoleMessageDelegate& OnConsoleMessage() override
	{
		return ConsoleMessageDelegate;
	}

	DECLARE_DERIVED_EVENT(FNativeWebBrowserProxy, IWebBrowserWindow::FOnShowPopup, FOnShowPopup);
	virtual FOnShowPopup& OnShowPopup() override
	{
		return ShowPopupEvent;
	}

	DECLARE_DERIVED_EVENT(FNativeWebBrowserProxy, IWebBrowserWindow::FOnDismissPopup, FOnDismissPopup);
	virtual FOnDismissPopup& OnDismissPopup() override
	{
		return DismissPopupEvent;
	}

	virtual FOnShowDialog& OnShowDialog() override
	{
		return ShowDialogDelegate;
	}

	virtual FOnDismissAllDialogs& OnDismissAllDialogs() override
	{
		return DismissAllDialogsDelegate;
	}

	virtual FOnSuppressContextMenu& OnSuppressContextMenu() override
	{
		return SuppressContextMenuDelgate;
	}

	virtual FOnUnhandledKeyDown& OnUnhandledKeyDown() override
	{
		return UnhandledKeyDownDelegate;
	}

	virtual FOnUnhandledKeyUp& OnUnhandledKeyUp() override
	{
		return UnhandledKeyUpDelegate;
	}

	virtual FOnUnhandledKeyChar& OnUnhandledKeyChar() override
	{
		return UnhandledKeyCharDelegate;
	}

private:

	/** Delegate for broadcasting load state changes. */
	FOnDocumentStateChanged DocumentStateChangedEvent;

	/** Delegate for broadcasting title changes. */
	FOnTitleChanged TitleChangedEvent;

	/** Delegate for broadcasting address changes. */
	FOnUrlChanged UrlChangedEvent;

	/** Delegate for broadcasting when the browser wants to show a tool tip. */
	FOnToolTip ToolTipEvent;

	/** Delegate for notifying that the window needs refreshing. */
	FOnNeedsRedraw NeedsRedrawEvent;

	/** Delegate that is executed prior to browser navigation. */
	FOnBeforeBrowse BeforeBrowseDelegate;

	/** Delegate for overriding Url contents. */
	FOnLoadUrl LoadUrlDelegate;

	/** Delegate for notifying that a popup window is attempting to open. */
	FOnBeforePopupDelegate BeforePopupDelegate;
	
	/** Delegate that is invoked before the browser loads a resource. Its primary purpose is to inject headers into the request. */
	FOnBeforeResourceLoadDelegate BeforeResourceLoadDelegate;

	/** Delegate that is invoked on completion of browser resource loads. Its primary purpose is to allow response to failures. */
	FOnResourceLoadCompleteDelegate ResourceLoadCompleteDelegate;

	/** Delegate that is invoked for each console message */
	FOnConsoleMessageDelegate ConsoleMessageDelegate;

	/** Delegate for handling requests to create new windows. */
	FOnCreateWindow CreateWindowDelegate;

	/** Delegate for handling requests to close new windows that were created. */
	FOnCloseWindow CloseWindowDelegate;

	/** Delegate for handling requests to close from the webview floating close button. */
	FOnFloatingCloseButtonPressed FloatingCloseButtonPressedDelegate;

	/** Delegate for handling requests to show the popup menu. */
	FOnShowPopup ShowPopupEvent;

	/** Delegate for handling requests to dismiss the current popup menu. */
	FOnDismissPopup DismissPopupEvent;

	/** Delegate for showing dialogs. */
	FOnShowDialog ShowDialogDelegate;

	/** Delegate for dismissing all dialogs. */
	FOnDismissAllDialogs DismissAllDialogsDelegate;

	/** Delegate for suppressing context menu */
	FOnSuppressContextMenu SuppressContextMenuDelgate;

	/** Delegate for handling key down events not handled by the browser. */
	FOnUnhandledKeyDown UnhandledKeyDownDelegate;
	
	/** Delegate for handling key up events not handled by the browser. */
	FOnUnhandledKeyUp UnhandledKeyUpDelegate;
	
	/** Delegate for handling key char events not handled by the browser. */
	FOnUnhandledKeyChar UnhandledKeyCharDelegate;

	FOnDragWindow		DragWindowDelegate;

	bool bJSBindingToLoweringEnabled;
	FNativeJSScriptingPtr Scripting;
};
