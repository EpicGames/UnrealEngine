// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWebBrowser.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SThrobber.h"
#include "Framework/Application/SlateApplication.h"

#if WITH_CEF3
#include "CEF/CEFWebBrowserWindow.h"
#endif // WITH_CEF3

#define LOCTEXT_NAMESPACE "WebBrowser"

SWebBrowser::SWebBrowser()
{
}

SWebBrowser::~SWebBrowser()
{
#if WITH_CEF3
	UnbindInputMethodSystem();
#endif
}

/*virtual*/ void SWebBrowser::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

#if WITH_EDITOR
#if WITH_CEF3
	if (WebBrowserWindow.IsValid()) 
	{
		const TSharedPtr<FCEFWebBrowserWindow> CefWebBrowserWindow = StaticCastSharedPtr<FCEFWebBrowserWindow>(WebBrowserWindow);
		if (CefWebBrowserWindow.IsValid() && CefWebBrowserWindow->HasResizeBug() && CefWebBrowserWindow->IsCefResizeNeeded())
		{
			CefWebBrowserWindow->RequestCefResize();
		}
	}
#endif // WITH_CEF3
#endif // WITH_EDITOR
}

void SWebBrowser::Construct(const FArguments& InArgs, const TSharedPtr<IWebBrowserWindow>& InWebBrowserWindow)
{
	OnLoadCompleted = InArgs._OnLoadCompleted;
	OnLoadError = InArgs._OnLoadError;
	OnLoadStarted = InArgs._OnLoadStarted;
	OnTitleChanged = InArgs._OnTitleChanged;
	OnUrlChanged = InArgs._OnUrlChanged;
	OnBeforeNavigation = InArgs._OnBeforeNavigation;
	OnLoadUrl = InArgs._OnLoadUrl;
	OnShowDialog = InArgs._OnShowDialog;
	OnDismissAllDialogs = InArgs._OnDismissAllDialogs;
	OnBeforePopup = InArgs._OnBeforePopup;
	OnConsoleMessage = InArgs._OnConsoleMessage;
	OnCreateWindow = InArgs._OnCreateWindow;
	OnCloseWindow = InArgs._OnCloseWindow;
	bShowInitialThrobber = InArgs._ShowInitialThrobber;
	
	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			.Visibility((InArgs._ShowControls || InArgs._ShowAddressBar) ? EVisibility::Visible : EVisibility::Collapsed)
			+ SHorizontalBox::Slot()
			.Padding(0, 5)
			.AutoWidth()
			[
				SNew(SHorizontalBox)
				.Visibility(InArgs._ShowControls ? EVisibility::Visible : EVisibility::Collapsed)
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.Text(LOCTEXT("Back","Back"))
					.IsEnabled(this, &SWebBrowser::CanGoBack)
					.OnClicked(this, &SWebBrowser::OnBackClicked)
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.Text(LOCTEXT("Forward", "Forward"))
					.IsEnabled(this, &SWebBrowser::CanGoForward)
					.OnClicked(this, &SWebBrowser::OnForwardClicked)
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.Text(this, &SWebBrowser::GetReloadButtonText)
					.OnClicked(this, &SWebBrowser::OnReloadClicked)
				]
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.Padding(5)
				[
					SNew(STextBlock)
					.Visibility(InArgs._ShowAddressBar ? EVisibility::Collapsed : EVisibility::Visible )
					.Text(this, &SWebBrowser::GetTitleText)
					.Justification(ETextJustify::Right)
				]
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			.Padding(5.f, 5.f)
			[
				// @todo: A proper addressbar widget should go here, for now we use a simple textbox.
				SAssignNew(InputText, SEditableTextBox)
				.Visibility(InArgs._ShowAddressBar ? EVisibility::Visible : EVisibility::Collapsed)
				.OnTextCommitted(this, &SWebBrowser::OnUrlTextCommitted)
				.Text(this, &SWebBrowser::GetAddressBarUrlText)
				.SelectAllTextWhenFocused(true)
				.ClearKeyboardFocusOnCommit(true)
				.RevertTextOnEscape(true)
			]
		]
		+SVerticalBox::Slot()
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SAssignNew(BrowserView, SWebBrowserView, InWebBrowserWindow)
				.ParentWindow(InArgs._ParentWindow)
				.InitialURL(InArgs._InitialURL)
				.ContentsToLoad(InArgs._ContentsToLoad)
				.UserAgentApplication(InArgs._UserAgentApplication)
				.ShowErrorMessage(InArgs._ShowErrorMessage)
				.SupportsTransparency(InArgs._SupportsTransparency)
				.SupportsThumbMouseButtonNavigation(InArgs._SupportsThumbMouseButtonNavigation)
				.BackgroundColor(InArgs._BackgroundColor)
				.PopupMenuMethod(InArgs._PopupMenuMethod)
				.ViewportSize(InArgs._ViewportSize)
				.OnLoadCompleted(OnLoadCompleted)
				.OnLoadError(OnLoadError)
				.OnLoadStarted(OnLoadStarted)
				.OnTitleChanged(OnTitleChanged)
				.OnUrlChanged(OnUrlChanged)
				.OnBeforePopup(OnBeforePopup)
				.OnCreateWindow(OnCreateWindow)
				.OnCloseWindow(OnCloseWindow)
				.OnBeforeNavigation(OnBeforeNavigation)
				.OnLoadUrl(OnLoadUrl)
				.OnShowDialog(OnShowDialog)
				.OnDismissAllDialogs(OnDismissAllDialogs)
				.Visibility(this, &SWebBrowser::GetViewportVisibility)
				.OnSuppressContextMenu(InArgs._OnSuppressContextMenu)
				.OnDragWindow(InArgs._OnDragWindow)
				.OnConsoleMessage(OnConsoleMessage)
				.BrowserFrameRate(InArgs._BrowserFrameRate)
			]
			+ SOverlay::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SCircularThrobber)
				.Radius(10.0f)
				.ToolTipText(LOCTEXT("LoadingThrobberToolTip", "Loading page..."))
				.Visibility(this, &SWebBrowser::GetLoadingThrobberVisibility)
			]
		]
	];


	WebBrowserWindow = (InWebBrowserWindow.IsValid() ? InWebBrowserWindow : BrowserView->GetBrowserWindow());	


#if WITH_CEF3
	if (WebBrowserWindow.IsValid()) 
	{
		const TSharedPtr<FCEFWebBrowserWindow> CefWebBrowserWindow = StaticCastSharedPtr<FCEFWebBrowserWindow>(WebBrowserWindow);
		if (CefWebBrowserWindow.IsValid())
		{
			CefWebBrowserWindow->SetCaptureCefFrames(InArgs._UseSmoothResizing);
		}
	}

	// Make the IME (Input Method Editor) work with HTML text input boxes on web pages,
	// for non-ASCII languages, like "Chinese (Traditional, Taiwan)".
	if (FSlateApplication::IsInitialized())
	{
		if (ITextInputMethodSystem* InputMethodSystem = FSlateApplication::Get().GetTextInputMethodSystem())
		{
			BindInputMethodSystem(InputMethodSystem);
		}
	}
#endif
}

void SWebBrowser::LoadURL(FString NewURL)
{
	if (BrowserView.IsValid())
	{
		BrowserView->LoadURL(MoveTemp(NewURL));
	}
}

void SWebBrowser::LoadString(FString Contents, FString DummyURL)
{
	if (BrowserView.IsValid())
	{
		BrowserView->LoadString(MoveTemp(Contents), MoveTemp(DummyURL));
	}
}

void SWebBrowser::Reload()
{
	if (BrowserView.IsValid())
	{
		BrowserView->Reload();
	}
}

void SWebBrowser::StopLoad()
{
	if (BrowserView.IsValid())
	{
		BrowserView->StopLoad();
	}
}

FText SWebBrowser::GetTitleText() const
{
	if (BrowserView.IsValid())
	{
		return BrowserView->GetTitleText();
	}
	return LOCTEXT("InvalidWindow", "Browser Window is not valid/supported");
}

FString SWebBrowser::GetUrl() const
{
	if (BrowserView.IsValid())
	{
		return BrowserView->GetUrl();
	}

	return FString();
}

FText SWebBrowser::GetAddressBarUrlText() const
{
	if(BrowserView.IsValid())
	{
		return BrowserView->GetAddressBarUrlText();
	}
	return FText::GetEmpty();
}

bool SWebBrowser::IsLoaded() const
{
	if (BrowserView.IsValid())
	{
		return BrowserView->IsLoaded();
	}

	return false;
}

bool SWebBrowser::IsLoading() const
{
	if (BrowserView.IsValid())
	{
		return BrowserView->IsLoading();
	}

	return false;
}

bool SWebBrowser::CanGoBack() const
{
	if (BrowserView.IsValid())
	{
		return BrowserView->CanGoBack();
	}
	return false;
}

void SWebBrowser::GoBack()
{
	if (BrowserView.IsValid())
	{
		BrowserView->GoBack();
	}
}

FReply SWebBrowser::OnBackClicked()
{
	GoBack();
	return FReply::Handled();
}

bool SWebBrowser::CanGoForward() const
{
	if (BrowserView.IsValid())
	{
		return BrowserView->CanGoForward();
	}
	return false;
}

void SWebBrowser::GoForward()
{
	if (BrowserView.IsValid())
	{
		BrowserView->GoForward();
	}
}

FReply SWebBrowser::OnForwardClicked()
{
	GoForward();
	return FReply::Handled();
}

FText SWebBrowser::GetReloadButtonText() const
{
	static FText ReloadText = LOCTEXT("Reload", "Reload");
	static FText StopText = LOCTEXT("StopText", "Stop");

	if (BrowserView.IsValid())
	{
		if (BrowserView->IsLoading())
		{
			return StopText;
		}
	}
	return ReloadText;
}

FReply SWebBrowser::OnReloadClicked()
{
	if (IsLoading())
	{
		StopLoad();
	}
	else
	{
		Reload();
	}
	return FReply::Handled();
}

void SWebBrowser::OnUrlTextCommitted( const FText& NewText, ETextCommit::Type CommitType )
{
	if(CommitType == ETextCommit::OnEnter)
	{
		LoadURL(NewText.ToString());
	}
}

EVisibility SWebBrowser::GetViewportVisibility() const
{
	if (!bForceThrobber && (!bShowInitialThrobber || BrowserView->IsInitialized()))
	{
		return EVisibility::Visible;
	}
	return EVisibility::Hidden;
}

EVisibility SWebBrowser::GetLoadingThrobberVisibility() const
{
	if (bForceThrobber || (bShowInitialThrobber && !BrowserView->IsInitialized()))
	{
		return EVisibility::Visible;
	}
	return EVisibility::Hidden;
}


void SWebBrowser::ExecuteJavascript(const FString& ScriptText)
{
	if (BrowserView.IsValid())
	{
		BrowserView->ExecuteJavascript(ScriptText);
	}
}

void SWebBrowser::GetSource(TFunction<void (const FString&)> Callback) const
{
	if (BrowserView.IsValid())
	{
		BrowserView->GetSource(Callback);
	}
}

void SWebBrowser::BindUObject(const FString& Name, UObject* Object, bool bIsPermanent)
{
	if (BrowserView.IsValid())
	{
		BrowserView->BindUObject(Name, Object, bIsPermanent);
	}
}

void SWebBrowser::UnbindUObject(const FString& Name, UObject* Object, bool bIsPermanent)
{
	if (BrowserView.IsValid())
	{
		BrowserView->UnbindUObject(Name, Object, bIsPermanent);
	}
}

void SWebBrowser::BindAdapter(const TSharedRef<IWebBrowserAdapter>& Adapter)
{
	if (BrowserView.IsValid())
	{
		BrowserView->BindAdapter(Adapter);
	}
}

void SWebBrowser::UnbindAdapter(const TSharedRef<IWebBrowserAdapter>& Adapter)
{
	if (BrowserView.IsValid())
	{
		BrowserView->UnbindAdapter(Adapter);
	}
}

void SWebBrowser::BindInputMethodSystem(ITextInputMethodSystem* TextInputMethodSystem)
{
	UnbindInputMethodSystem();

	if (!TextInputMethodSystem || !FSlateApplication::IsInitialized())
	{
		return;
	}

	
	if (BrowserView.IsValid())
	{
		BrowserView->BindInputMethodSystem(TextInputMethodSystem);
	}

#if WITH_CEF3
	// During shutdown, avoids crash, ensures we unbind Input Method System while the UI is still valid.
	InputMethodSystemSlatePreShutdownDelegateHandle = FSlateApplication::Get().OnPreShutdown().AddSP(
		SharedThis(this), &SWebBrowser::UnbindInputMethodSystem);
#endif
}

void SWebBrowser::UnbindInputMethodSystem()
{
	if (!FSlateApplication::IsInitialized())
	{
		return;
	}
	
	
	if (BrowserView.IsValid())
	{
		BrowserView->UnbindInputMethodSystem();
	}

#if WITH_CEF3
	if (InputMethodSystemSlatePreShutdownDelegateHandle.IsValid())
	{
		FSlateApplication::Get().OnPreShutdown().Remove(InputMethodSystemSlatePreShutdownDelegateHandle);
		InputMethodSystemSlatePreShutdownDelegateHandle.Reset();
	}
#endif
}

void SWebBrowser::SetParentWindow(TSharedPtr<SWindow> Window)
{
	if (BrowserView.IsValid())
	{
		BrowserView->SetParentWindow(Window);
	}
}

void SWebBrowser::ShowThrobber()
{
	bForceThrobber = true;
}

void SWebBrowser::HideThrobber()
{
	bForceThrobber = false;
}

FReply SWebBrowser::OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent)
{
	if (BrowserView.IsValid() && InFocusEvent.GetCause() == EFocusCause::SetDirectly)
	{
		FReply Reply = FReply::Handled();
		Reply.SetUserFocus(BrowserView.ToSharedRef(), InFocusEvent.GetCause(), true);
		return Reply;
	}

	return SCompoundWidget::OnFocusReceived(MyGeometry, InFocusEvent);
}

#undef LOCTEXT_NAMESPACE
