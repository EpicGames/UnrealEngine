// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWebBrowserView.h"
#include "Async/Async.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Containers/Ticker.h"
#include "WebBrowserModule.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "IWebBrowserDialog.h"
#include "IWebBrowserWindow.h"
#include "WebBrowserViewport.h"
#include "IWebBrowserAdapter.h"

#if PLATFORM_ANDROID && USE_ANDROID_JNI
#	include "Android/AndroidWebBrowserWindow.h"
#elif WITH_CEF3
#	include "CEF/CEFWebBrowserWindow.h"
#elif PLATFORM_IOS || PLATFORM_MAC
#	include "Apple/ApplePlatformWebBrowser.h"
#elif PLATFORM_SPECIFIC_WEB_BROWSER
#	include COMPILED_PLATFORM_HEADER(PlatformWebBrowser.h)
#else
#	define DUMMY_WEB_BROWSER 1
#endif

#define LOCTEXT_NAMESPACE "WebBrowser"

SWebBrowserView::SWebBrowserView()
{
}

SWebBrowserView::~SWebBrowserView()
{
	if (BrowserWindow.IsValid())
	{
		BrowserWindow->OnDocumentStateChanged().RemoveAll(this);
		BrowserWindow->OnNeedsRedraw().RemoveAll(this);
		BrowserWindow->OnTitleChanged().RemoveAll(this);
		BrowserWindow->OnUrlChanged().RemoveAll(this);
		BrowserWindow->OnToolTip().RemoveAll(this);
		BrowserWindow->OnShowPopup().RemoveAll(this);
		BrowserWindow->OnDismissPopup().RemoveAll(this);

		BrowserWindow->OnShowDialog().Unbind();
		BrowserWindow->OnDismissAllDialogs().Unbind();
		BrowserWindow->OnCreateWindow().Unbind();
		BrowserWindow->OnCloseWindow().Unbind();
		BrowserWindow->OnSuppressContextMenu().Unbind();
		BrowserWindow->OnDragWindow().Unbind();
		BrowserWindow->OnConsoleMessage().Unbind();

		if (BrowserWindow->OnBeforeBrowse().IsBoundToObject(this))
		{
			BrowserWindow->OnBeforeBrowse().Unbind();
		}

		if (BrowserWindow->OnLoadUrl().IsBoundToObject(this))
		{
			BrowserWindow->OnLoadUrl().Unbind();
		}

		if (BrowserWindow->OnBeforePopup().IsBoundToObject(this))
		{
			BrowserWindow->OnBeforePopup().Unbind();
		}
	}

	TSharedPtr<SWindow> SlateParentWindow = SlateParentWindowPtr.Pin();
	if (SlateParentWindow.IsValid())
	{
		SlateParentWindow->GetOnWindowDeactivatedEvent().RemoveAll(this);
	}

	if (SlateParentWindow.IsValid())
	{
		SlateParentWindow->GetOnWindowActivatedEvent().RemoveAll(this);
	}
}

void SWebBrowserView::Construct(const FArguments& InArgs, const TSharedPtr<IWebBrowserWindow>& InWebBrowserWindow)
{
	OnLoadCompleted = InArgs._OnLoadCompleted;
	OnLoadError = InArgs._OnLoadError;
	OnLoadStarted = InArgs._OnLoadStarted;
	OnTitleChanged = InArgs._OnTitleChanged;
	OnUrlChanged = InArgs._OnUrlChanged;
	OnBeforeNavigation = InArgs._OnBeforeNavigation;
	OnLoadUrl = InArgs._OnLoadUrl;
	OnShowDialog = InArgs._OnShowDialog;
	OnDismissAllDialogs = InArgs._OnDismissAllDialogs;
	OnBeforePopup = InArgs._OnBeforePopup;
	OnCreateWindow = InArgs._OnCreateWindow;
	OnCloseWindow = InArgs._OnCloseWindow;
	OnFloatingCloseButtonPressed = InArgs._OnFloatingCloseButtonPressed;
	AddressBarUrl = FText::FromString(InArgs._InitialURL);
	PopupMenuMethod = InArgs._PopupMenuMethod;
	OnUnhandledKeyDown = InArgs._OnUnhandledKeyDown;
	OnUnhandledKeyUp = InArgs._OnUnhandledKeyUp;
	OnUnhandledKeyChar = InArgs._OnUnhandledKeyChar;
	OnConsoleMessage = InArgs._OnConsoleMessage;

	BrowserWindow = InWebBrowserWindow;
	if(!BrowserWindow.IsValid())
	{

		static bool AllowCEF = !FParse::Param(FCommandLine::Get(), TEXT("nocef"));
		bool bBrowserEnabled = true;
		GConfig->GetBool(TEXT("Browser"), TEXT("bEnabled"), bBrowserEnabled, GEngineIni);
		if (AllowCEF && bBrowserEnabled)
		{
			FCreateBrowserWindowSettings Settings;
			Settings.InitialURL = InArgs._InitialURL;
			Settings.bUseTransparency = InArgs._SupportsTransparency;
			Settings.bInterceptLoadRequests = InArgs._InterceptLoadRequests;
			Settings.bThumbMouseButtonNavigation = InArgs._SupportsThumbMouseButtonNavigation;
			Settings.ContentsToLoad = InArgs._ContentsToLoad;
			Settings.UserAgentApplication = InArgs._UserAgentApplication;
			Settings.bShowErrorMessage = InArgs._ShowErrorMessage;
			Settings.BackgroundColor = InArgs._BackgroundColor;
			Settings.BrowserFrameRate = InArgs._BrowserFrameRate;
			Settings.Context = InArgs._ContextSettings;
			Settings.AltRetryDomains = InArgs._AltRetryDomains;
			Settings.bEnablePaymentRequest = InArgs._EnablePaymentRequest;

			// IWebBrowserModule::Get() was already callled in WebBrowserWidgetModule.cpp so we don't need to force the load again here
			if (IWebBrowserModule::IsAvailable() && IWebBrowserModule::Get().IsWebModuleAvailable())
			{
				BrowserWindow = IWebBrowserModule::Get().GetSingleton()->CreateBrowserWindow(Settings);
			}
		}
	}

	SlateParentWindowPtr = InArgs._ParentWindow;

	if (BrowserWindow.IsValid())
	{
#ifndef DUMMY_WEB_BROWSER
		// The inner widget creation is handled by the WebBrowserWindow implementation.
		const auto& BrowserWidgetRef = static_cast<FWebBrowserWindow*>(BrowserWindow.Get())->CreateWidget();
		ChildSlot
		[
			BrowserWidgetRef
		];
		BrowserWidget = BrowserWidgetRef;
#endif

		if(OnCreateWindow.IsBound())
		{
			BrowserWindow->OnCreateWindow().BindSP(this, &SWebBrowserView::HandleCreateWindow);
		}

		if(OnCloseWindow.IsBound())
		{
			BrowserWindow->OnCloseWindow().BindSP(this, &SWebBrowserView::HandleCloseWindow);
		}

		if (OnFloatingCloseButtonPressed.IsBound())
		{
			BrowserWindow->OnFloatingCloseButtonPressed().BindSP(this, &SWebBrowserView::HandleFloatingCloseButtonPressed);
		}

		BrowserWindow->OnDocumentStateChanged().AddSP(this, &SWebBrowserView::HandleBrowserWindowDocumentStateChanged);
		BrowserWindow->OnNeedsRedraw().AddSP(this, &SWebBrowserView::HandleBrowserWindowNeedsRedraw);
		BrowserWindow->OnTitleChanged().AddSP(this, &SWebBrowserView::HandleTitleChanged);
		BrowserWindow->OnUrlChanged().AddSP(this, &SWebBrowserView::HandleUrlChanged);
		BrowserWindow->OnToolTip().AddSP(this, &SWebBrowserView::HandleToolTip);
		OnCreateToolTip = InArgs._OnCreateToolTip;

		if (!BrowserWindow->OnBeforeBrowse().IsBound())
		{
			BrowserWindow->OnBeforeBrowse().BindSP(this, &SWebBrowserView::HandleBeforeNavigation);
		}
		else
		{
			check(!OnBeforeNavigation.IsBound());
		}

		if (!BrowserWindow->OnLoadUrl().IsBound())
		{
			BrowserWindow->OnLoadUrl().BindSP(this, &SWebBrowserView::HandleLoadUrl);
		}
		else
		{
			check(!OnLoadUrl.IsBound());
		}

		if (!BrowserWindow->OnBeforePopup().IsBound())
		{
			BrowserWindow->OnBeforePopup().BindSP(this, &SWebBrowserView::HandleBeforePopup);
		}
		else
		{
			check(!OnBeforePopup.IsBound());
		}

		if (!BrowserWindow->OnUnhandledKeyDown().IsBound())
		{
			BrowserWindow->OnUnhandledKeyDown().BindSP(this, &SWebBrowserView::UnhandledKeyDown);
		}

		if (!BrowserWindow->OnUnhandledKeyUp().IsBound())
		{
			BrowserWindow->OnUnhandledKeyUp().BindSP(this, &SWebBrowserView::UnhandledKeyUp);
		}

		if (!BrowserWindow->OnUnhandledKeyChar().IsBound())
		{
			BrowserWindow->OnUnhandledKeyChar().BindSP(this, &SWebBrowserView::UnhandledKeyChar);
		}
		
		BrowserWindow->OnShowDialog().BindSP(this, &SWebBrowserView::HandleShowDialog);
		BrowserWindow->OnDismissAllDialogs().BindSP(this, &SWebBrowserView::HandleDismissAllDialogs);
		BrowserWindow->OnShowPopup().AddSP(this, &SWebBrowserView::HandleShowPopup);
		BrowserWindow->OnDismissPopup().AddSP(this, &SWebBrowserView::HandleDismissPopup);

		BrowserWindow->OnSuppressContextMenu().BindSP(this, &SWebBrowserView::HandleSuppressContextMenu);


		OnSuppressContextMenu = InArgs._OnSuppressContextMenu;

		BrowserWindow->OnDragWindow().BindSP(this, &SWebBrowserView::HandleDrag);
		OnDragWindow = InArgs._OnDragWindow;

		if (!BrowserWindow->OnConsoleMessage().IsBound())
		{
			BrowserWindow->OnConsoleMessage().BindSP(this, &SWebBrowserView::HandleConsoleMessage);
		}

		BrowserViewport = MakeShareable(new FWebBrowserViewport(BrowserWindow));
#if WITH_CEF3 || PLATFORM_APPLE
		BrowserWidget->SetViewportInterface(BrowserViewport.ToSharedRef());
#endif
		// If we could not obtain the parent window during widget construction, we'll defer and keep trying.
		SetupParentWindowHandlers();
	}
	else
	{
		OnLoadError.ExecuteIfBound();
	}
}

int32 SWebBrowserView::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	if (!SlateParentWindowPtr.IsValid())
	{
		SWebBrowserView* MutableThis = const_cast<SWebBrowserView*>(this);
		MutableThis->SetupParentWindowHandlers();
	}

	int32 Layer = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	
	// Cache a reference to our parent window, if we didn't already reference it.
	SWindow* ParentWindow = OutDrawElements.GetPaintWindow();

	TSharedRef<SWindow> SlateParentWindowRef = StaticCastSharedRef<SWindow>(ParentWindow->AsShared());
	
	if (!SlateParentWindowPtr.IsValid() || SlateParentWindowPtr != SlateParentWindowRef)
	{
		SlateParentWindowPtr = SlateParentWindowRef;
		if (BrowserWindow.IsValid())
		{
			BrowserWindow->SetParentWindow(SlateParentWindowRef);
		}
	}

	return Layer;
}

void SWebBrowserView::HandleWindowDeactivated()
{
	if (BrowserViewport.IsValid())
	{
		BrowserViewport->OnFocusLost(FFocusEvent());
	}
}

FReply SWebBrowserView::OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent)
{
	FReply Reply = FReply::Handled();
	if (InFocusEvent.GetCause() != EFocusCause::Cleared)
	{
		if (BrowserWidget.IsValid())
		{
			Reply.SetUserFocus(BrowserWidget.ToSharedRef(), InFocusEvent.GetCause());
		}
		else
		{
			Reply.SetUserFocus(this->AsShared());
		}
		if (BrowserWindow.IsValid())
		{
			BrowserWindow->OnFocus(true, false);
		}
	}
	return Reply;
}

void SWebBrowserView::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (BrowserWindow.IsValid())
	{
		BrowserWindow->OnDragEnter(MyGeometry, DragDropEvent);
	}
}

FReply SWebBrowserView::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (BrowserWindow.IsValid())
	{
		return BrowserWindow->OnDragOver(MyGeometry, DragDropEvent);
	}
	return FReply::Unhandled();
}

void SWebBrowserView::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	if (BrowserWindow.IsValid())
	{
		BrowserWindow->OnDragLeave(DragDropEvent);
	}
}

FReply SWebBrowserView::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (BrowserWindow.IsValid())
	{
		return BrowserWindow->OnDrop(MyGeometry, DragDropEvent);
	}
	return FReply::Unhandled();
}

void SWebBrowserView::HandleWindowActivated()
{
	if (BrowserViewport.IsValid())
	{
		if (HasAnyUserFocusOrFocusedDescendants())
		{
			BrowserViewport->OnFocusReceived(FFocusEvent());
		}	
	}
}

void SWebBrowserView::LoadURL(FString NewURL)
{
	AddressBarUrl = FText::FromString(NewURL);
	if (BrowserWindow.IsValid())
	{
		BrowserWindow->LoadURL(MoveTemp(NewURL));
	}
}

void SWebBrowserView::LoadString(FString Contents, FString DummyURL)
{
	if (BrowserWindow.IsValid())
	{
		BrowserWindow->LoadString(MoveTemp(Contents), MoveTemp(DummyURL));
	}
}

void SWebBrowserView::Reload()
{
	if (BrowserWindow.IsValid())
	{
		BrowserWindow->Reload();
	}
}

void SWebBrowserView::StopLoad()
{
	if (BrowserWindow.IsValid())
	{
		BrowserWindow->StopLoad();
	}
}

FText SWebBrowserView::GetTitleText() const
{
	if (BrowserWindow.IsValid())
	{
		return FText::FromString(BrowserWindow->GetTitle());
	}
	return LOCTEXT("InvalidWindow", "Browser Window is not valid/supported");
}

FString SWebBrowserView::GetUrl() const
{
	if (BrowserWindow.IsValid())
	{
		return BrowserWindow->GetUrl();
	}

	return FString();
}

FText SWebBrowserView::GetAddressBarUrlText() const
{
	if(BrowserWindow.IsValid())
	{
		return AddressBarUrl;
	}
	return FText::GetEmpty();
}

bool SWebBrowserView::IsLoaded() const
{
	if (BrowserWindow.IsValid())
	{
		return (BrowserWindow->GetDocumentLoadingState() == EWebBrowserDocumentState::Completed);
	}

	return false;
}

bool SWebBrowserView::IsLoading() const
{
	if (BrowserWindow.IsValid())
	{
		return (BrowserWindow->GetDocumentLoadingState() == EWebBrowserDocumentState::Loading);
	}

	return false;
}

bool SWebBrowserView::CanGoBack() const
{
	if (BrowserWindow.IsValid())
	{
		return BrowserWindow->CanGoBack();
	}
	return false;
}

void SWebBrowserView::GoBack()
{
	if (BrowserWindow.IsValid())
	{
		BrowserWindow->GoBack();
	}
}

bool SWebBrowserView::CanGoForward() const
{
	if (BrowserWindow.IsValid())
	{
		return BrowserWindow->CanGoForward();
	}
	return false;
}

void SWebBrowserView::GoForward()
{
	if (BrowserWindow.IsValid())
	{
		BrowserWindow->GoForward();
	}
}

bool SWebBrowserView::IsInitialized() const
{
	return BrowserWindow.IsValid() &&  BrowserWindow->IsInitialized();
}

void SWebBrowserView::SetupParentWindowHandlers()
{
	if (!SlateParentWindowPtr.IsValid())
	{
		SlateParentWindowPtr = FSlateApplication::Get().FindWidgetWindow(SharedThis(this));

		TSharedPtr<SWindow> SlateParentWindow = SlateParentWindowPtr.Pin();
		if (SlateParentWindow.IsValid() && BrowserWindow.IsValid())
		{
			if (!SlateParentWindow->GetOnWindowDeactivatedEvent().IsBoundToObject(this))
			{
				SlateParentWindow->GetOnWindowDeactivatedEvent().AddSP(this, &SWebBrowserView::HandleWindowDeactivated);
			}

			if (!SlateParentWindow->GetOnWindowActivatedEvent().IsBoundToObject(this))
			{
				SlateParentWindow->GetOnWindowActivatedEvent().AddSP(this, &SWebBrowserView::HandleWindowActivated);
			}

			BrowserWindow->SetParentWindow(SlateParentWindow);
		}
	}
}

void SWebBrowserView::HandleBrowserWindowDocumentStateChanged(EWebBrowserDocumentState NewState)
{
	switch (NewState)
	{
	case EWebBrowserDocumentState::Completed:
		{
			if (BrowserWindow.IsValid())
			{
				for (auto Adapter : Adapters)
				{
					Adapter->ConnectTo(BrowserWindow.ToSharedRef());
				}
			}

			OnLoadCompleted.ExecuteIfBound();
		}
		break;

	case EWebBrowserDocumentState::Error:
		OnLoadError.ExecuteIfBound();
		break;

	case EWebBrowserDocumentState::Loading:
		OnLoadStarted.ExecuteIfBound();
		break;
	}
}

void SWebBrowserView::HandleBrowserWindowNeedsRedraw()
{
	if (FSlateApplication::IsInitialized() && FSlateApplication::Get().IsSlateAsleep())
	{
		// Tell slate that the widget needs to wake up for one frame to get redrawn
		RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateLambda([this](double InCurrentTime, float InDeltaTime) { return EActiveTimerReturnType::Stop; }));
	}
}

void SWebBrowserView::HandleTitleChanged( FString NewTitle )
{
	const FText NewTitleText = FText::FromString(NewTitle);
	OnTitleChanged.ExecuteIfBound(NewTitleText);
}

void SWebBrowserView::HandleUrlChanged( FString NewUrl )
{
	AddressBarUrl = FText::FromString(NewUrl);
	
	AsyncTask(ENamedThreads::Type::GameThread, [this] 
    {
		OnUrlChanged.ExecuteIfBound(AddressBarUrl);
	});
}

void SWebBrowserView::CloseBrowser()
{
	BrowserWindow->CloseBrowser(true /*force*/, true /*block until closed*/);
}

void SWebBrowserView::ShowFloatingCloseButton(bool bShow, bool bDraggable)
{
	BrowserWindow->ShowFloatingCloseButton(bShow, bDraggable);
}

TSharedPtr<IWebBrowserWindow> SWebBrowserView::GetBrowserWindow() const
{
	return BrowserWindow;
}

void SWebBrowserView::HandleToolTip(FString ToolTipText)
{
	if(ToolTipText.IsEmpty())
	{
		FSlateApplication::Get().CloseToolTip();
		SetToolTip(nullptr);
	}
	else if (OnCreateToolTip.IsBound())
	{
		SetToolTip(OnCreateToolTip.Execute(FText::FromString(ToolTipText)));
		FSlateApplication::Get().UpdateToolTip(true);
	}
	else
	{
		SetToolTipText(FText::FromString(ToolTipText));
		FSlateApplication::Get().UpdateToolTip(true);
	}
}

bool SWebBrowserView::HandleBeforeNavigation(const FString& Url, const FWebNavigationRequest& Request)
{
	if(OnBeforeNavigation.IsBound())
	{
		return OnBeforeNavigation.Execute(Url, Request);
	}
	return false;
}

bool SWebBrowserView::HandleLoadUrl(const FString& Method, const FString& Url, FString& OutResponse)
{
	if(OnLoadUrl.IsBound())
	{
		return OnLoadUrl.Execute(Method, Url, OutResponse);
	}
	return false;
}

EWebBrowserDialogEventResponse SWebBrowserView::HandleShowDialog(const TWeakPtr<IWebBrowserDialog>& DialogParams)
{
	if(OnShowDialog.IsBound())
	{
		return OnShowDialog.Execute(DialogParams);
	}
	return EWebBrowserDialogEventResponse::Unhandled;
}

void SWebBrowserView::HandleDismissAllDialogs()
{
	OnDismissAllDialogs.ExecuteIfBound();
}


bool SWebBrowserView::HandleBeforePopup(FString URL, FString Target)
{
	if (OnBeforePopup.IsBound())
	{
		return OnBeforePopup.Execute(URL, Target);
	}

	return false;
}

void SWebBrowserView::ExecuteJavascript(const FString& ScriptText)
{
	if (BrowserWindow.IsValid())
	{
		BrowserWindow->ExecuteJavascript(ScriptText);
	}
}

void SWebBrowserView::GetSource(TFunction<void (const FString&)> Callback) const
{
	if (BrowserWindow.IsValid())
	{
		BrowserWindow->GetSource(Callback);
	}
}


bool SWebBrowserView::HandleCreateWindow(const TWeakPtr<IWebBrowserWindow>& NewBrowserWindow, const TWeakPtr<IWebBrowserPopupFeatures>& PopupFeatures)
{
	if(OnCreateWindow.IsBound())
	{
		return OnCreateWindow.Execute(NewBrowserWindow, PopupFeatures);
	}
	return false;
}

bool SWebBrowserView::HandleCloseWindow(const TWeakPtr<IWebBrowserWindow>& NewBrowserWindow)
{
	if(OnCloseWindow.IsBound())
	{
		return OnCloseWindow.Execute(NewBrowserWindow);
	}
	return false;
}

void SWebBrowserView::HandleFloatingCloseButtonPressed()
{
	if (OnFloatingCloseButtonPressed.IsBound())
	{
		OnFloatingCloseButtonPressed.Execute();
	}
}

void SWebBrowserView::BindUObject(const FString& Name, UObject* Object, bool bIsPermanent)
{
	if (BrowserWindow.IsValid())
	{
		BrowserWindow->BindUObject(Name, Object, bIsPermanent);
	}
}

void SWebBrowserView::UnbindUObject(const FString& Name, UObject* Object, bool bIsPermanent)
{
	if (BrowserWindow.IsValid())
	{
		BrowserWindow->UnbindUObject(Name, Object, bIsPermanent);
	}
}

void SWebBrowserView::BindAdapter(const TSharedRef<IWebBrowserAdapter>& Adapter)
{
	Adapters.Add(Adapter);
	if (BrowserWindow.IsValid())
	{
		Adapter->ConnectTo(BrowserWindow.ToSharedRef());
	}
}

void SWebBrowserView::UnbindAdapter(const TSharedRef<IWebBrowserAdapter>& Adapter)
{
	Adapters.Remove(Adapter);
	if (BrowserWindow.IsValid())
	{
		Adapter->DisconnectFrom(BrowserWindow.ToSharedRef());
	}
}

void SWebBrowserView::BindInputMethodSystem(ITextInputMethodSystem* TextInputMethodSystem)
{
	if (BrowserWindow.IsValid())
	{
		BrowserWindow->BindInputMethodSystem(TextInputMethodSystem);
	}
}

void SWebBrowserView::UnbindInputMethodSystem()
{
	if (BrowserWindow.IsValid())
	{
		BrowserWindow->UnbindInputMethodSystem();
	}
}

void SWebBrowserView::HandleShowPopup(const FIntRect& PopupSize)
{
	check(!PopupMenuPtr.IsValid())

	TSharedPtr<SViewport> MenuContent;
	SAssignNew(MenuContent, SViewport)
				.ViewportSize(PopupSize.Size())
				.EnableGammaCorrection(false)
				.EnableBlending(false)
				.IgnoreTextureAlpha(true)
				.Visibility(EVisibility::Visible);
		MenuViewport = MakeShareable(new FWebBrowserViewport(BrowserWindow, true));
	MenuContent->SetViewportInterface(MenuViewport.ToSharedRef());

	FWidgetPath WidgetPath;
	FSlateApplication::Get().GeneratePathToWidgetUnchecked(SharedThis(this), WidgetPath);

	if (WidgetPath.IsValid())
	{
		TSharedRef< SWidget > MenuContentRef = MenuContent.ToSharedRef();
		const FGeometry& BrowserGeometry = WidgetPath.Widgets.Last().Geometry;
		const FVector2D NewPosition = BrowserGeometry.LocalToAbsolute(PopupSize.Min);

		// Open the pop-up. The popup method will be queried from the widget path passed in.
		TSharedPtr<IMenu> NewMenu = FSlateApplication::Get().PushMenu(SharedThis(this), WidgetPath, MenuContentRef, NewPosition, FPopupTransitionEffect( FPopupTransitionEffect::ComboButton ), false);

		if (NewMenu.IsValid())
		{
			NewMenu->GetOnMenuDismissed().AddSP(this, &SWebBrowserView::HandleMenuDismissed);
			PopupMenuPtr = NewMenu;
		}
	}

}

void SWebBrowserView::HandleMenuDismissed(TSharedRef<IMenu>)
{
	PopupMenuPtr.Reset();
}

void SWebBrowserView::HandleDismissPopup()
{
	if (PopupMenuPtr.IsValid())
	{
		PopupMenuPtr.Pin()->Dismiss();
		FSlateApplication::Get().SetKeyboardFocus(SharedThis(this), EFocusCause::SetDirectly);
	}
}

bool SWebBrowserView::HandleSuppressContextMenu()
{
	if (OnSuppressContextMenu.IsBound())
	{
		return OnSuppressContextMenu.Execute();
	}

	return false;
}

bool SWebBrowserView::HandleDrag(const FPointerEvent& MouseEvent)
{
	if (OnDragWindow.IsBound())
	{
		return OnDragWindow.Execute(MouseEvent);
	}
	return false;
}

bool SWebBrowserView::UnhandledKeyDown(const FKeyEvent& KeyEvent)
{
	if (OnUnhandledKeyDown.IsBound())
	{
		return OnUnhandledKeyDown.Execute(KeyEvent);
	}
	return false;
}

bool SWebBrowserView::UnhandledKeyUp(const FKeyEvent& KeyEvent)
{
	if (OnUnhandledKeyUp.IsBound())
	{
		return OnUnhandledKeyUp.Execute(KeyEvent);
	}
	return false;
}

bool SWebBrowserView::UnhandledKeyChar(const FCharacterEvent& CharacterEvent)
{
	if (OnUnhandledKeyChar.IsBound())
	{
		return OnUnhandledKeyChar.Execute(CharacterEvent);
	}
	return false;
}

void SWebBrowserView::SetParentWindow(TSharedPtr<SWindow> Window)
{
	SetupParentWindowHandlers();
	if (BrowserWindow.IsValid())
	{
		BrowserWindow->SetParentWindow(Window);
	}
}

void SWebBrowserView::SetBrowserKeyboardFocus()
{
	BrowserWindow->OnFocus(HasAnyUserFocusOrFocusedDescendants(), false);
}

void SWebBrowserView::HandleConsoleMessage(const FString& Message, const FString& Source, int32 Line, EWebBrowserConsoleLogSeverity Serverity)
{
	OnConsoleMessage.ExecuteIfBound(Message, Source, Line, Serverity);
}

#undef LOCTEXT_NAMESPACE
