// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "UObject/GCObject.h"
#include "IWebBrowserWindow.h"
#include "IWebBrowserAdapter.h"

class FDefaultWebBrowserAdapter
	: public IWebBrowserAdapter
	, public FGCObject

{
public:

	virtual FString GetName() const override
	{
		return Name;
	}

	virtual bool IsPermanent() const override
	{
		return bIsPermanent;
	}

	virtual void ConnectTo(const TSharedRef<IWebBrowserWindow>& BrowserWindow) override
	{
		if (JSBridge != nullptr)
		{
			BrowserWindow->BindUObject(Name, JSBridge, bIsPermanent);
		}

		if (!ConnectScriptText.IsEmpty())
		{
			BrowserWindow->ExecuteJavascript(ConnectScriptText);
		}
	}

	virtual void DisconnectFrom(const TSharedRef<IWebBrowserWindow>& BrowserWindow) override
	{
		if (!DisconnectScriptText.IsEmpty())
		{
			BrowserWindow->ExecuteJavascript(DisconnectScriptText);
		}

		if (JSBridge != nullptr)
		{
			BrowserWindow->UnbindUObject(Name, JSBridge, bIsPermanent);
		}
	}

	// FGCObject API
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		if (JSBridge != nullptr)
		{
			Collector.AddReferencedObject(JSBridge);
		}
	}
	virtual FString GetReferencerName() const override
	{
		return TEXT("FDefaultWebBrowserAdapter");
	}

private:

	FDefaultWebBrowserAdapter(
		const FString InName,
		const FString InConnectScriptText,
		const FString InDisconnectScriptText,
		UObject* InJSBridge,
		const bool InIsPermanent)
		: Name(InName)
		, ConnectScriptText(InConnectScriptText)
		, DisconnectScriptText(InDisconnectScriptText)
		, JSBridge(InJSBridge)
		, bIsPermanent(InIsPermanent)
	{ }

private:

	const FString Name;
	const FString ConnectScriptText;
	const FString DisconnectScriptText;

	TObjectPtr<UObject> JSBridge;

	const bool bIsPermanent;

	friend FWebBrowserAdapterFactory;
};

TSharedRef<IWebBrowserAdapter> FWebBrowserAdapterFactory::Create(const FString& Name, UObject* JSBridge, bool IsPermanent)
{
	return MakeShareable(new FDefaultWebBrowserAdapter(Name, FString(), FString(), JSBridge, IsPermanent));
}

TSharedRef<IWebBrowserAdapter> FWebBrowserAdapterFactory::Create(const FString& Name, UObject* JSBridge, bool IsPermanent, const FString& ConnectScriptText, const FString& DisconnectScriptText)
{
	return MakeShareable(new FDefaultWebBrowserAdapter(Name, ConnectScriptText, DisconnectScriptText, JSBridge, IsPermanent));
}
