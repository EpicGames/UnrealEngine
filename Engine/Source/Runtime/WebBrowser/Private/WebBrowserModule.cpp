// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebBrowserModule.h"
#include "WebBrowserLog.h"
#include "WebBrowserSingleton.h"
#include "Misc/App.h"
#include "Misc/EngineVersion.h"
#include "Misc/Paths.h"
#if WITH_CEF3
#	include "CEF3Utils.h"
#	if CEF_VERSION_MAJOR >= 147
#		include "include/cef_version_info.h"
#	else
#		include "include/cef_version.h"
#	endif
#	if PLATFORM_MAC
#		include "include/wrapper/cef_library_loader.h"
#	endif
#endif

DEFINE_LOG_CATEGORY(LogWebBrowser);

static FWebBrowserSingleton* WebBrowserSingleton = nullptr;

FString IWebBrowserModule::MakeUserAgentApplication(const FString& ApplicationName, const FString& ApplicationVersion)
{
	return FString::Printf(TEXT("%s/%s UnrealEngine/%s"), *ApplicationName, ApplicationVersion.IsEmpty() ? TEXT("1.0") : *ApplicationVersion, *FEngineVersion::Current().ToString());
}

FWebBrowserInitSettings::FWebBrowserInitSettings()
	: ProductVersion(IWebBrowserModule::MakeUserAgentApplication(FApp::GetProjectName(), FApp::GetBuildVersion()))
{
}

class FWebBrowserModule : public IWebBrowserModule
{
private:
	// IModuleInterface Interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

public:
	virtual bool IsWebModuleAvailable() const override;
	virtual IWebBrowserSingleton* GetSingleton() override;
	virtual bool CustomInitialize(const FWebBrowserInitSettings& WebBrowserInitSettings) override;

private:
#if WITH_CEF3
	bool bLoadedCEFModule = false;
	bool bMatchingCEFVersion = false;
#if PLATFORM_MAC
	// Dynamically load the CEF framework library.
	CefScopedLibraryLoader *CEFLibraryLoader = nullptr;
#endif
#endif
};

IMPLEMENT_MODULE( FWebBrowserModule, WebBrowser );

void FWebBrowserModule::StartupModule()
{
#if WITH_CEF3
	if (!IsRunningCommandlet())
	{
		CEF3Utils::BackupCEF3Logfile(FPaths::ProjectLogDir());
	}
	bLoadedCEFModule = CEF3Utils::LoadCEF3Modules(true);
	if (!bLoadedCEFModule)
	{
		return;
	}
#if PLATFORM_MAC
	// Dynamically load the CEF framework library into this dylibs memory space.
	// CEF now loads function pointers at runtime so we need this to be dylib specific.
	CEFLibraryLoader = new CefScopedLibraryLoader();
	if (!CEFLibraryLoader->LoadInMain(TCHAR_TO_ANSI(*CEF3Utils::GetCEF3ModulePath())))
	{
		UE_LOGF(LogWebBrowser, Error, "Chromium loader initialization failed");
		return;
	}
#endif // PLATFORM_MAC
	const int CefVersionMajor = cef_version_info(0);
	const int CefVersionMinor = cef_version_info(1);
	const int CefVersionPatch = cef_version_info(2);
	const int CefCommitNumber = cef_version_info(3);
	UE_LOGF(LogWebBrowser, Log, "Loaded CEF3 version %i.%i.%i.%i from %ls", CefVersionMajor, CefVersionMinor, CefVersionPatch, CefCommitNumber, *CEF3Utils::GetCEF3ModulePath());
	if (CefVersionMajor != CEF_VERSION_MAJOR || CefVersionMinor != CEF_VERSION_MINOR || CefVersionPatch != CEF_VERSION_PATCH || CefCommitNumber != CEF_COMMIT_NUMBER)
	{
		UE_LOGF(LogWebBrowser, Warning, "CEF3 loaded version mismatch! Module was built against %i.%i.%i.%i, check if library loading path is correct", CEF_VERSION_MAJOR, CEF_VERSION_MINOR, CEF_VERSION_PATCH, CEF_COMMIT_NUMBER);
	}
	else
	{
		bMatchingCEFVersion = true;
	}
#endif
}

void FWebBrowserModule::ShutdownModule()
{
	if (WebBrowserSingleton != nullptr)
	{
		delete WebBrowserSingleton;
		WebBrowserSingleton = nullptr;
	}

#if WITH_CEF3
	CEF3Utils::UnloadCEF3Modules();
	bLoadedCEFModule = false;
	bMatchingCEFVersion = false;
#if PLATFORM_MAC
	delete CEFLibraryLoader;
	CEFLibraryLoader = nullptr;
#endif // PLATFORM_MAC
#endif
}

bool FWebBrowserModule::CustomInitialize(const FWebBrowserInitSettings& WebBrowserInitSettings)
{
	if (WebBrowserSingleton == nullptr)
	{
		WebBrowserSingleton = new FWebBrowserSingleton(WebBrowserInitSettings);
		return true;
	}
	return false;
}

IWebBrowserSingleton* FWebBrowserModule::GetSingleton()
{
	if (WebBrowserSingleton == nullptr)
	{
		WebBrowserSingleton = new FWebBrowserSingleton(FWebBrowserInitSettings());
	}
	return WebBrowserSingleton;
}


bool FWebBrowserModule::IsWebModuleAvailable() const
{
#if WITH_CEF3
	return bLoadedCEFModule && bMatchingCEFVersion;
#else
	return true;
#endif
}

