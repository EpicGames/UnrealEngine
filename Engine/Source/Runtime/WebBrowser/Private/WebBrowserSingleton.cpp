// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebBrowserSingleton.h"
#include "Misc/Paths.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Internationalization/Culture.h"
#include "Misc/App.h"
#include "WebBrowserModule.h"
#include "Misc/EngineVersion.h"
#include "Framework/Application/SlateApplication.h"
#include "IWebBrowserCookieManager.h"
#include "WebBrowserLog.h"
#include "HAL/Platform.h"

#if PLATFORM_APPLE
#include "Apple/ScopeAutoreleasePool.h"
#endif

#if WITH_CEF3
#include "CEF3Utils.h"
#include "Misc/ScopeLock.h"
#include "Async/Async.h"
#include "HAL/PlatformApplicationMisc.h"
#include "CEF/CEFBrowserApp.h"
#include "CEF/CEFBrowserHandler.h"
#include "CEF/CEFWebBrowserWindow.h"
#include "CEF/CEFSchemeHandler.h"
#include "CEF/CEFResourceContextHandler.h"
#include "CEF/CEFBrowserClosureTask.h"
#	if PLATFORM_WINDOWS
#		include "Windows/AllowWindowsPlatformTypes.h"
#	endif
#	pragma push_macro("OVERRIDE")
#		undef OVERRIDE // cef headers provide their own OVERRIDE macro
THIRD_PARTY_INCLUDES_START
#if PLATFORM_APPLE
PRAGMA_DISABLE_DEPRECATION_WARNINGS
#endif
#		include "include/cef_app.h"
#		include "include/cef_version.h"
#if PLATFORM_APPLE
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
THIRD_PARTY_INCLUDES_END
#	pragma pop_macro("OVERRIDE")
#	if PLATFORM_WINDOWS
#		include "Windows/HideWindowsPlatformTypes.h"
#		include <delayimp.h>
#	endif
#endif

#if BUILD_EMBEDDED_APP
#	include "Native/NativeWebBrowserProxy.h"
#endif

#if PLATFORM_ANDROID && USE_ANDROID_JNI
#	include "Android/AndroidWebBrowserWindow.h"
#	include <Android/AndroidCookieManager.h>
#elif PLATFORM_IOS || PLATFORM_MAC
#	include <Apple/ApplePlatformWebBrowser.h>
#	include <Apple/AppleCookieManager.h>
#elif PLATFORM_SPECIFIC_WEB_BROWSER
#	include COMPILED_PLATFORM_HEADER(PlatformWebBrowser.h)
#endif

// Define some platform-dependent file locations
#if WITH_CEF3
#	define CEF_VERSION_DIR TEXT("/") TEXT(CEF_VERSION)
#	if CEF3_USE_EXPERIMENTAL_VERSION
#		define CEF_VERSION_SUFFIX TEXT("")
#	else
#		define CEF_VERSION_SUFFIX TEXT("+v2")
#	endif
#	define CEF3_BIN_DIR TEXT("Binaries/ThirdParty/CEF3")
#	if PLATFORM_WINDOWS && PLATFORM_CPU_ARM_FAMILY
#		if PLATFORM_WINDOWS_ARM64EC
#			define CEF3_RESOURCES_DIR CEF3_BIN_DIR TEXT("/Win64") CEF_VERSION_DIR CEF_VERSION_SUFFIX TEXT("/Resources")
#			define CEF3_SUBPROCES_EXE TEXT("Binaries/Win64/EpicWebHelper.exe")
#		else
#			define CEF3_RESOURCES_DIR CEF3_BIN_DIR TEXT("/WinArm64") CEF_VERSION_DIR TEXT("/Resources")
#			define CEF3_SUBPROCES_EXE TEXT("Binaries/Win64/EpicWebHelperarm64.exe")
#		endif
#	elif PLATFORM_WINDOWS && !PLATFORM_CPU_ARM_FAMILY
#		define CEF3_RESOURCES_DIR CEF3_BIN_DIR TEXT("/Win64") CEF_VERSION_DIR CEF_VERSION_SUFFIX TEXT("/Resources")
#		define CEF3_SUBPROCES_EXE TEXT("Binaries/Win64/EpicWebHelper.exe")
#	elif PLATFORM_MAC
#		define CEF3_FRAMEWORK_DIR CEF3_BIN_DIR TEXT("/Mac") CEF_VERSION_DIR TEXT("/Chromium Embedded Framework.framework")
#		define CEF3_RESOURCES_DIR CEF3_FRAMEWORK_DIR TEXT("/Resources")
#		define CEF3_SUBPROCES_EXE TEXT("Binaries/Mac/EpicWebHelper")
#	elif PLATFORM_LINUX // @todo Linux
#		define CEF3_RESOURCES_DIR CEF3_BIN_DIR TEXT("/Linux") CEF_VERSION_DIR TEXT("/Resources")
#		define CEF3_SUBPROCES_EXE TEXT("Binaries/Linux/EpicWebHelper")
#	endif
	// Caching is enabled by default.
#	ifndef CEF3_DEFAULT_CACHE
#		define CEF3_DEFAULT_CACHE 1
#	endif
#	define CEF3_INIT_MAX_RETRIES 3
#endif

FString FWebBrowserSingleton::ApplicationCacheDir() const
{
#if PLATFORM_MAC
	// OSX wants caches in a separate location from other app data
	static TCHAR Result[MAC_MAX_PATH] = TEXT("");
	if (!Result[0])
	{
		SCOPED_AUTORELEASE_POOL;
		NSString *CacheBaseDir = [NSSearchPathForDirectoriesInDomains(NSCachesDirectory, NSUserDomainMask, YES) objectAtIndex: 0];
		NSString* BundleID = [[NSBundle mainBundle] bundleIdentifier];
		if(!BundleID)
		{
			BundleID = [[NSProcessInfo processInfo] processName];
		}
		check(BundleID);

		NSString* AppCacheDir = [CacheBaseDir stringByAppendingPathComponent: BundleID];
		FPlatformString::CFStringToTCHAR((CFStringRef)AppCacheDir, Result);
	}
	return FString(Result);
#else
	// Only store the web cache under ProjectSavedDir() if it points outside the .uproject dir (e.g. installed games or custom options set)
	if (FPaths::ShouldSaveToUserDir())
	{
		return FPaths::ProjectSavedDir();
	}

	return FPaths::Combine(FPlatformProcess::UserSettingsDir(), FApp::GetEpicProductIdentifier(), FApp::HasProjectName() ? FApp::GetProjectName() : TEXT("Editor"));
#endif
}


class FWebBrowserWindowFactory
	: public IWebBrowserWindowFactory
{
public:

	virtual ~FWebBrowserWindowFactory()
	{ }

	virtual TSharedPtr<IWebBrowserWindow> Create(
		TSharedPtr<FCEFWebBrowserWindow>& BrowserWindowParent,
		TSharedPtr<FWebBrowserWindowInfo>& BrowserWindowInfo) override
	{
		return IWebBrowserModule::Get().GetSingleton()->CreateBrowserWindow(
			BrowserWindowParent,
			BrowserWindowInfo);
	}

	virtual TSharedPtr<IWebBrowserWindow> Create(
		void* OSWindowHandle,
		FString InitialURL,
		bool bUseTransparency,
		bool bThumbMouseButtonNavigation,
		bool bInterceptLoadRequests = true,
		TOptional<FString> ContentsToLoad = TOptional<FString>(),
		TOptional<FString> UserAgentApplication = TOptional<FString>(),
		bool ShowErrorMessage = true,
		FColor BackgroundColor = FColor(255, 255, 255, 255),
		bool bMobileJSReturnInDict = true) override
	{
		FCreateBrowserWindowSettings Settings;
		Settings.OSWindowHandle = OSWindowHandle;
		Settings.InitialURL = MoveTemp(InitialURL);
		Settings.bUseTransparency = bUseTransparency;
		Settings.bThumbMouseButtonNavigation = bThumbMouseButtonNavigation;
		Settings.ContentsToLoad = MoveTemp(ContentsToLoad);
		Settings.UserAgentApplication = MoveTemp(UserAgentApplication);
		Settings.bShowErrorMessage = ShowErrorMessage;
		Settings.BackgroundColor = BackgroundColor;
		Settings.bInterceptLoadRequests = bInterceptLoadRequests;
		Settings.bMobileJSReturnInDict = bMobileJSReturnInDict;

		return IWebBrowserModule::Get().GetSingleton()->CreateBrowserWindow(Settings);
	}
};

class FNoWebBrowserWindowFactory
	: public IWebBrowserWindowFactory
{
public:

	virtual ~FNoWebBrowserWindowFactory()
	{ }

	virtual TSharedPtr<IWebBrowserWindow> Create(
		TSharedPtr<FCEFWebBrowserWindow>& BrowserWindowParent,
		TSharedPtr<FWebBrowserWindowInfo>& BrowserWindowInfo) override
	{
		return nullptr;
	}

	virtual TSharedPtr<IWebBrowserWindow> Create(
		void* OSWindowHandle,
		FString InitialURL,
		bool bUseTransparency,
		bool bThumbMouseButtonNavigation,
		bool bInterceptLoadRequests = true,
		TOptional<FString> ContentsToLoad = TOptional<FString>(),
		TOptional<FString> UserAgentApplication = TOptional<FString>(),
		bool ShowErrorMessage = true,
		FColor BackgroundColor = FColor(255, 255, 255, 255),
		bool bMobileJSReturnInDict = true) override
	{
		return nullptr;
	}
};

#if WITH_CEF3

static FString MakeUserAgentProductString(const FString& UserAgentApplication)
{
	// Append Chrome engine version to the product part of the user agent string
	return FString::Printf(TEXT("%s Chrome/%d.%d.%d.%d"), *UserAgentApplication, CHROME_VERSION_MAJOR, CHROME_VERSION_MINOR, CHROME_VERSION_BUILD, CHROME_VERSION_PATCH);
}

#if PLATFORM_MAC || PLATFORM_LINUX
class FPosixSignalPreserver
{
public:
	FPosixSignalPreserver()
	{
		struct sigaction Sigact;
		for (uint32 i = 0; i < UE_ARRAY_COUNT(PreserveSignals); ++i)
		{
			FMemory::Memset(&Sigact, 0, sizeof(Sigact));
			if (sigaction(PreserveSignals[i], nullptr, &Sigact) != 0)
			{
				UE_LOGF(LogWebBrowser, Warning, "Failed to backup signal handler for %i.", PreserveSignals[i]);
			}
			OriginalSignalHandlers[i] = Sigact;
		}
	}

	~FPosixSignalPreserver()
	{
		for (uint32 i = 0; i < UE_ARRAY_COUNT(PreserveSignals); ++i)
		{
			if(sigaction(PreserveSignals[i], &OriginalSignalHandlers[i], nullptr) != 0)
			{
				UE_LOGF(LogWebBrowser, Warning, "Failed to restore signal handler for %i.", PreserveSignals[i]);
			}
		}
	}

private:
	// Backup the list of signals that CEF/Chromium overrides, derived from SetupSignalHandlers() in
	//  https://chromium.googlesource.com/chromium/src.git/+/2fc330d0b93d4bfd7bd04b9fdd3102e529901f91/services/service_manager/embedder/main.cc
	const int PreserveSignals[13] = {SIGHUP, SIGINT, SIGQUIT, SIGILL, SIGABRT,
		SIGFPE, SIGSEGV, SIGALRM, SIGTERM, SIGCHLD, SIGBUS, SIGTRAP, SIGPIPE};

	struct sigaction OriginalSignalHandlers[UE_ARRAY_COUNT(PreserveSignals)];
};

static bool IsCEFLockFileCreated(const FString& LockFile)
{
	// Use lstat() to check the link file itself rather than the target (which isn't a valid file path)
	struct stat FileInfo;
	return lstat(TCHAR_TO_UTF8(*LockFile), &FileInfo) != -1 && S_ISLNK(FileInfo.st_mode);
}

static bool IsValidCEFLockFile(const FString& LockFile)
{
	// the lockfile on Linux/Mac is actually a symlink with a fake path formatted as <hostname>-<pid> 
	char Buffer[128];
	const ssize_t Len = readlink(TCHAR_TO_UTF8(*LockFile), Buffer, sizeof(Buffer));
	if (Len <= -1 || Len >= sizeof(Buffer))
	{
		return false;
	}
	Buffer[Len] = '\0';
	FString LockFileValue = UTF8_TO_TCHAR(Buffer);

	int32 PidStartIndex = 0;
	if (!LockFileValue.FindLastChar(TEXT('-'), PidStartIndex))
	{
		return false;
	}
	
	uint32 Pid = 0;
	if (!LexTryParseString(Pid, *LockFileValue.RightChop(++PidStartIndex)))
	{
		return false;
	}

	FProcHandle Handle = FPlatformProcess::OpenProcess(Pid);
	ON_SCOPE_EXIT{ FPlatformProcess::CloseProc(Handle); };
	return FPlatformProcess::IsProcRunning(Handle);
}

#endif // PLATFORM_MAC || PLATFORM_LINUX
#endif // WITH_CEF3

FWebBrowserSingleton::FWebBrowserSingleton(const FWebBrowserInitSettings& WebBrowserInitSettings)
#if WITH_CEF3
	: WebBrowserWindowFactory(MakeShareable(new FWebBrowserWindowFactory()))
	, bCEFInitialized(false)
	, bTaskQueueFlushed(false)
#else
	: WebBrowserWindowFactory(MakeShareable(new FNoWebBrowserWindowFactory()))
#endif
	, UserAgentApplication(WebBrowserInitSettings.ProductVersion)
	, bDevToolsShortcutEnabled(UE_BUILD_DEBUG)
	, bJSBindingsToLoweringEnabled(true)
	, bAppIsFocused(false)
#if WITH_ENGINE
	, DefaultMaterial(FSoftObjectPath(FString(TEXT("/Engine/WebBrowser/WebTexture_M.WebTexture_M"))))
	, DefaultTranslucentMaterial(FSoftObjectPath(FString(TEXT("/Engine/WebBrowser/WebTexture_TM.WebTexture_TM"))))
#endif
{
#if WITH_CEF3
	
	// Only enable CEF if we have CEF3, we are not running a commandlet without rendering (e.g. cooking assets) and it has not been explicitly disabled
	// Disallow CEF if we never plan on rendering, ie, with CanEverRender. This includes servers
	bool bAllowCEF = (!IsRunningCommandlet() || (IsAllowCommandletRendering() && FParse::Param(FCommandLine::Get(), TEXT("AllowCommandletCEF")))) &&
				FApp::CanEverRender() && !FParse::Param(FCommandLine::Get(), TEXT("nocef")) &&
				IWebBrowserModule::Get().IsWebModuleAvailable();
	if (bAllowCEF)
	{
		// The FWebBrowserSingleton must be initialized on the game thread
		check(IsInGameThread());

		// Provide CEF with command-line arguments.
#if PLATFORM_WINDOWS
		CefMainArgs MainArgs(hInstance);
#else
		CefMainArgs MainArgs;
#endif

		bool bVerboseLogging = FParse::Param(FCommandLine::Get(), TEXT("cefverbose")) || FParse::Param(FCommandLine::Get(), TEXT("debuglog"));
		// CEFBrowserApp implements application-level callbacks.
		CEFBrowserApp = new FCEFBrowserApp;

		// Specify CEF global settings here.
		CefSettings Settings;
		Settings.no_sandbox = true;
		Settings.command_line_args_disabled = true;
		Settings.external_message_pump = true;
		//@todo change to threaded version instead of using external_message_pump & OnScheduleMessagePumpWork
		Settings.multi_threaded_message_loop = false;
		//Set the default background for browsers to be opaque black, this is used for windowed (not OSR) browsers
		//  setting it black here prevents the white flash on load
		Settings.background_color = CefColorSetARGB(255, 0, 0, 0);

#if PLATFORM_LINUX
		Settings.windowless_rendering_enabled = true;
#endif

		FString CefLogFile(FPaths::Combine(*FPaths::ProjectLogDir(), TEXT("cef3.log")));
		CefLogFile = FPaths::ConvertRelativePathToFull(CefLogFile);
		CefString(&Settings.log_file) = TCHAR_TO_WCHAR(*CefLogFile);
		Settings.log_severity = bVerboseLogging ? LOGSEVERITY_VERBOSE : LOGSEVERITY_WARNING;

		uint16 DebugPort;
		if(FParse::Value(FCommandLine::Get(), TEXT("cefdebug="), DebugPort))
		{
			Settings.remote_debugging_port = DebugPort;
		}

		// Specify locale from our settings
		FString LocaleCode = GetCurrentLocaleCode();
		CefString(&Settings.locale) = TCHAR_TO_WCHAR(*LocaleCode);

		FString UserAgentProduct = MakeUserAgentProductString(UserAgentApplication);
		CefString(&Settings.user_agent_product) = TCHAR_TO_WCHAR(*UserAgentProduct);

#if CEF3_DEFAULT_CACHE
		// First clean up existing cache folders which contain outdated or problematic data
		ClearOldCacheFolders(ApplicationCacheDir(), TEXT("webcache"));

		// Enable on disk cache
		const FString CachePathBase(FPaths::Combine(ApplicationCacheDir(), TEXT("webcache")));
		FString CachePath = FPaths::ConvertRelativePathToFull(GenerateWebCacheFolderName(CachePathBase));
		CefString(&Settings.cache_path) = TCHAR_TO_WCHAR(*CachePath);
#endif

		// Specify path to resources
		FString ResourcesPath(FPaths::Combine(*FPaths::EngineDir(), CEF3_RESOURCES_DIR));
		ResourcesPath = FPaths::ConvertRelativePathToFull(ResourcesPath);
		if (!FPaths::DirectoryExists(ResourcesPath))
		{
			UE_LOGF(LogWebBrowser, Error, "Chromium Resources information not found at: %ls.", *ResourcesPath);
		}
		CefString(&Settings.resources_dir_path) = TCHAR_TO_WCHAR(*ResourcesPath);

#if !PLATFORM_MAC
		// On Mac Chromium ignores custom locales dir. Files need to be stored in Resources folder in the app bundle
		FString LocalesPath(FPaths::Combine(*ResourcesPath, TEXT("locales")));
		LocalesPath = FPaths::ConvertRelativePathToFull(LocalesPath);
		if (!FPaths::DirectoryExists(LocalesPath))
		{
			UE_LOGF(LogWebBrowser, Error, "Chromium Locales information not found at: %ls.", *LocalesPath);
		}
		CefString(&Settings.locales_dir_path) = TCHAR_TO_WCHAR(*LocalesPath);
#else
		// LocaleCode may contain region, which for some languages may make CEF unable to find the locale pak files
		// In that case use the language name for CEF locale
		FString LocalePakPath = ResourcesPath + TEXT("/") + LocaleCode.Replace(TEXT("-"), TEXT("_")) + TEXT(".lproj/locale.pak");
		if (!FPaths::FileExists(LocalePakPath))
		{
			FCultureRef Culture = FInternationalization::Get().GetCurrentCulture();
			LocaleCode = Culture->GetTwoLetterISOLanguageName();
			LocalePakPath = ResourcesPath + TEXT("/") + LocaleCode + TEXT(".lproj/locale.pak");
			if (FPaths::FileExists(LocalePakPath))
			{
				CefString(&Settings.locale) = TCHAR_TO_WCHAR(*LocaleCode);
			}
		}

		// Let CEF know where we have put the framework bundle as it is non-default
		FString CefFrameworkPath(FPaths::Combine(*FPaths::EngineDir(), CEF3_FRAMEWORK_DIR));
		CefFrameworkPath = FPaths::ConvertRelativePathToFull(CefFrameworkPath);
		CefString(&Settings.framework_dir_path) = TCHAR_TO_WCHAR(*CefFrameworkPath);
		CefString(&Settings.main_bundle_path) = TCHAR_TO_WCHAR(*CefFrameworkPath);
#endif

		// Specify path to sub process exe
		FString SubProcessPath(FPaths::Combine(*FPaths::EngineDir(), CEF3_SUBPROCES_EXE));
		SubProcessPath = FPaths::ConvertRelativePathToFull(SubProcessPath);

		if (!IPlatformFile::GetPlatformPhysical().FileExists(*SubProcessPath))
		{
			UE_LOGF(LogWebBrowser, Error, "EpicWebHelper.exe not found, check that this program has been built and is placed in: %ls.", *SubProcessPath);
		}
		CefString(&Settings.browser_subprocess_path) = TCHAR_TO_WCHAR(*SubProcessPath);

#if PLATFORM_MAC || PLATFORM_LINUX
		// this class automatically preserves the sigaction handlers we have set
		FPosixSignalPreserver PosixSignalPreserver;
#endif

		// Initialize CEF.
		PRAGMA_DISABLE_UNREACHABLE_CODE_WARNINGS // only windows uses retries, but restructuring for other platforms is not worth it
		for (int NumRetries = 0; NumRetries <= CEF3_INIT_MAX_RETRIES; ++NumRetries)
		{
		PRAGMA_RESTORE_UNREACHABLE_CODE_WARNINGS
			bCEFInitialized = CefInitialize(MainArgs, Settings, CEFBrowserApp.get(), nullptr);
			if (bCEFInitialized)
			{
				break;
			}

			const int CefExitCode = CefGetExitCode();
			// There's a race condition that can occur when two UE instances are started close together:
			// We check the presence of a lockfile inside the CEF cache dir to detect if that cache is already in use by another UE instance,
			// but if the second instance checks before the first instance has created the lockfile then we'll incorrectly reuse the same cache.
			// CEF itself handles this situation by notifying the first instance of the second launch (see FCEFBrowserApp::OnAlreadyRunningAppRelaunch)
			// and returning CEF_RESULT_CODE_NORMAL_EXIT_PROCESS_NOTIFIED in the second instance for early exit, basically a single-app-instance pattern.
			// If this happens, we try initializing again with another cache dir.
			if (CefExitCode == CEF_RESULT_CODE_NORMAL_EXIT_PROCESS_NOTIFIED && NumRetries < CEF3_INIT_MAX_RETRIES)
			{
#if PLATFORM_WINDOWS
				UE_LOGF(LogWebBrowser, Warning, "Detected concurrent CEF initialization for cache dir %ls! Retrying... (%d of %d)", *CachePath, NumRetries + 1, CEF3_INIT_MAX_RETRIES);

				// CEF/Chromium doesn't allow calling CefInitialize multiple times, even if the initialization failed,
				// we have to fully unload the DLLs then reload them to restart from scratch.
				// But first we must call __FUnloadDelayLoadedDLL2 to reset the delay-import address table for the symbols
				// that were already resolved, since the DLL base address may change after the reload.
				// Note that __FUnloadDelayLoadedDLL2 must be called here because it uses __ImageBase, so calling it from CEF3Utils wouldn't work.
				TCHAR CEFModuleFileName[MAX_PATH] = {};
				GetModuleFileName((HMODULE)CEF3Utils::GetCEF3ModuleHandle(), CEFModuleFileName, MAX_PATH);
				if (!__FUnloadDelayLoadedDLL2(TCHAR_TO_UTF8(*FPaths::GetCleanFilename(CEFModuleFileName))))
				{
					// Don't attempt reloading if we couldn't clear the delay import address table, or it will result in crashes due to stale entries
					UE_LOGF(LogWebBrowser, Error, "Unable to cleanly reload CEF DLL, can't retry initialization!");
					break;
				}

				CEF3Utils::UnloadCEF3Modules();
				CEF3Utils::LoadCEF3Modules(true);

				// The cache dir returned should be a different one than previously, as the lockfile will now be detected.
				// However the same race could be happening with this new cache dir and another UE instance starting,
				// that's why we keep retrying as long as we get that exit code from CefInitialize.
				CachePath = FPaths::ConvertRelativePathToFull(GenerateWebCacheFolderName(CachePathBase));
				CefString(&Settings.cache_path) = TCHAR_TO_WCHAR(*CachePath);
				continue;
#else
				// No DLL unload/reload support on Linux or Mac, we have to bail out if CefInitialize reports an already existing instance
				UE_LOGF(LogWebBrowser, Warning, "Another CEF instance is already running with cache dir %ls! WebBrowser module won't be functional.", *CachePath);
				break;
#endif // PLATFORM_WINDOWS
			}
			UE_LOGF(LogWebBrowser, Error, "CEF initialization for cache dir %ls failed with exit code %d!", *CachePath, CefExitCode);
			break;
		}
		// this will upload a report to our online crash tool so we can find out why the initialization failed
		ensure(bCEFInitialized);

		// Set the thread name back to GameThread.
		FPlatformProcess::SetThreadName(*FName(NAME_GameThread).GetPlainNameString());

		if (bCEFInitialized)
		{
			DefaultCookieManager = FCefWebBrowserCookieManagerFactory::Create(CefCookieManager::GetGlobalManager(nullptr));
		}
	}
#elif (PLATFORM_MAC || PLATFORM_IOS) && !BUILD_EMBEDDED_APP
	DefaultCookieManager = MakeShareable(new FAppleCookieManager());
#elif PLATFORM_ANDROID
	DefaultCookieManager = MakeShareable(new FAndroidCookieManager());
#endif
	
#if WITH_ENGINE
	DefaultMaterial.LoadSynchronous();
	DefaultTranslucentMaterial.LoadSynchronous();
#endif
}


#if WITH_CEF3
void FWebBrowserSingleton::WaitForTaskQueueFlush()
{
	// Keep pumping messages until we see the one below clear the queue
	bTaskQueueFlushed = false;
	CefPostTask(TID_UI, new FCEFBrowserClosureTask(nullptr, [this]()
		{
			bTaskQueueFlushed = true;
		}));

	const double StartWaitAppTime = FPlatformTime::Seconds();
	while (!bTaskQueueFlushed)
	{
		FPlatformProcess::Sleep(0.01f);
		// CEF needs the windows message pump run to be able to finish closing a browser, so run it manually here
		if (FSlateApplication::IsInitialized())
		{
			FSlateApplication::Get().PumpMessages();
		}
		CefDoMessageLoopWork();
		// Wait at most 1 second for tasks to clear, in case CEF crashes/hangs during process lifetime
		if (FPlatformTime::Seconds() - StartWaitAppTime > 1.0f)
		{
			break; // don't spin forever
		}
	}
}
#endif


FWebBrowserSingleton::~FWebBrowserSingleton()
{
#if WITH_CEF3
	if (bCEFInitialized)
	{
		{
			FScopeLock Lock(&WindowInterfacesCS);
			// Force all existing browsers to close in case any haven't been deleted
			for (int32 Index = 0; Index < WindowInterfaces.Num(); ++Index)
			{
				auto BrowserWindow = WindowInterfaces[Index].Pin();
				if (BrowserWindow.IsValid() && BrowserWindow->IsValid())
				{
					// Call CloseBrowser directly on the Host object as FWebBrowserWindow::CloseBrowser is delayed
					BrowserWindow->InternalCefBrowser->GetHost()->CloseBrowser(true);
				}
			}
			// Clear this before CefShutdown() below
			WindowInterfaces.Reset();
		}

		// Remove references to the scheme handler factories
		CefClearSchemeHandlerFactories();
		for (const TPair<FString, CefRefPtr<CefRequestContext>>& RequestContextPair : RequestContexts)
		{
			RequestContextPair.Value->ClearSchemeHandlerFactories();
		}
		// Clear this before CefShutdown() below
		RequestContexts.Reset();

		// make sure any handler before load delegates are unbound
		for (const TPair <FString,CefRefPtr<FCEFResourceContextHandler>>& HandlerPair : RequestResourceHandlers)
		{
			HandlerPair.Value->OnBeforeLoad().Unbind();
		}
		// Clear this before CefShutdown() below
		RequestResourceHandlers.Reset();
		// CefRefPtr takes care of delete
		CEFBrowserApp = nullptr;

		WaitForTaskQueueFlush();

		// Shut down CEF.
		CefShutdown();
	}
	bCEFInitialized = false;
#elif PLATFORM_IOS || PLATFORM_SPECIFIC_WEB_BROWSER || (PLATFORM_ANDROID && USE_ANDROID_JNI)
	{
		FScopeLock Lock(&WindowInterfacesCS);
		// Clear this before CefShutdown() below
		WindowInterfaces.Reset();
	}
#endif
}

bool FWebBrowserSingleton::IsShuttingDown() const
{
#if WITH_CEF3
	return bCEFInitialized && !CEFBrowserApp;
#else
	return false;
#endif
}

TSharedRef<IWebBrowserWindowFactory> FWebBrowserSingleton::GetWebBrowserWindowFactory() const
{
	return WebBrowserWindowFactory;
}

TSharedPtr<IWebBrowserWindow> FWebBrowserSingleton::CreateBrowserWindow(
	TSharedPtr<FCEFWebBrowserWindow>& BrowserWindowParent,
	TSharedPtr<FWebBrowserWindowInfo>& BrowserWindowInfo
	)
{
#if WITH_CEF3
	if (bCEFInitialized)
	{
		TOptional<FString> ContentsToLoad;

		bool bShowErrorMessage = BrowserWindowParent->IsShowingErrorMessages();
		bool bThumbMouseButtonNavigation = BrowserWindowParent->IsThumbMouseButtonNavigationEnabled();
		bool bUseTransparency = BrowserWindowParent->UseTransparency();
		bool bUsingAcceleratedPaint = BrowserWindowParent->UsingAcceleratedPaint();
		FString UserAgentProduct = BrowserWindowParent->GetUserAgentProduct();
		FString InitialURL = WCHAR_TO_TCHAR(BrowserWindowInfo->Browser->GetMainFrame()->GetURL().ToWString().c_str());
		TSharedPtr<FCEFWebBrowserWindow> NewBrowserWindow(new FCEFWebBrowserWindow(BrowserWindowInfo->Browser, BrowserWindowInfo->Handler, InitialURL, ContentsToLoad, UserAgentProduct, bShowErrorMessage, bThumbMouseButtonNavigation, bUseTransparency, bJSBindingsToLoweringEnabled, bUsingAcceleratedPaint));
		BrowserWindowInfo->Handler->SetBrowserWindow(NewBrowserWindow);
		{
			FScopeLock Lock(&WindowInterfacesCS);
			WindowInterfaces.Add(NewBrowserWindow);
		}
		NewBrowserWindow->GetCefBrowser()->GetHost()->SetWindowlessFrameRate(BrowserWindowParent->GetCefBrowser()->GetHost()->GetWindowlessFrameRate());
		return NewBrowserWindow;
	}
#endif
	return nullptr;
}

TSharedPtr<IWebBrowserWindow> FWebBrowserSingleton::CreateBrowserWindow(const FCreateBrowserWindowSettings& WindowSettings)
{
	bool bBrowserEnabled = true;
	GConfig->GetBool(TEXT("Browser"), TEXT("bEnabled"), bBrowserEnabled, GEngineIni);
	if (!bBrowserEnabled || !FApp::CanEverRender())
	{
		return nullptr;
	}

#if WITH_CEF3
	if (bCEFInitialized)
	{
		// Information used when creating the native window.
		CefWindowInfo WindowInfo;

		// Specify CEF browser settings here.
		CefBrowserSettings BrowserSettings;

		// The color to paint before a document is loaded
		// if using a windowed(native) browser window AND bUseTransparency is true then the background actually uses Settings.background_color from above
		// if using a OSR window and bUseTransparency is true then you get a transparency channel in your BGRA OnPaint
		// if bUseTransparency is false then you get the background color defined by your RGB setting here
		BrowserSettings.background_color = CefColorSetARGB(WindowSettings.bUseTransparency ? 0 : WindowSettings.BackgroundColor.A, WindowSettings.BackgroundColor.R, WindowSettings.BackgroundColor.G, WindowSettings.BackgroundColor.B);

#if PLATFORM_WINDOWS
		// Create the widget as a child window on windows when passing in a parent window
		if (WindowSettings.OSWindowHandle != nullptr)
		{
			RECT ClientRect = { 0, 0, 0, 0 };
			if (!GetClientRect((HWND)WindowSettings.OSWindowHandle, &ClientRect))
			{
				UE_LOGF(LogWebBrowser, Error, "Failed to get client rect");
			}
			RECT WindowRect = {};
			float DPIScale = 1.0f;
			if (!GetWindowRect((HWND)WindowSettings.OSWindowHandle, &WindowRect))
			{
				UE_LOGF(LogWebBrowser, Error, "Failed to get window rect");
			}
			else
			{
				DPIScale = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(WindowRect.left, WindowRect.top);
			}
			WindowInfo.SetAsChild((CefWindowHandle)WindowSettings.OSWindowHandle, { 0, 0, static_cast<int>((ClientRect.right - ClientRect.left) / DPIScale), static_cast<int>((ClientRect.bottom - ClientRect.top) / DPIScale) });
		}
		else
#endif
		{
			// Use off screen rendering so we can integrate with our windows
			WindowInfo.SetAsWindowless(kNullWindowHandle);
			WindowInfo.shared_texture_enabled = FCEFWebBrowserWindow::CanSupportAcceleratedPaint() ? 1 : 0;
			int BrowserFrameRate = WindowSettings.BrowserFrameRate;
			if (FCEFWebBrowserWindow::CanSupportAcceleratedPaint() && BrowserFrameRate == 24)
			{
				// Use 60 fps if the accelerated renderer is enabled and the default framerate was otherwise selected
				BrowserFrameRate = 60;
			}
			BrowserSettings.windowless_frame_rate = BrowserFrameRate;
		}

		FString UserAgentProduct = MakeUserAgentProductString(WindowSettings.UserAgentApplication.Get(UserAgentApplication /*DefaultValue*/));
		// The user_agent_product setting is only available for Windows and Linux at the moment,
		// as it requires updated EpicWebHelper binaries and that's not possible anymore on Mac until we update CEF
#if PLATFORM_WINDOWS || PLATFORM_LINUX
		CefString(&BrowserSettings.user_agent_product) = TCHAR_TO_WCHAR(*UserAgentProduct);
#endif

		TArray<FString> AuthorizationHeaderAllowListURLS;
		GConfig->GetArray(TEXT("Browser"), TEXT("AuthorizationHeaderAllowListURLS"), AuthorizationHeaderAllowListURLS, GEngineIni);

		// WebBrowserHandler implements browser-level callbacks.
		CefRefPtr<FCEFBrowserHandler> NewHandler(new FCEFBrowserHandler(WindowSettings.bUseTransparency, WindowSettings.bInterceptLoadRequests ,WindowSettings.AltRetryDomains, AuthorizationHeaderAllowListURLS));

		CefRefPtr<CefRequestContext> RequestContext = nullptr;
		if (WindowSettings.Context.IsSet())
		{
			const FBrowserContextSettings Context = WindowSettings.Context.GetValue();
			const CefRefPtr<CefRequestContext>* ExistingRequestContext = RequestContexts.Find(Context.Id);

			if (ExistingRequestContext == nullptr)
			{
				CefRequestContextSettings RequestContextSettings;
				CefString(&RequestContextSettings.accept_language_list) = Context.AcceptLanguageList.IsEmpty() ? TCHAR_TO_WCHAR(*GetCurrentLocaleCode()) : TCHAR_TO_WCHAR(*Context.AcceptLanguageList);
				CefString(&RequestContextSettings.cache_path) = TCHAR_TO_WCHAR(*GenerateWebCacheFolderName(Context.CookieStorageLocation));
				RequestContextSettings.persist_session_cookies = Context.bPersistSessionCookies;

				CefRefPtr<FCEFResourceContextHandler> ResourceContextHandler = new FCEFResourceContextHandler(this);
				ResourceContextHandler->OnBeforeLoad() = Context.OnBeforeContextResourceLoad;
				RequestResourceHandlers.Add(Context.Id, ResourceContextHandler);

				//Create a new one
				RequestContext = CefRequestContext::CreateContext(RequestContextSettings, ResourceContextHandler);
				RequestContexts.Add(Context.Id, RequestContext);
			}
			else
			{
				RequestContext = *ExistingRequestContext;
			}
			SchemeHandlerFactories.RegisterFactoriesWith(RequestContext);
			UE_LOGF(LogWebBrowser, Log, "Creating browser for ContextId=%ls.", *WindowSettings.Context.GetValue().Id);
		}
		if (RequestContext == nullptr)
		{
			// As of CEF drop 4430 the CreateBrowserSync call requires a non-null request context, so fall back to the default one if needed
			RequestContext = CefRequestContext::GetGlobalContext();
		}

		// Create the CEF browser window.
		CefRefPtr<CefBrowser> Browser = CefBrowserHost::CreateBrowserSync(WindowInfo, NewHandler.get(), TCHAR_TO_WCHAR(*WindowSettings.InitialURL), BrowserSettings, nullptr, RequestContext);
		if (Browser.get())
		{
			// Create new window
			TSharedPtr<FCEFWebBrowserWindow> NewBrowserWindow = MakeShareable(new FCEFWebBrowserWindow(
				Browser,
				NewHandler,
				WindowSettings.InitialURL,
				WindowSettings.ContentsToLoad,
				UserAgentProduct,
				WindowSettings.bShowErrorMessage,
				WindowSettings.bThumbMouseButtonNavigation,
				WindowSettings.bUseTransparency,
				bJSBindingsToLoweringEnabled,
				WindowInfo.shared_texture_enabled == 1 ? true : false));
			NewHandler->SetBrowserWindow(NewBrowserWindow);
			{
				FScopeLock Lock(&WindowInterfacesCS);
				WindowInterfaces.Add(NewBrowserWindow);
			}

			return NewBrowserWindow;
		}
	}
	return nullptr;
#elif PLATFORM_ANDROID && USE_ANDROID_JNI
	// Create new window
	TSharedPtr<FAndroidWebBrowserWindow> NewBrowserWindow = MakeShareable(new FAndroidWebBrowserWindow(
		WindowSettings.InitialURL,
		WindowSettings.ContentsToLoad,
		WindowSettings.bShowErrorMessage,
		WindowSettings.bThumbMouseButtonNavigation,
		WindowSettings.bUseTransparency,
		bJSBindingsToLoweringEnabled,
		WindowSettings.UserAgentApplication.Get(UserAgentApplication /*DefaultValue*/),
		WindowSettings.bMobileJSReturnInDict,
	    WindowSettings.bEnablePaymentRequest));

	{
		FScopeLock Lock(&WindowInterfacesCS);
		WindowInterfaces.Add(NewBrowserWindow);
	}
	return NewBrowserWindow;
#elif PLATFORM_IOS || PLATFORM_MAC
	// Create new window
	TSharedPtr<FWebBrowserWindow> NewBrowserWindow = MakeShareable(new FWebBrowserWindow(
		WindowSettings.InitialURL, 
		WindowSettings.ContentsToLoad, 
		WindowSettings.bShowErrorMessage, 
		WindowSettings.bThumbMouseButtonNavigation, 
		WindowSettings.bUseTransparency,
		bJSBindingsToLoweringEnabled,
		WindowSettings.UserAgentApplication.Get(UserAgentApplication /*DefaultValue*/),
		WindowSettings.bMobileJSReturnInDict));

	{
		FScopeLock Lock(&WindowInterfacesCS);
		WindowInterfaces.Add(NewBrowserWindow);
	}
	return NewBrowserWindow;
#elif PLATFORM_SPECIFIC_WEB_BROWSER
	// Create new window
	TSharedPtr<FWebBrowserWindow> NewBrowserWindow = MakeShareable(new FWebBrowserWindow(
		WindowSettings.InitialURL,
		WindowSettings.ContentsToLoad,
		WindowSettings.bShowErrorMessage,
		WindowSettings.bThumbMouseButtonNavigation,
		WindowSettings.bUseTransparency));

	{
		FScopeLock Lock(&WindowInterfacesCS);
		WindowInterfaces.Add(NewBrowserWindow);
	}
	return NewBrowserWindow;
#else
	return nullptr;
#endif
}

#if BUILD_EMBEDDED_APP
TSharedPtr<IWebBrowserWindow> FWebBrowserSingleton::CreateNativeBrowserProxy()
{
	TSharedPtr<FNativeWebBrowserProxy> NewBrowserWindow = MakeShareable(new FNativeWebBrowserProxy(
		bJSBindingsToLoweringEnabled
	));
	NewBrowserWindow->Initialize();
	return NewBrowserWindow;
}
#endif //BUILD_EMBEDDED_APP

bool FWebBrowserSingleton::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FWebBrowserSingleton_Tick);

#if WITH_CEF3
	if (bCEFInitialized)
	{
		{
			FScopeLock Lock(&WindowInterfacesCS);
			bool bIsSlateAwake = FSlateApplication::IsInitialized() && !FSlateApplication::Get().IsSlateAsleep();
			// Remove any windows that have been deleted and check whether it's currently visible
			for (int32 Index = WindowInterfaces.Num() - 1; Index >= 0; --Index)
			{
				if (!WindowInterfaces[Index].IsValid())
				{
					WindowInterfaces.RemoveAt(Index);
				}
				else if (bIsSlateAwake) // only check for Tick activity if Slate is currently ticking
				{
					TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = WindowInterfaces[Index].Pin();
					if(BrowserWindow.IsValid())
					{
						// Test if we've ticked recently. If not assume the browser window has become hidden.
						BrowserWindow->CheckTickActivity();
					}
				}
			}
		}

	if (CEFBrowserApp != nullptr)
	{
		bool bForceMessageLoop = false;
		GConfig->GetBool(TEXT("Browser"), TEXT("bForceMessageLoop"), bForceMessageLoop, GEngineIni);

		// Get the configured minimum hertz and make sure the value is within a reasonable range
		static const int MaxFrameRateClamp = 60;
		int32 MinMessageLoopHz = 1;
		GConfig->GetInt(TEXT("Browser"), TEXT("MinMessageLoopHertz"), MinMessageLoopHz, GEngineIni);
		MinMessageLoopHz = FMath::Clamp(MinMessageLoopHz, 1, 60);

		// Get the configured forced maximum hertz and make sure the value is within a reasonable range
		int32 MaxForcedMessageLoopHz = 15;
		GConfig->GetInt(TEXT("Browser"), TEXT("MaxForcedMessageLoopHertz"), MaxForcedMessageLoopHz, GEngineIni);
		MaxForcedMessageLoopHz = FMath::Clamp(MaxForcedMessageLoopHz, MinMessageLoopHz, 60);

		// @todo: Hack: We rely on OnScheduleMessagePumpWork() which tells us to drive the CEF message pump, 
		//  there appear to be some edge cases where we might not be getting a signal from it so for the time being 
		//  we force a minimum rates here and let it run at a configurable maximum rate when we have any WindowInterfaces.

		// Convert to seconds which we'll use to compare against the time we accumulated since last pump / left till next pump
		float MinMessageLoopSeconds = 1.0f / MinMessageLoopHz;
		float MaxForcedMessageLoopSeconds = 1.0f / MaxForcedMessageLoopHz;

		static float SecondsSinceLastPump = 0;
		static float SecondsSinceLastAppFocusCheck = MaxForcedMessageLoopSeconds;
		static float SecondsToNextForcedPump = MaxForcedMessageLoopSeconds;

		// Accumulate time since last pump by adding DeltaTime which gives us the amount of time that has passed since last tick in seconds
		SecondsSinceLastPump += DeltaTime;
		SecondsSinceLastAppFocusCheck += DeltaTime;
		// Time left till next pump
		SecondsToNextForcedPump -= DeltaTime;

		bool bWantForce = bForceMessageLoop;								  // True if we wish to force message pump
		bool bCanForce = SecondsToNextForcedPump <= 0;                        // But can we?
		bool bMustForce = SecondsSinceLastPump >= MinMessageLoopSeconds;      // Absolutely must force (Min frequency rate hit)
		if (SecondsSinceLastAppFocusCheck > MinMessageLoopSeconds && WindowInterfaces.Num() > 0)
		{
			SecondsSinceLastAppFocusCheck = 0;
			// only check app being foreground at the min message loop rate (1hz) and if we have a browser window to save CPU
			bAppIsFocused = FPlatformApplicationMisc::IsThisApplicationForeground(); 
		}
		// NOTE - bAppIsFocused could be stale if WindowInterfaces.Num() == 0
		bool bAppIsFocusedAndWebWindows = WindowInterfaces.Num() > 0 && bAppIsFocused;

		// if we won't force AND are the foreground OS app AND we have windows created see if any are visible (not minimized) right now
		if (bWantForce == false && bMustForce  == false && bAppIsFocusedAndWebWindows == true )
		{
			for (int32 Index = 0; Index < WindowInterfaces.Num(); Index++)
			{
				if (WindowInterfaces[Index].IsValid())
				{
					TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = WindowInterfaces[Index].Pin();
					if (BrowserWindow->GetParentWindow().IsValid())
					{
						TSharedPtr<SWindow> BrowserParentWindow = BrowserWindow->GetParentWindow();
						if (!BrowserParentWindow->IsWindowMinimized())
						{
							bWantForce = true;
						}
					}
				}
			}
		}

		// tick the CEF app to determine when to run CefDoMessageLoopWork
		if (CEFBrowserApp->TickMessagePump(DeltaTime, (bWantForce && bCanForce) || bMustForce))
		{
			SecondsSinceLastPump = 0;
			SecondsToNextForcedPump = MaxForcedMessageLoopSeconds;
		}
	}

		// Update video buffering for any windows that need it
		for (int32 Index = 0; Index < WindowInterfaces.Num(); Index++)
		{
			if (WindowInterfaces[Index].IsValid())
			{
				TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = WindowInterfaces[Index].Pin();
				if (BrowserWindow.IsValid())
				{
					BrowserWindow->UpdateVideoBuffering();
				}
			}
		}
	}

#elif PLATFORM_APPLE || PLATFORM_SPECIFIC_WEB_BROWSER || (PLATFORM_ANDROID && USE_ANDROID_JNI)
	FScopeLock Lock(&WindowInterfacesCS);
	bool bIsSlateAwake = FSlateApplication::IsInitialized() && !FSlateApplication::Get().IsSlateAsleep();
	// Remove any windows that have been deleted and check whether it's currently visible
	for (int32 Index = WindowInterfaces.Num() - 1; Index >= 0; --Index)
	{
		if (!WindowInterfaces[Index].IsValid())
		{
			WindowInterfaces.RemoveAt(Index);
		}
		else if (bIsSlateAwake) // only check for Tick activity if Slate is currently ticking
		{
			TSharedPtr<IWebBrowserWindow> BrowserWindow = WindowInterfaces[Index].Pin();
			if (BrowserWindow.IsValid())
			{
				// Test if we've ticked recently. If not assume the browser window has become hidden.
				BrowserWindow->CheckTickActivity();
			}
		}
	}

#endif
	return true;
}

FString FWebBrowserSingleton::GetCurrentLocaleCode()
{
	FCultureRef Culture = FInternationalization::Get().GetCurrentCulture();
	FString LocaleCode = Culture->GetTwoLetterISOLanguageName();
	FString Country = Culture->GetRegion();
	if (!Country.IsEmpty())
	{
		LocaleCode = LocaleCode + TEXT("-") + Country;
	}
	return LocaleCode;
}

TSharedPtr<IWebBrowserCookieManager> FWebBrowserSingleton::GetCookieManager(TOptional<FString> ContextId) const
{
	if (ContextId.IsSet())
	{
#if WITH_CEF3
		if (bCEFInitialized)
		{
			const CefRefPtr<CefRequestContext>* ExistingContext = RequestContexts.Find(ContextId.GetValue());

			if (ExistingContext && ExistingContext->get())
			{
				// Cache these cookie managers?
				return FCefWebBrowserCookieManagerFactory::Create((*ExistingContext)->GetCookieManager(nullptr));
			}
			else
			{
				UE_LOGF(LogWebBrowser, Log, "No cookie manager for ContextId=%ls.  Using default cookie manager", *ContextId.GetValue());
			}
		}
#endif
	}
	// No ContextId or cookie manager instance associated with it.  Use default
	return DefaultCookieManager;
}

#if WITH_CEF3
bool FWebBrowserSingleton::URLRequestAllowsCredentials(const FString& URL)
{
	FScopeLock Lock(&WindowInterfacesCS);
	// The FCEFResourceContextHandler::OnBeforeResourceLoad call doesn't get the browser/frame associated with the load
	// (because bugs) so just look at each browser and see if it thinks it knows about this URL
	for (int32 Index = WindowInterfaces.Num() - 1; Index >= 0; --Index)
	{
		TSharedPtr<FCEFWebBrowserWindow> BrowserWindow = WindowInterfaces[Index].Pin();
		if (BrowserWindow.IsValid() && BrowserWindow->URLRequestAllowsCredentials(URL))
		{
			return true;
		}
	}

	return false;
}

FString FWebBrowserSingleton::GenerateWebCacheFolderName(const FString& InputPath)
{
	if (InputPath.IsEmpty())
		return InputPath;

	// append the version of this CEF build to our requested cache folder path
	// this means each new CEF build gets its own cache folder, making downgrading safe
	FString VersionedCachePath = InputPath + "_" + MAKE_STRING(CHROME_VERSION_BUILD);

	// CEF prevents concurrent browser cache access between multiple processes.
	// It locks the cache folder by creating a lockfile in it during initialization,
	// so that other CEF initializations will fail as long as the cache folder is locked.
	for (uint32 CacheNumber = 0; CacheNumber < 32; ++CacheNumber) {
		FString ConcurrentCachePath = VersionedCachePath;
		// don't append the first _0 so we can reuse caches created as VersionedCachePath
		if (CacheNumber)
		{
			ConcurrentCachePath.Append(FString::Format(TEXT("_{0}"), { CacheNumber }));
		}
#if PLATFORM_WINDOWS
		// The lockfile is guaranteed to be deleted when the locking process ends, even in case of a crash
		if (!FPaths::FileExists(FPaths::ConvertRelativePathToFull(FPaths::Combine(ConcurrentCachePath, TEXT("lockfile")))))
		{
			return ConcurrentCachePath;
		}
#else
		const FString LockFile = FPaths::ConvertRelativePathToFull(FPaths::Combine(ConcurrentCachePath, TEXT("SingletonLock")));
		if (IsCEFLockFileCreated(*LockFile))
		{
			// Unlike on Windows, the lockfile is *NOT* deleted if the locking process crashes,
			// so we have to check if it's still valid. If it's not, we can use the cache dir:
			// CEF will replace the old lockfile with a new one.
			if (!IsValidCEFLockFile(LockFile))
			{
				UE_LOGF(LogWebBrowser, Warning, "Found stale CEF lockfile at %ls, overwriting", *LockFile);
				return ConcurrentCachePath;
			}
		}
		else
		{
			// We can't re-attempt CEF initialization on Linux/Mac, so we check for the lockfile
			// a second time after a short sleep to work around most creation races.
			FPlatformProcess::Sleep(0.01f/*10ms*/);
			if (!IsCEFLockFileCreated(*LockFile))
			{
				return ConcurrentCachePath;
			}
		}
#endif // PLATFORM_WINDOWS
	}

	// just fall back to the base path and let CEF initialization fail if we've reached the limit
	return VersionedCachePath;
}
#endif

void FWebBrowserSingleton::ClearOldCacheFolders(const FString &CachePathRoot, const FString &CachePrefix)
{
#if WITH_CEF3
	// only CEF3 currently has version dependant cache folders that may need cleanup
	TArray<FString> CacheFoldersToDelete;
	IPlatformFile::GetPlatformPhysical().IterateDirectory(*CachePathRoot,
		[&CachePrefix, &CacheFoldersToDelete] (const TCHAR* FilenameOrDirectory, bool bIsDirectory)
		{
			if (bIsDirectory)
			{
				FString DirName(FilenameOrDirectory);
				if (FPaths::GetPathLeaf(DirName).StartsWith(CachePrefix))
				{
					// Clear out cache dirs that contain unwanted folders and files automatically downloaded by Chromium's component updater but not used by UE
					// "Crowd Deny" is one such folder, it contains files with lists of inappropriate domains (only created by CEF 128 before the component updater was disabled)
					if (FPaths::DirectoryExists(FPaths::Combine(DirName, TEXT("Crowd Deny"))))
					{
						CacheFoldersToDelete.Add(DirName);
					}
				}
			}
			return true;
		}
	);

	// When FPaths::ProjectSavedDir() points to the .uproject folder, check for and clean up any leftover webcache folders in there.
	// (webcache folders are now always stored under FPlatformProcess::UserSettingsDir())
	if (!FPaths::ShouldSaveToUserDir() && FPaths::ProjectSavedDir() != CachePathRoot)
	{
		IPlatformFile::GetPlatformPhysical().IterateDirectory(*FPaths::ProjectSavedDir(),
			[&CachePrefix, &CacheFoldersToDelete](const TCHAR* FilenameOrDirectory, bool bIsDirectory)
			{
				if (bIsDirectory)
				{
					FString DirName(FilenameOrDirectory);
					if (FPaths::GetPathLeaf(DirName).StartsWith(CachePrefix))
					{
						CacheFoldersToDelete.Add(DirName);
					}
				}
				return true;
			}
		);
	}

	for (const FString& CacheFolder: CacheFoldersToDelete)
	{
		UE_LOGF(LogWebBrowser, Log, "Deleting outdated webcache folder %ls", *CacheFolder);
		IPlatformFile::GetPlatformPhysical().DeleteDirectoryRecursively(*CacheFolder);
	}
#endif
}

bool FWebBrowserSingleton::RegisterContext(const FBrowserContextSettings& Settings)
{
#if WITH_CEF3
	if (bCEFInitialized)
	{
		const CefRefPtr<CefRequestContext>* ExistingContext = RequestContexts.Find(Settings.Id);

		if (ExistingContext != nullptr)
		{
			// You can't register the same context twice and
			// you can't update the settings for a context that already exists
			return false;
		}

		CefRequestContextSettings RequestContextSettings;
		CefString(&RequestContextSettings.accept_language_list) = Settings.AcceptLanguageList.IsEmpty() ? TCHAR_TO_WCHAR(*GetCurrentLocaleCode()) : TCHAR_TO_WCHAR(*Settings.AcceptLanguageList);
		CefString(&RequestContextSettings.cache_path) = TCHAR_TO_WCHAR(*GenerateWebCacheFolderName(Settings.CookieStorageLocation));
		RequestContextSettings.persist_session_cookies = Settings.bPersistSessionCookies;

		//Create a new one
		CefRefPtr<FCEFResourceContextHandler> ResourceContextHandler = new FCEFResourceContextHandler(this);
		ResourceContextHandler->OnBeforeLoad() = Settings.OnBeforeContextResourceLoad;
		RequestResourceHandlers.Add(Settings.Id, ResourceContextHandler);
		CefRefPtr<CefRequestContext> RequestContext = CefRequestContext::CreateContext(RequestContextSettings, ResourceContextHandler);
		RequestContexts.Add(Settings.Id, RequestContext);
		SchemeHandlerFactories.RegisterFactoriesWith(RequestContext);
		UE_LOGF(LogWebBrowser, Log, "Registering ContextId=%ls.", *Settings.Id);
		return true;
	}
#endif
	return false;
}

bool FWebBrowserSingleton::UnregisterContext(const FString& ContextId)
{
#if WITH_CEF3
	bool bFoundContext = false;
	if (bCEFInitialized)
	{
		UE_LOGF(LogWebBrowser, Log, "Unregistering ContextId=%ls.", *ContextId);

		WaitForTaskQueueFlush();
	
		CefRefPtr<CefRequestContext> Context;
		if (RequestContexts.RemoveAndCopyValue(ContextId, Context))
		{
			bFoundContext = true;
			Context->ClearSchemeHandlerFactories();
		}

		CefRefPtr<FCEFResourceContextHandler> ResourceHandler;
		if (RequestResourceHandlers.RemoveAndCopyValue(ContextId, ResourceHandler))
		{
			ResourceHandler->OnBeforeLoad().Unbind();
		}
	}
	return bFoundContext;
#else
	return false;
#endif
}

bool FWebBrowserSingleton::RegisterSchemeHandlerFactory(FString Scheme, FString Domain, IWebBrowserSchemeHandlerFactory* WebBrowserSchemeHandlerFactory)
{
#if WITH_CEF3
	if (bCEFInitialized)
	{
		SchemeHandlerFactories.AddSchemeHandlerFactory(MoveTemp(Scheme), MoveTemp(Domain), WebBrowserSchemeHandlerFactory);
		return true;
	}
#endif
	return false;
}

bool FWebBrowserSingleton::UnregisterSchemeHandlerFactory(IWebBrowserSchemeHandlerFactory* WebBrowserSchemeHandlerFactory)
{
#if WITH_CEF3
	if (bCEFInitialized)
	{
		SchemeHandlerFactories.RemoveSchemeHandlerFactory(WebBrowserSchemeHandlerFactory);
		return true;
	}
#endif
	return false;
}

// Cleanup macros to avoid having them leak outside this source file
#undef CEF3_BIN_DIR
#undef CEF3_FRAMEWORK_DIR
#undef CEF3_RESOURCES_DIR
#undef CEF3_SUBPROCES_EXE
