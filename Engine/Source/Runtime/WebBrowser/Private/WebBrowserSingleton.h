// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Ticker.h"
#include "IWebBrowserSingleton.h"

#if WITH_ENGINE
#include "Materials/Material.h"
#endif

#if WITH_CEF3
#if PLATFORM_WINDOWS
	#include "Windows/AllowWindowsPlatformTypes.h"
	#include "Windows/AllowWindowsPlatformAtomics.h"
#endif
#pragma push_macro("OVERRIDE")
#undef OVERRIDE // cef headers provide their own OVERRIDE macro
THIRD_PARTY_INCLUDES_START
#if PLATFORM_APPLE
PRAGMA_DISABLE_DEPRECATION_WARNINGS
#endif
#include "include/internal/cef_ptr.h"
#include "include/cef_request_context.h"
#if PLATFORM_APPLE
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
THIRD_PARTY_INCLUDES_END
#pragma pop_macro("OVERRIDE")
#if PLATFORM_WINDOWS
	#include "Windows/HideWindowsPlatformAtomics.h"
	#include "Windows/HideWindowsPlatformTypes.h"
#endif
#include "CEF/CEFSchemeHandler.h"
#include "CEF/CEFResourceContextHandler.h"
class CefListValue;
class FCEFBrowserApp;
class FCEFWebBrowserWindow;
#endif

class IWebBrowserCookieManager;
class IWebBrowserWindow;
struct FWebBrowserWindowInfo;
struct FWebBrowserInitSettings;
class UMaterialInterface;

PRAGMA_DISABLE_DEPRECATION_WARNINGS

/**
 * Implementation of singleton class that takes care of general web browser tasks
 */
class FWebBrowserSingleton
	: public IWebBrowserSingleton
	, public FTSTickerObjectBase
{
public:

	/** Constructor. */
	FWebBrowserSingleton(const FWebBrowserInitSettings& WebBrowserInitSettings);

	/** Virtual destructor. */
	virtual ~FWebBrowserSingleton();

	/**
	* Gets the Current Locale Code in the format CEF expects
	*
	* @return Locale code as either "xx" or "xx-YY"
	*/
	static FString GetCurrentLocaleCode();


public:

	// IWebBrowserSingleton Interface

	virtual bool IsShuttingDown() const override;

	virtual FString ApplicationCacheDir() const override;

	virtual TSharedRef<IWebBrowserWindowFactory> GetWebBrowserWindowFactory() const override;

	TSharedPtr<IWebBrowserWindow> CreateBrowserWindow(
		TSharedPtr<FCEFWebBrowserWindow>& BrowserWindowParent,
		TSharedPtr<FWebBrowserWindowInfo>& BrowserWindowInfo) override;

	TSharedPtr<IWebBrowserWindow> CreateBrowserWindow(const FCreateBrowserWindowSettings& Settings) override;

#if	BUILD_EMBEDDED_APP
	TSharedPtr<IWebBrowserWindow> CreateNativeBrowserProxy() override;
#endif

	virtual TSharedPtr<IWebBrowserCookieManager> GetCookieManager() const override
	{
		return DefaultCookieManager;
	}

	virtual TSharedPtr<IWebBrowserCookieManager> GetCookieManager(TOptional<FString> ContextId) const override;

	virtual bool RegisterContext(const FBrowserContextSettings& Settings) override;

	virtual bool UnregisterContext(const FString& ContextId) override;

	virtual bool RegisterSchemeHandlerFactory(FString Scheme, FString Domain, IWebBrowserSchemeHandlerFactory* WebBrowserSchemeHandlerFactory) override;

	virtual bool UnregisterSchemeHandlerFactory(IWebBrowserSchemeHandlerFactory* WebBrowserSchemeHandlerFactory) override;

	virtual bool IsDevToolsShortcutEnabled() override
	{
		return bDevToolsShortcutEnabled;
	}

	virtual void SetDevToolsShortcutEnabled(bool Value) override
	{
		bDevToolsShortcutEnabled = Value;
	}

	virtual void SetJSBindingToLoweringEnabled(bool bEnabled) override
	{
		bJSBindingsToLoweringEnabled = bEnabled;
	}

	virtual void ClearOldCacheFolders(const FString& CachePathRoot, const FString& CachePrefix) override;

	/** Set a reference to UWebBrowser's default material*/
	virtual void SetDefaultMaterial(UMaterialInterface* InDefaultMaterial) override
	{
#if WITH_ENGINE
		DefaultMaterial = InDefaultMaterial;
#endif
	}

	/** Set a reference to UWebBrowser's translucent material*/
	virtual void SetDefaultTranslucentMaterial(UMaterialInterface* InDefaultMaterial) override
	{
#if WITH_ENGINE
		DefaultTranslucentMaterial = InDefaultMaterial;
#endif
	}

	/** Get a reference to UWebBrowser's default material*/
	virtual UMaterialInterface* GetDefaultMaterial() override
	{
#if WITH_ENGINE
		return DefaultMaterial.Get();
#else
		return nullptr;
#endif
	}

	/** Get a reference to UWebBrowser's translucent material*/
	virtual UMaterialInterface* GetDefaultTranslucentMaterial() override
	{
#if WITH_ENGINE
		return DefaultTranslucentMaterial.Get();
#else
		return nullptr;
#endif
	}
public:

	// FTSTickerObjectBase Interface

	virtual bool Tick(float DeltaTime) override;

#if WITH_CEF3
	/** Return true if this URL will support adding an Authorization header to it */
	bool URLRequestAllowsCredentials(const FString& URL);
#endif
private:

	/** Critical section for thread safe modification of WindowInterfaces array. */
	FCriticalSection WindowInterfacesCS;

	TSharedRef<IWebBrowserWindowFactory> WebBrowserWindowFactory;

	TSharedPtr<IWebBrowserCookieManager> DefaultCookieManager;

#if WITH_CEF3
	/** Helper function to generate the CEF build unique name for the cache_path */
	FString GenerateWebCacheFolderName(const FString &InputPath);
	/** Helper function that blocks until the CEF task queue has processed a posted task, flushing the queue */
	void WaitForTaskQueueFlush();

	bool bCEFInitialized;
	bool bTaskQueueFlushed;

	/** Pointer to the CEF App implementation */
	CefRefPtr<FCEFBrowserApp>			CEFBrowserApp;

	TMap<FString, CefRefPtr<CefRequestContext>> RequestContexts;
	TMap<FString, CefRefPtr<FCEFResourceContextHandler>> RequestResourceHandlers;
	FCefSchemeHandlerFactories SchemeHandlerFactories;

	/** List of currently existing browser windows */
	TArray<TWeakPtr<FCEFWebBrowserWindow>> WindowInterfaces;
#elif PLATFORM_IOS || PLATFORM_MAC || PLATFORM_SPECIFIC_WEB_BROWSER || (PLATFORM_ANDROID && USE_ANDROID_JNI)
	/** List of currently existing browser windows */
	TArray<TWeakPtr<IWebBrowserWindow>>	WindowInterfaces;
#endif

	/** Application part of user agent string passed to HTTP requests*/
	FString UserAgentApplication;

	bool bDevToolsShortcutEnabled;

	bool bJSBindingsToLoweringEnabled;

	bool bAppIsFocused;

#if WITH_ENGINE
	/** Reference to UWebBrowser's default material*/
	TSoftObjectPtr<UMaterialInterface> DefaultMaterial;

	/** Reference to UWebBrowser's translucent material*/
	TSoftObjectPtr<UMaterialInterface> DefaultTranslucentMaterial;
#endif
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_CEF3

class CefCookieManager;

class FCefWebBrowserCookieManagerFactory
{
public:
	static TSharedRef<IWebBrowserCookieManager> Create(
		const CefRefPtr<CefCookieManager>& CookieManager);
};

#endif
