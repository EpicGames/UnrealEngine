// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebBrowserViewport.h"
#include "Textures/SlateShaderResource.h"
#include "Widgets/SWidget.h"
#include "IWebBrowserWindow.h"
#include "Layout/WidgetPath.h"

#if WITH_CEF3
#include "CEF/CEFWebBrowserWindow.h"
#endif

FIntPoint FWebBrowserViewport::GetSize() const
{
	return (WebBrowserWindow->GetTexture(bIsPopup) != nullptr)
		? FIntPoint(WebBrowserWindow->GetTexture(bIsPopup)->GetWidth(), WebBrowserWindow->GetTexture(bIsPopup)->GetHeight())
		: FIntPoint();
}

FSlateShaderResource* FWebBrowserViewport::GetViewportRenderTargetTexture() const
{
	return WebBrowserWindow->GetTexture(bIsPopup);
}

void FWebBrowserViewport::Tick( const FGeometry& AllottedGeometry, double InCurrentTime, float DeltaTime )
{
	if (!bIsPopup)
	{
		const float DPI = (WebBrowserWindow->GetParentWindow().IsValid() ? WebBrowserWindow->GetParentWindow()->GetNativeWindow()->GetDPIScaleFactor() : 1.0f);
		const float DPIScale = AllottedGeometry.Scale / DPI;
		FVector2D AbsoluteSize = AllottedGeometry.GetLocalSize() * DPIScale;
		WebBrowserWindow->SetViewportSize(AbsoluteSize.IntPoint(), AllottedGeometry.GetAbsolutePosition().IntPoint());

#if WITH_CEF3
		// Forward the AllottedGeometry to the WebBrowserWindow so the IME implementation can use it
		TSharedPtr<FCEFWebBrowserWindow> CefWebBrowserWindow = StaticCastSharedPtr<FCEFWebBrowserWindow>(WebBrowserWindow);
		CefWebBrowserWindow->UpdateCachedGeometry(AllottedGeometry);
#endif
	}
}

bool FWebBrowserViewport::RequiresVsync() const
{
	return false;
}

FCursorReply FWebBrowserViewport::OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent )
{
	return WebBrowserWindow->OnCursorQuery(MyGeometry, CursorEvent);
}

FReply FWebBrowserViewport::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// Capture mouse on left/middle/right button down so that you can drag out of the viewport
	FReply Reply = WebBrowserWindow->OnMouseButtonDown(MyGeometry, MouseEvent, bIsPopup);
	if (MouseEvent.GetPressedButtons().Num() == 1 &&
		(MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton || MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton || MouseEvent.GetEffectingButton() == EKeys::RightMouseButton))
	{
		const FWidgetPath* Path = MouseEvent.GetEventPath();
		if (Path->IsValid())
		{
			TSharedRef<SWidget> TopWidget = Path->Widgets.Last().Widget;
			return Reply.CaptureMouse(TopWidget);
		}
	}
	return Reply;
}

FReply FWebBrowserViewport::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// Release mouse capture when all buttons released
	FReply Reply = WebBrowserWindow->OnMouseButtonUp(MyGeometry, MouseEvent, bIsPopup);
	if (MouseEvent.GetPressedButtons().IsEmpty())
	{
		return Reply.ReleaseMouseCapture();
	}
	return Reply;
}

void FWebBrowserViewport::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
}

void FWebBrowserViewport::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	WebBrowserWindow->OnMouseLeave(MouseEvent);
}

FReply FWebBrowserViewport::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return WebBrowserWindow->OnMouseMove(MyGeometry, MouseEvent, bIsPopup);
}

FReply FWebBrowserViewport::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return WebBrowserWindow->OnMouseWheel(MyGeometry, MouseEvent, bIsPopup);
}

FReply FWebBrowserViewport::OnTouchGesture( const FGeometry& MyGeometry, const FPointerEvent& GestureEvent )
{
	return WebBrowserWindow->OnTouchGesture(MyGeometry, GestureEvent, bIsPopup);
}

FReply FWebBrowserViewport::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	FReply Reply = WebBrowserWindow->OnMouseButtonDoubleClick(InMyGeometry, InMouseEvent, bIsPopup);
	return Reply;
}

FReply FWebBrowserViewport::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	return WebBrowserWindow->OnKeyDown(InKeyEvent) ? FReply::Handled() : FReply::Unhandled();
}

FReply FWebBrowserViewport::OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	return WebBrowserWindow->OnKeyUp(InKeyEvent) ? FReply::Handled() : FReply::Unhandled();
}

FReply FWebBrowserViewport::OnKeyChar( const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent )
{
	return WebBrowserWindow->OnKeyChar(InCharacterEvent) ? FReply::Handled() : FReply::Unhandled();
}

FReply FWebBrowserViewport::OnFocusReceived(const FFocusEvent& InFocusEvent)
{
	WebBrowserWindow->OnFocus(true, bIsPopup);
	return FReply::Handled();
}

void FWebBrowserViewport::OnFocusLost(const FFocusEvent& InFocusEvent)
{
	WebBrowserWindow->OnFocus(false, bIsPopup);
}
