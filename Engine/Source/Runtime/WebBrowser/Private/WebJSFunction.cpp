// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebJSFunction.h"
#include "WebJSScripting.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WebJSFunction)

#if WITH_CEF3
#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/AllowWindowsPlatformAtomics.h"
#endif
#pragma push_macro("OVERRIDE")
#undef OVERRIDE // cef headers provide their own OVERRIDE macro
THIRD_PARTY_INCLUDES_START
#if PLATFORM_APPLE
PRAGMA_DISABLE_DEPRECATION_WARNINGS
#endif
#include "include/cef_values.h"
#if PLATFORM_APPLE
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
THIRD_PARTY_INCLUDES_END
#pragma pop_macro("OVERRIDE")
#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformAtomics.h"
#include "Windows/HideWindowsPlatformTypes.h"
#endif
#endif

FWebJSParam::~FWebJSParam()
{
	// Since the FString, StructWrapper, TArray, and TMap members are in a union, they may or may not be valid, so we have to call the destructors manually.
	switch (Tag)
	{
		case PTYPE_STRING:
			delete StringValue;
			break;
		case PTYPE_STRUCT:
			delete StructValue;
			break;
		case PTYPE_ARRAY:
			delete ArrayValue;
			break;
		case PTYPE_MAP:
			delete MapValue;
			break;
		default:
			break;
	}
}

FWebJSParam::FWebJSParam(const FWebJSParam& Other)
	: Tag(Other.Tag)
{
	switch (Other.Tag)
	{
		case PTYPE_BOOL:
			BoolValue = Other.BoolValue;
			break;
		case PTYPE_DOUBLE:
			DoubleValue = Other.DoubleValue;
			break;
		case PTYPE_INT:
			IntValue = Other.IntValue;
			break;
		case PTYPE_STRING:
			StringValue = new FString(*Other.StringValue);
			break;
		case PTYPE_NULL:
			break;
		case PTYPE_OBJECT:
			ObjectValue = Other.ObjectValue;
			break;
		case PTYPE_STRUCT:
			StructValue = Other.StructValue->Clone();
			break;
		case PTYPE_ARRAY:
			ArrayValue = new TArray<FWebJSParam>(*Other.ArrayValue);
			break;
		case PTYPE_MAP:
			MapValue = new TMap<FString, FWebJSParam>(*Other.MapValue);
			break;
	}
}

FWebJSParam::FWebJSParam(FWebJSParam&& Other)
	: Tag(Other.Tag)
{
	switch (Other.Tag)
	{
	case PTYPE_BOOL:
		BoolValue = Other.BoolValue;
		break;
	case PTYPE_DOUBLE:
		DoubleValue = Other.DoubleValue;
		break;
	case PTYPE_INT:
		IntValue = Other.IntValue;
		break;
	case PTYPE_STRING:
		StringValue = Other.StringValue;
		Other.StringValue = nullptr;
		break;
	case PTYPE_NULL:
		break;
	case PTYPE_OBJECT:
		ObjectValue = Other.ObjectValue;
		Other.ObjectValue = nullptr;
		break;
	case PTYPE_STRUCT:
		StructValue = Other.StructValue;
		Other.StructValue = nullptr;
		break;
	case PTYPE_ARRAY:
		ArrayValue = Other.ArrayValue;
		Other.ArrayValue = nullptr;
		break;
	case PTYPE_MAP:
		MapValue = Other.MapValue;
		Other.MapValue = nullptr;
		break;
	}

	Other.Tag = PTYPE_NULL;
}

void FWebJSCallbackBase::Invoke(int32 ArgCount, FWebJSParam Arguments[], bool bIsError) const
{
	TSharedPtr<FWebJSScripting> Scripting = ScriptingPtr.Pin();
	if (Scripting.IsValid())
	{
		Scripting->InvokeJSFunction(CallbackId, ArgCount, Arguments, bIsError);
	}
}

