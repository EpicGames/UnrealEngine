// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "WebJSFunction.h"
#include "UObject/GCObject.h"

class Error;

/**
 * Implements handling of bridging UObjects client side with JavaScript renderer side.
 */
class FWebJSScripting
	: public FGCObject
{
public:
	FWebJSScripting(bool bInJSBindingToLoweringEnabled)
		: BaseGuid(FGuid::NewGuid())
		, bJSBindingToLoweringEnabled(bInJSBindingToLoweringEnabled)
	{}

	virtual void BindUObject(const FString& Name, UObject* Object, bool bIsPermanent = true) =0;
	virtual void UnbindUObject(const FString& Name, UObject* Object = nullptr, bool bIsPermanent = true) =0;


	virtual void InvokeJSFunction(FGuid FunctionId, int32 ArgCount, FWebJSParam Arguments[], bool bIsError=false) =0;
	virtual void InvokeJSErrorResult(FGuid FunctionId, const FString& Error) =0;

	FString GetBindingName(const FString& Name, UObject* Object) const
	{
		return bJSBindingToLoweringEnabled ? Name.ToLower() : Name;
	}

	FString GetBindingName(const FFieldVariant& Property) const
	{
		return bJSBindingToLoweringEnabled ? Property.GetName().ToLower() : Property.GetName();
	}

public:

	// FGCObject API
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override
	{
		// Ensure bound UObjects are not garbage collected as long as this object is valid.
		for (auto& Binding : BoundObjects)
		{
			Collector.AddReferencedObject(Binding.Key);
		}
	}
	virtual FString GetReferencerName() const override
	{
		return TEXT("FWebJSScripting");
	}

protected:
	// Creates a reversible memory addres -> psuedo-guid mapping.
	// This is done by xoring the address with the first 64 bits of a base guid owned by the instance.
	// Used to identify UObjects from the render process withough exposing internal pointers.
	FGuid PtrToGuid(UObject* Ptr)
	{
		FGuid Guid = BaseGuid;
		if (Ptr == nullptr)
		{
			Guid.Invalidate();
		}
		else
		{
			UPTRINT IntPtr = reinterpret_cast<UPTRINT>(Ptr);
			if (sizeof(UPTRINT) > 4)
			{
				Guid[0] ^= (static_cast<uint64>(IntPtr) >> 32);
			}
			Guid[1] ^= IntPtr & 0xFFFFFFFF;
		}
		return Guid;
	}

	// In addition to reversing the mapping, it verifies that we are currently holding on to an instance of that UObject
	UObject* GuidToPtr(const FGuid& Guid)
	{
		UPTRINT IntPtr = 0;
		if (sizeof(UPTRINT) > 4)
		{
			IntPtr = static_cast<UPTRINT>(static_cast<uint64>(Guid[0] ^ BaseGuid[0]) << 32);
		}
		IntPtr |= (Guid[1] ^ BaseGuid[1]) & 0xFFFFFFFF;

		UObject* Result = reinterpret_cast<UObject*>(IntPtr);
		if (BoundObjects.Contains(Result))
		{
			return Result;
		}
		else
		{
			return nullptr;
		}
	}

	void RetainBinding(UObject* Object)
	{
		if (BoundObjects.Contains(Object))
		{
			if(!BoundObjects[Object].bIsPermanent)
			{
				BoundObjects[Object].Refcount++;
			}
		}
		else
		{
			BoundObjects.Add(Object, {false, 1});
		}
	}

	void ReleaseBinding(UObject* Object)
	{
		if (BoundObjects.Contains(Object))
		{
			ObjectBinding& Binding = BoundObjects[Object];
			if(!Binding.bIsPermanent)
			{
				Binding.Refcount--;
				if (Binding.Refcount <= 0)
				{
					BoundObjects.Remove(Object);
				}
			}
		}
	}

	struct ObjectBinding
	{
		bool bIsPermanent;
		int32 Refcount;
	};

	/** Private data */
	FGuid BaseGuid;

	/** UObjects currently visible on the renderer side. */
	TMap<TObjectPtr<UObject>, ObjectBinding> BoundObjects;

	/** Reverse lookup for permanent bindings */
	TMap<FString, UObject*> PermanentUObjectsByName;

	/** The to-lowering option enable for the binding names. */
	const bool bJSBindingToLoweringEnabled;
};
