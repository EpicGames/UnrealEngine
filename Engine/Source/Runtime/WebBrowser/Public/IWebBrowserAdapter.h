// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IWebBrowserWindow.h"

class IWebBrowserAdapter
{
public:

	virtual FString GetName() const = 0;

	virtual bool IsPermanent() const = 0;

	virtual void ConnectTo(const TSharedRef<IWebBrowserWindow>& BrowserWindow) = 0;

	virtual void DisconnectFrom(const TSharedRef<IWebBrowserWindow>& BrowserWindow) = 0;

};

class FWebBrowserAdapterFactory 
{ 
public: 

	static WEBBROWSER_API TSharedRef<IWebBrowserAdapter> Create(const FString& Name, UObject* JSBridge, bool IsPermanent); 

	static WEBBROWSER_API TSharedRef<IWebBrowserAdapter> Create(const FString& Name, UObject* JSBridge, bool IsPermanent, const FString& ConnectScriptText, const FString& DisconnectScriptText);
}; 
