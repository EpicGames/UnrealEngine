// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

struct FWebBrowserCookie
{
public:
	// The cookie name.
	FString Name;

	// The cookie value.
	FString Value;

	// If is empty a host cookie will be created instead of a domain
	// cookie. Domain cookies are stored with a leading "." and are visible to
	// sub-domains whereas host cookies are not.
	FString Domain;

	// If is non-empty only URLs at or below the path will get the cookie
	// value.
	FString Path;

	// If true the cookie will only be sent for HTTPS requests.
	bool bSecure;

	// If true the cookie will only be sent for HTTP requests.
	bool bHttpOnly;

	// If true the cookie will expire at the specified Expires datetime.
	bool bHasExpires;

	// The cookie expiration date is only valid if bHasExpires is true.
	FDateTime Expires;
};

class IWebBrowserCookieManager
{
public:
	typedef struct FWebBrowserCookie FCookie;
public:

	/**
	 * Sets a cookie given a valid URL. 
	 *
	 * This function expects each attribute to be well-formed. It will
	 * check for disallowed characters (e.g. the ';' character is disallowed
	 * within the cookie Value field) and fail without setting the cookie if
	 * such characters are found. 
	 *
	 * @param URL The base URL to match when searching for cookies to remove. Use blank to match all URLs.
	 * @param Cookie The struct defining the state of the cookie to set
	 * @param Completed A callback function that will be invoked asynchronously on the game thread when the set is complete passing success bool.
	 */
	virtual void SetCookie(const FString& URL, const FCookie& Cookie, TFunction<void(bool)> Completed = nullptr) = 0;

	/**
	 * Delete all browser cookies.
	 *
	 * Removes all matching cookies. Leave both URL and CookieName blank to delete the entire cookie database.
	 * The actual deletion will be scheduled on the browser IO thread, so the operation may not have completed when the function returns.
	 *
	 * @param URL The base URL to match when searching for cookies to remove. Use blank to match all URLs.
	 * @param CookieName The name of the cookie to delete. If left unspecified, all cookies will be removed.
	 * @param Completed A callback function that will be invoked asynchronously on the game thread when the deletion is complete passing in the number of deleted objects.
	 */
	virtual void DeleteCookies(const FString& URL = TEXT(""), const FString& CookieName = TEXT(""), TFunction<void(int)> Completed = nullptr) = 0;
};