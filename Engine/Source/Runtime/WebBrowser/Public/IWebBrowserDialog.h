// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Type of browser dialog to show.
 */

enum class EWebBrowserDialogType
{
	/** Alert dialog. Used to show a message to a user. It should only have a single button to dismiss it. */
	Alert  = 0,
	/** Confirm dialog. Shows a message to the user. It should have two buttons to either accept or decline the prompt. */
	Confirm,
	/** Prompt dialog. Shows a prompt asking for user input. The user can enter text and either confirm or dismiss it. */
	Prompt,
	/** Unload dialog. This type of dialog is shown to confirm navigating away from a page containing user-edited content. */
	Unload = 127,
};

/**
 * Return value from dialog event handle specifying what action should be taken.
 */
enum class EWebBrowserDialogEventResponse
{
	/** Return Unhandled to use the default dialog implementation. This is the default behavior when no handler is attached. */
	Unhandled,
	/** Do not show any dialog and return as if the user accepted the action. */
	Continue,
	/** Do not show any dialog and return as if the user dismissed the action. */
	Ignore,
	/** The event handler will take care of showing the dialog. It must call IWebBrowserDialog::Continue once it has been dismissed. */
	Handled,
};

/**
 * Browser dialog parameters passed to OnBrowserDialog event handlers.
 */
class IWebBrowserDialog
{
public:

	virtual ~IWebBrowserDialog() = default;

	/** 
	 * What kind of dialog should be shown.
	 * @return the dialog type 
	 */
	virtual EWebBrowserDialogType GetType() = 0;

	/**
	 * Tell the browser to continue with the result of the dialog.
	 * If this method is used, the original event handler must return EWebBrowserDialogEventResponse::Handled.
	 *
	 * @param Success Did the user accept the dialog or not?
	 * @param UserResponse Only for Prompt dialog, the text entered by the user.
	 */
	virtual void Continue(bool Success = true, const FText& UserResponse = FText::GetEmpty()) = 0;

	/**
	 * Get the dialog message.
	 * @return the message to show in the dialog.
	 */
	virtual const FText& GetMessageText() = 0;

	/**
	 * Only valid for Prompt dialogs.
	 * @return the default value to show in the text entry box.
	 */
	virtual const FText& GetDefaultPrompt() = 0;

	/**
	 * Only valid for Unload dialogs.
	 * @return true if the dialog is confirming a reload of the current page.
	 */
	virtual bool IsReload() = 0;

	
};
