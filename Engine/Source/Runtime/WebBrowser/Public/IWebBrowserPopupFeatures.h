// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class IWebBrowserPopupFeatures
{
public:
	virtual ~IWebBrowserPopupFeatures() = default;

	virtual int GetX() const = 0;
	virtual bool IsXSet() const = 0;
	virtual int GetY() const = 0;
	virtual bool IsYSet() const = 0;
	virtual int GetWidth() const = 0;
	virtual bool IsWidthSet() const = 0;
	virtual int GetHeight() const = 0;
	virtual bool IsHeightSet() const = 0;
	virtual bool IsMenuBarVisible() const = 0;
	virtual bool IsStatusBarVisible() const = 0;
	virtual bool IsToolBarVisible() const = 0;
	virtual bool IsLocationBarVisible() const = 0;
	virtual bool IsScrollbarsVisible() const = 0;
	virtual bool IsResizable() const = 0;
	virtual bool IsFullscreen() const = 0;
	virtual bool IsDialog() const = 0;
	virtual TArray<FString> GetAdditionalFeatures() const = 0;
};
