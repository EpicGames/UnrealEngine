// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

typedef TMap<FString, FString> FContextRequestHeaders;
DECLARE_DELEGATE_FourParams(FOnBeforeContextResourceLoadDelegate, FString /*Url*/, FString /*ResourceType*/, FContextRequestHeaders& /*AdditionalHeaders*/, const bool /*AllowUserCredentials*/);
