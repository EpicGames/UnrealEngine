// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

/**
 * This is the interface that needs to be implemented to handle a request made via a custom scheme.
 * It will be created by implementing an IWebBrowserSchemeHandlerFactory, given to the web browser singleton.
 */
class IWebBrowserSchemeHandler
{
public:
	/**
	 * An interface for setting response headers emulating a http implementation.
	 */
	class IHeaders
	{
	public:
		/**
		 * Sets the mime type for the response.
		 * @param   MimeType        The Mime Type.
		 */
		virtual void SetMimeType(const TCHAR* MimeType) = 0;

		/**
		 * Sets the status code for the response.
		 * @param   StatusCode      The status code.
		 */
		virtual void SetStatusCode(int32 StatusCode) = 0;

		/**
		 * Sets the content length for the response.
		 * @param   ContentLength   The size of the response content in bytes.
		 */
		virtual void SetContentLength(int32 ContentLength) = 0;

		/**
		 * Sets a redirect url for the response. Other calls will be ignored if this is used.
		 * @param   Url             The url to redirect to.
		 */
		virtual void SetRedirect(const TCHAR* Url) = 0;

		/**
		 * Sets a header for the response.
		 * @param   Key             The header key.
		 * @param   Value           The header value.
		 */
		virtual void SetHeader(const TCHAR* Key, const TCHAR* Value) = 0;
	};

public:
	virtual ~IWebBrowserSchemeHandler() = default;

	/**
	 * Process an incoming request.
	 * @param   Verb            This is the verb used for the request (GET, PUT, POST, etc).
	 * @param   Url             This is the full url for the request being made.
	 * @param   OnHeadersReady  You must execute this delegate once the response headers are ready to be retrieved with GetResponseHeaders.
	 *                            You may execute it during this call to state headers are available now.
	 * @return You should return true if the request has been accepted and will be processed, otherwise false to cancel this request.
	 */
	virtual bool ProcessRequest(const FString& Verb, const FString& Url, const FSimpleDelegate& OnHeadersReady) = 0;

	/**
	 * Retrieves the headers for this request.
	 * @param   OutHeaders      The interface to use to set headers.
	 */
	virtual void GetResponseHeaders(IHeaders& OutHeaders) = 0;

	/**
	 * Retrieves the headers for this request.
	 * @param   OutBytes            You should copy up to BytesToRead of data to this ptr.
	 * @param   BytesToRead         The maximum number of bytes that can be copied to OutBytes.
	 * @param   BytesRead           You should set this to the number of bytes that were copied.
	 *                                This can be set to zero, to indicate more data is not ready yet, and OnMoreDataReady must then be
	 *                                executed when there is.
	 * @param   OnMoreDataReady     You should execute this delegate when more data is available to read.
	 * @return You should return true if more data needs to be read, otherwise false if this is the end of the response data.
	 */
	virtual bool ReadResponse(uint8* OutBytes, int32 BytesToRead, int32& BytesRead, const FSimpleDelegate& OnMoreDataReady) = 0;

	/**
	 * Called if the request should be canceled.
	 */
	virtual void Cancel() = 0;
};

/**
 * This is the interface that needs to be implemented to instantiate a scheme request handler.
 */
class IWebBrowserSchemeHandlerFactory
{
public:
	virtual ~IWebBrowserSchemeHandlerFactory() = default;

	/**
	 * Instantiates an appropriate handler for the given request details.
	 * @param   Verb            This is the verb used for the request (GET, PUT, POST, etc).
	 * @param   Url             This is the full url for the request being made.
	 */
	virtual TUniquePtr<IWebBrowserSchemeHandler> Create(FString Verb, FString Url) = 0;
};
