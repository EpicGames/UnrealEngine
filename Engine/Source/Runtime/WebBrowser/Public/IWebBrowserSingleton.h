// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Rendering/SlateRenderer.h"
#include "IWebBrowserResourceLoader.h"
class FCEFWebBrowserWindow;
class IWebBrowserCookieManager;
class IWebBrowserWindow;
class IWebBrowserSchemeHandlerFactory;
class UMaterialInterface;
struct FWebBrowserWindowInfo;

class IWebBrowserWindowFactory
{
public:

	virtual TSharedPtr<IWebBrowserWindow> Create(
		TSharedPtr<FCEFWebBrowserWindow>& BrowserWindowParent,
		TSharedPtr<FWebBrowserWindowInfo>& BrowserWindowInfo) = 0;

	virtual TSharedPtr<IWebBrowserWindow> Create(
		void* OSWindowHandle,
		FString InitialURL,
		bool bUseTransparency,
		bool bThumbMouseButtonNavigation,
		bool InterceptLoadRequests = true,
		TOptional<FString> ContentsToLoad = TOptional<FString>(),
		TOptional<FString> UserAgentApplication = TOptional<FString>(),
		bool ShowErrorMessage = true,
		FColor BackgroundColor = FColor(255, 255, 255, 255),
		bool bMobileJSReturnInDict = true) = 0;
};

struct FBrowserContextSettings
{
	FBrowserContextSettings(const FString& InId)
		: Id(InId)
		, AcceptLanguageList()
		, CookieStorageLocation()
		, bPersistSessionCookies(false)
		, bIgnoreCertificateErrors(false)
		, bEnableNetSecurityExpiration(true)
	{ }

	FString Id;
	FString AcceptLanguageList;
	FString CookieStorageLocation;
	bool bPersistSessionCookies;
	bool bIgnoreCertificateErrors;
	bool bEnableNetSecurityExpiration;
	FOnBeforeContextResourceLoadDelegate OnBeforeContextResourceLoad;
};


struct FCreateBrowserWindowSettings
{
	void* OSWindowHandle = nullptr;
	FString InitialURL;
	FColor BackgroundColor = FColor(255, 255, 255, 255);
	int BrowserFrameRate = 24;
	bool bUseTransparency = false;
	bool bInterceptLoadRequests = true;
	bool bThumbMouseButtonNavigation = false;
	bool bShowErrorMessage = true;
	bool bMobileJSReturnInDict = true;
    /** Whether to enable the W3C PaymentRequest API in this WebView. */
    bool bEnablePaymentRequest = false;
	TOptional<FBrowserContextSettings> Context;
	TOptional<FString> ContentsToLoad;
	TOptional<FString> UserAgentApplication;
	TArray<FString> AltRetryDomains;
};

/**
 * A singleton class that takes care of general web browser tasks
 */
class IWebBrowserSingleton
{
public:
	/**
	 * Virtual Destructor
	 */
	virtual ~IWebBrowserSingleton() = default;

	/** Indicates whether the browser engine is in the shutdown phase. */
	virtual bool IsShuttingDown() const = 0;

	/** @return A factory object that can be used to construct additional WebBrowserWindows on demand. */
	virtual TSharedRef<IWebBrowserWindowFactory> GetWebBrowserWindowFactory() const = 0;


	/**
	 * Create a new web browser window
	 *
	 * @param BrowserWindowParent The parent browser window
	 * @param BrowserWindowInfo Info for setting up the new browser window
	 * @return New Web Browser Window Interface (may be null if not supported)
	 */
	virtual TSharedPtr<IWebBrowserWindow> CreateBrowserWindow(
		TSharedPtr<FCEFWebBrowserWindow>& BrowserWindowParent,
		TSharedPtr<FWebBrowserWindowInfo>& BrowserWindowInfo
		) = 0;

	/**
	 * Create a new web browser window
	 *
	 * @param Settings Struct containing browser window settings
	 * @return New Web Browser Window Interface (may be null if not supported)
	 */
	virtual TSharedPtr<IWebBrowserWindow> CreateBrowserWindow(const FCreateBrowserWindowSettings& Settings) = 0;

#if	BUILD_EMBEDDED_APP
	virtual TSharedPtr<IWebBrowserWindow> CreateNativeBrowserProxy() = 0;
#endif

	virtual TSharedPtr<class IWebBrowserCookieManager> GetCookieManager() const = 0;

	virtual TSharedPtr<class IWebBrowserCookieManager> GetCookieManager(TOptional<FString> ContextId) const = 0;

	virtual bool RegisterContext(const FBrowserContextSettings& Settings) = 0;

	virtual bool UnregisterContext(const FString& ContextId) = 0;

	// @return the application cache dir where the cookies are stored
	virtual FString ApplicationCacheDir() const = 0;
	/**
	 * Registers a custom scheme handler factory, for a given scheme and domain. The domain is ignored if the scheme is not a browser built in scheme
	 * and all requests will go through this factory.
	 * @param Scheme                            The scheme name to handle.
	 * @param Domain                            The domain name to handle.
	 * @param WebBrowserSchemeHandlerFactory    The factory implementation for creating request handlers for this scheme.
	 */
	virtual bool RegisterSchemeHandlerFactory(FString Scheme, FString Domain, IWebBrowserSchemeHandlerFactory* WebBrowserSchemeHandlerFactory) = 0;

	/**
	 * Unregister a custom scheme handler factory. The factory may still be used by existing open browser windows, but will no longer be provided for new ones.
	 * @param WebBrowserSchemeHandlerFactory    The factory implementation to remove.
	 */
	virtual bool UnregisterSchemeHandlerFactory(IWebBrowserSchemeHandlerFactory* WebBrowserSchemeHandlerFactory) = 0;

	/**
	 * Enable or disable CTRL/CMD-SHIFT-I shortcut to show the Chromium Dev tools window.
	 * The value defaults to true on debug builds, otherwise false.
	 *
	 * The relevant handlers for spawning new browser windows have to be set up correctly in addition to this flag being true before anything is shown.
	 *
	 * @param Value a boolean value to enable or disable the keyboard shortcut.
	 */
	virtual void SetDevToolsShortcutEnabled(bool Value) = 0;


	/**
	 * Returns whether the CTRL/CMD-SHIFT-I shortcut to show the Chromium Dev tools window is enabled.
	 *
	 * The relevant handlers for spawning new browser windows have to be set up correctly in addition to this flag being true before anything is shown.
	 *
	 * @return a boolean value indicating whether the keyboard shortcut is enabled or not.
	 */
	virtual bool IsDevToolsShortcutEnabled() = 0;


	/**
	 * Enable or disable to-lowering of JavaScript object member bindings.
	 *
	 * Due to how JavaScript to UObject bridges require the use of FNames for building up the JS API objects, it is possible for case-sensitivity issues
	 * to develop if an FName has been previously created with differing case to your function or property names. To-lowering the member names allows
	 * a guaranteed casing for the web page's JS to reference.
	 *
	 * Default behavior is enabled, so that all JS side objects have only lowercase members.
	 *
	 * @param bEnabled a boolean value to enable or disable the to-lowering.
	 */
	virtual void SetJSBindingToLoweringEnabled(bool bEnabled) = 0;


	/**
	 * Delete old/unused web cache paths. Some Web implementations (i.e CEF) use version specific cache folders, this
	 * call lets you schedule a deletion of any now unused folders. Calling this may resulting in async disk I/O.
	 *
	 * @param CachePathRoot the base path used for saving the webcache folder
	 * @param CachePrefix the filename prefix we use for the cache folder (i.e "webcache")
	 */
	virtual void ClearOldCacheFolders(const FString &CachePathRoot, const FString &CachePrefix) = 0;


	/** Set a reference to UWebBrowser's default material*/
	virtual void SetDefaultMaterial(UMaterialInterface* InDefaultMaterial) = 0;
	/** Set a reference to UWebBrowser's translucent material*/
	virtual void SetDefaultTranslucentMaterial(UMaterialInterface* InDefaultMaterial) = 0;

	/** Get a reference to UWebBrowser's default material*/
	virtual UMaterialInterface* GetDefaultMaterial() = 0;
	/** Get a reference to UWebBrowser's transparent material*/
	virtual UMaterialInterface* GetDefaultTranslucentMaterial() = 0;
};
