// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/CursorReply.h"
#include "Input/DragAndDrop.h"
#include "Input/Reply.h"
#include "Widgets/SWindow.h"
#include "SWebBrowser.h"

class Error;
class FSlateShaderResource;
class IWebBrowserDialog;
class IWebBrowserPopupFeatures;
enum class EWebBrowserDialogEventResponse;

enum class EWebBrowserDocumentState
{
	Completed,
	Error,
	Loading,
	NoDocument
};

enum class EWebBrowserConsoleLogSeverity
{
	Default,
	Verbose,
	Debug,
	Info,
	Warning,
	Error,
	Fatal
};


enum class EWebTransitionSource
{
	Unknown,
	/** Source is a link click or the JavaScript window.open function. */
	Link,
	/** Source is some other "explicit" navigation action such as creating a new browser or using the LoadURL function. */
	Explicit,
	/** Source is a subframe navigation. */
	AutoSubframe,
	/** Source is a subframe navigation explicitly requested by the user. */
	ManualSubframe,
	/** Source is a form submission by the user. */
	FormSubmit,
	/** Source is a "reload" of the page via the Reload function */
	Reload
};

enum class EWebTransitionSourceQualifier
{
	Unknown,
	/** Attempted to visit a URL but was blocked. */
	Blocked,
	/** Used the Forward or Back function to navigate among browsing history. */
	ForwardBack,
	/** The beginning of a navigation chain. */
	ChainStart,
	/** The last transition in a redirect chain. */
	ChainEnd,
	/** Redirects caused by JavaScript or a meta refresh tag on the page. */
	ClientRedirect,
	/** Used to test whether a transition involves a redirect. */
	ServerRedirect
};


struct FWebNavigationRequest
{
	bool bIsRedirect;
	bool bIsMainFrame;
	bool bIsExplicitTransition;
	EWebTransitionSource TransitionSource;
	EWebTransitionSourceQualifier TransitionSourceQualifier;
};

/**
 * Interface for dealing with a Web Browser window
 */
class IWebBrowserWindow
{
public:

	/**
	 * Load the specified URL
	 *
	 * @param NewURL New URL to load
	 */
	virtual void LoadURL(FString NewURL) = 0;

	/**
	 * Load a string as data to create a web page
	 *
	 * @param Contents String to load
	 * @param DummyURL Dummy URL for the page
	 */
	virtual void LoadString(FString Contents, FString DummyURL) = 0;

	/**
	 * Set the desired size of the web browser viewport
	 * 
	 * @param WindowSize Desired viewport size
	 */
	virtual void SetViewportSize(FIntPoint WindowSize, FIntPoint WindowPos = FIntPoint::NoneValue) = 0;

	/**
	* Gets the current size of the web browser viewport if available, FIntPoint::NoneValue otherwise
	*
	* @param WindowSize Desired viewport size
	*/
	virtual FIntPoint GetViewportSize() const = 0;

	/**
	 * Gets interface to the texture representation of the browser
	 *
	 * @param bISpopup Whether to return the popup menu texture instead of the main browser window.
	 * @return A slate shader resource that can be rendered
	 */
	virtual FSlateShaderResource* GetTexture(bool bIsPopup = false) = 0;

	/**
	 * Checks whether the web browser is valid and ready for use
	 */
	virtual bool IsValid() const = 0;

	/**
	 * Checks whether the web browser has finished loaded the initial page.
	 */
	virtual bool IsInitialized() const = 0;

	/**
	 * Checks whether the web browser is currently being shut down
	 */
	virtual bool IsClosing() const = 0;

	/** Gets the loading state of the current document. */
	virtual EWebBrowserDocumentState GetDocumentLoadingState() const = 0;

	/**
	 * Gets the current title of the Browser page
	 */
	virtual FString GetTitle() const = 0;

	/**
	 * Gets the currently loaded URL.
	 *
	 * @return The URL, or empty string if no document is loaded.
	 */
	virtual FString GetUrl() const = 0;

	/**
	 * Gets the source of the main frame as raw HTML.
	 * 
	 * This method has to be called asynchronously by passing a callback function, which will be called at a later point when the
	 * result is ready.
	 * @param	Callback	A callable that takes a single string reference for handling the result.
	 */
	virtual void GetSource(TFunction<void (const FString&)> Callback) const = 0;

	/**
	 * Notify the browser that a key has been pressed
	 *
	 * @param  InKeyEvent  Key event
	 */
	virtual bool OnKeyDown(const FKeyEvent& InKeyEvent) = 0;

	/**
	 * Notify the browser that a key has been released
	 *
	 * @param  InKeyEvent  Key event
	 */
	virtual bool OnKeyUp(const FKeyEvent& InKeyEvent) = 0;

	/**
	 * Notify the browser of a character event
	 *
	 * @param InCharacterEvent Character event
	 */
	virtual bool OnKeyChar(const FCharacterEvent& InCharacterEvent) = 0;

	/**
	 * Notify the browser that a mouse button was pressed within it
	 *
	 * @param MyGeometry The Geometry of the browser
	 * @param MouseEvent Information about the input event
	 * @param bIsPopup True if the coordinates are relative to a popup menu window, otherwise false.
	 *
	 * @return FReply::Handled() if the mouse event was handled, FReply::Unhandled() oterwise
	 */
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup) = 0;

	/**
	 * Notify the browser that a mouse button was released within it
	 *
	 * @param MyGeometry The Geometry of the browser
	 * @param MouseEvent Information about the input event
	 * @param bIsPopup True if the coordinates are relative to a popup menu window, otherwise false.
	 *
	 * @return FReply::Handled() if the mouse event was handled, FReply::Unhandled() oterwise
	 */
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup) = 0;

	/**
	 * Notify the browser of a double click event
	 *
	 * @param MyGeometry The Geometry of the browser
	 * @param MouseEvent Information about the input event
	 * @param bIsPopup True if the coordinates are relative to a popup menu window, otherwise false.
	 *
	 * @return FReply::Handled() if the mouse event was handled, FReply::Unhandled() oterwise
	 */
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup) = 0;

	/**
	 * Notify the browser that a mouse moved within it
	 *
	 * @param MyGeometry The Geometry of the browser
	 * @param MouseEvent Information about the input event
	 * @param bIsPopup True if the coordinates are relative to a popup menu window, otherwise false.
	 *
	 * @return FReply::Handled() if the mouse event was handled, FReply::Unhandled() oterwise
	 */
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup) = 0;

	/**
	 * Notify the browser that a mouse has left the window
	 *
	 * @param MouseEvent Information about the input event
	 */
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) = 0;

	/**
	 * Sets whether mouse wheel events should be handled by the window
	 */
	virtual void SetSupportsMouseWheel(bool bValue) = 0;

	/**
	 * Returns whether mouse wheel events should be handled by the window
	 */
	virtual bool GetSupportsMouseWheel() const = 0;

	/**
	 * Called when the mouse wheel is spun
	 *
	 * @param MyGeometry The Geometry of the browser
	 * @param MouseEvent Information about the input event
	 * @param bIsPopup True if the coordinates are relative to a popup menu window, otherwise false.
	 *
	 * @return FReply::Handled() if the mouse event was handled, FReply::Unhandled() oterwise
	 */
	virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bIsPopup) = 0;

	/**
	 * Called when a touch gesture is performed.
	 *
	 * @param MyGeometry The Geometry of the browser
	 * @param GestureEvent Information about the input event
	 * @param bIsPopup True if the coordinates are relative to a popup menu window, otherwise false.
	 *
	 * @return FReply::Handled() if the mouse event was handled, FReply::Unhandled() oterwise
	 */
	virtual FReply OnTouchGesture(const FGeometry& MyGeometry, const FPointerEvent& GestureEvent, bool bIsPopup) = 0;

	/**
	 * The system asks each widget under the mouse to provide a cursor. This event is bubbled.
	 * 
	 * @return FCursorReply::Unhandled() if the event is not handled; return FCursorReply::Cursor() otherwise.
	 */
	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) = 0;

	/**
	 * Called when browser receives/loses focus
	 * @param SetFocus Whether the window gained or lost focus.
	 * @param bIsPopup True if the coordinates are relative to a popup menu window, otherwise false.
	 */
	virtual void OnFocus(bool SetFocus, bool bIsPopup) = 0;

	/** Called when Capture lost */
	virtual void OnCaptureLost() = 0;

	/**
	 * Notify the browser that a drag operation has entered the window.
	 *
	 * @param MyGeometry The Geometry of the browser
	 * @param DragDropEvent Information about the drag event
	 */
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) = 0;

	/**
	 * Notify the browser that a drag operation is moving over the window.
	 *
	 * @param MyGeometry The Geometry of the browser
	 * @param DragDropEvent Information about the drag event
	 * @return FReply indicating how the event was handled
	 */
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) = 0;

	/**
	 * Notify the browser that a drag operation has left the window.
	 *
	 * @param DragDropEvent Information about the drag event
	 */
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) = 0;

	/**
	 * Notify the browser that a drag operation has been released on it.
	 *
	 * @param MyGeometry The Geometry of the browser
	 * @param DragDropEvent Information about the drag event
	 * @return FReply indicating how the event was handled
	 */
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) = 0;

	/**
	 * Returns true if the browser can navigate backwards.
	 */
	virtual bool CanGoBack() const = 0;

	/** Navigate backwards. */
	virtual void GoBack() = 0;

	/**
	 * Returns true if the browser can navigate forwards.
	 */
	virtual bool CanGoForward() const = 0;

	/** Navigate forwards. */
	virtual void GoForward() = 0;

	/**
	 * Returns true if the browser is currently loading.
	 */
	virtual bool IsLoading() const = 0;

	/** Reload the current page. */
	virtual void Reload() = 0;

	/** Stop loading the page. */
	virtual void StopLoad() = 0;

	/** Execute Javascript on the page. */
	virtual void ExecuteJavascript(const FString& Script) = 0;

	/**
	 * Close this window so that it can no longer be used.
	 *
	 * @param bForce Designates whether the web browser close should be forced.
	 * @param bBlockTillClosed Don't return until this browser object is fully closed.
	 */
	virtual void CloseBrowser(bool bForce, bool bBlockTillClosed = false) = 0;

	/** 
	 * Expose a UObject instance to the browser runtime.
	 * Properties and Functions will be accessible from JavaScript side.
	 * As all communication with the rendering procesis asynchronous, return values (both for properties and function results) are wrapped into JS Future objects.
	 *
	 * @param Name The name of the object. The object will show up as window.ue.{Name} on the javascript side. If there is an existing object of the same name, this object will replace it. If bIsPermanent is false and there is an existing permanent binding, the permanent binding will be restored when the temporary one is removed.
	 * @param Object The object instance.
	 * @param bIsPermanent If true, the object will be visible to all pages loaded through this browser widget, otherwise, it will be deleted when navigating away from the current page. Non-permanent bindings should be registered from inside an OnLoadStarted event handler in order to be available before JS code starts loading.
	 */
	virtual void BindUObject(const FString& Name, UObject* Object, bool bIsPermanent = true) = 0;

	/**
	 * Remove an existing script binding registered by BindUObject.
	 *
	 * @param Name The name of the object to remove.
	 * @param Object The object will only be removed if it is the same object as the one passed in.
	 * @param bIsPermanent Must match the bIsPermanent argument passed to BindUObject.
	 */
	virtual void UnbindUObject(const FString& Name, UObject* Object, bool bIsPermanent = true) = 0;

	virtual void BindInputMethodSystem(ITextInputMethodSystem* TextInputMethodSystem) {}

	virtual void UnbindInputMethodSystem() {}

	/**
	 * Get current load error.
	 *
	 * @return an error code if the last page load resulted in an error, otherwise 0.
	*/
	virtual int GetLoadError() = 0;
	
	/**
	 * Disable or enable web view.
	 *
	 * @param bValue Setting this to true will prevent any updates from the background web browser.
	 */
	virtual void SetIsDisabled(bool bValue) = 0;

	/**
	* Get parent SWindow for this window
	*/
	virtual TSharedPtr<class SWindow> GetParentWindow() const = 0;

	/**
	* Set parent SWindow for this window
	*/
	virtual void SetParentWindow(TSharedPtr<class SWindow> Window) = 0;

	/**
	* Shows floating button in the browser if supported by platform
	*/
	virtual void ShowFloatingCloseButton(bool bShow, bool bDraggable) = 0;

	virtual void CheckTickActivity() {};

public:

	/** A delegate that is invoked when the loading state of a document changed. */
	DECLARE_EVENT_OneParam(IWebBrowserWindow, FOnDocumentStateChanged, EWebBrowserDocumentState /*NewState*/);
	virtual FOnDocumentStateChanged& OnDocumentStateChanged() = 0;

	/** A delegate to allow callbacks when a browser title changes. */
	DECLARE_EVENT_OneParam(IWebBrowserWindow, FOnTitleChanged, FString /*NewTitle*/);
	virtual FOnTitleChanged& OnTitleChanged() = 0;

	/** A delegate to allow callbacks when a frame url changes. */
	DECLARE_EVENT_OneParam(IWebBrowserWindow, FOnUrlChanged, FString /*NewUrl*/);
	virtual FOnUrlChanged& OnUrlChanged() = 0;

	/** A delegate to allow callbacks when a frame url changes. */
	DECLARE_EVENT_OneParam(IWebBrowserWindow, FOnToolTip, FString /*ToolTipText*/);
	virtual FOnToolTip& OnToolTip() = 0;

	/** A delegate that is invoked when the off-screen window has been repainted and requires an update. */
	DECLARE_EVENT(IWebBrowserWindow, FOnNeedsRedraw)
	virtual FOnNeedsRedraw& OnNeedsRedraw() = 0;
	
	/** A delegate that is invoked prior to browser navigation. */
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnBeforeBrowse, const FString& /*Url*/, const FWebNavigationRequest& /*Request*/)
	virtual FOnBeforeBrowse& OnBeforeBrowse() = 0;
	
	/** A delegate that is invoked to allow user code to override the contents of a Url. */
	DECLARE_DELEGATE_RetVal_ThreeParams(bool, FOnLoadUrl, const FString& /*Method*/, const FString& /*Url*/, FString& /*OutBody*/)
	virtual FOnLoadUrl& OnLoadUrl() = 0;

	/** A delegate that is invoked when a popup window is attempting to open. */
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnBeforePopupDelegate, FString, FString);
	virtual FOnBeforePopupDelegate& OnBeforePopup() = 0;

	/** A delegate that is invoked before the browser loads a resource. Its primary purpose is to inject headers into the request. */
	typedef TMap<FString, FString> FRequestHeaders;
	DECLARE_DELEGATE_FourParams(FOnBeforeResourceLoadDelegate, FString /*Url*/, FString /*ResourceType*/, FRequestHeaders& /*AdditionalHeaders*/, const bool /*AllowUserCredentials*/);
	virtual FOnBeforeResourceLoadDelegate& OnBeforeResourceLoad() = 0;

	/** A delegate that is invoked on completion of browser resource loads. Its primary purpose is to allow response to failures. */
	DECLARE_DELEGATE_FourParams(FOnResourceLoadCompleteDelegate, FString /*Url*/, FString /*ResourceType*/, FString /*RequestStatus*/, int64 /*ContentLength*/);
	virtual FOnResourceLoadCompleteDelegate& OnResourceLoadComplete() = 0;

	/** A delegate that is invoked for each console message */
	DECLARE_DELEGATE_FourParams(FOnConsoleMessageDelegate, const FString& /*Message*/, const FString& /*Source*/, int32 /*Line*/, EWebBrowserConsoleLogSeverity /*severity*/);
	virtual FOnConsoleMessageDelegate& OnConsoleMessage() = 0;

	/** A delegate that is invoked when an existing browser requests creation of a new browser window. */
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnCreateWindow, const TWeakPtr<IWebBrowserWindow>& /*NewBrowserWindow*/, const TWeakPtr<IWebBrowserPopupFeatures>& /* PopupFeatures*/)
	virtual FOnCreateWindow& OnCreateWindow() = 0;

	/** A delegate that is invoked when closing created popup windows. */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnCloseWindow, const TWeakPtr<IWebBrowserWindow>& /*BrowserWindow*/)
	virtual FOnCloseWindow& OnCloseWindow() = 0;

	/** A delegate that is invoked when close is requested from the floating close button. */
	DECLARE_DELEGATE(FOnFloatingCloseButtonPressed)
	virtual FOnFloatingCloseButtonPressed& OnFloatingCloseButtonPressed() = 0;

	/** A delegate that is invoked when the browser needs to show a popup menu. */
	DECLARE_EVENT_OneParam(IWebBrowserWindow, FOnShowPopup, const FIntRect& /*PopupSize*/)
	virtual FOnShowPopup& OnShowPopup() = 0;

	/** A delegate that is invoked when the browser no longer wants to show the popup menu. */
	DECLARE_EVENT(IWebBrowserWindow, FOnDismissPopup)
	virtual FOnDismissPopup& OnDismissPopup() = 0;

	/** A delegate that is invoked when the browser needs to show a dialog. */
	DECLARE_DELEGATE_RetVal_OneParam(EWebBrowserDialogEventResponse, FOnShowDialog, const TWeakPtr<IWebBrowserDialog>& /*DialogParams*/)
	virtual FOnShowDialog& OnShowDialog() = 0;

	/** A delegate that is invoked when the browser needs to dismiss and reset all dialogs. */
	DECLARE_DELEGATE(FOnDismissAllDialogs)
	virtual FOnDismissAllDialogs& OnDismissAllDialogs() = 0;

	/** Should return true if this dialog wants to suppress the context menu */
	DECLARE_DELEGATE_RetVal(bool, FOnSuppressContextMenu);
	virtual FOnSuppressContextMenu& OnSuppressContextMenu() = 0;

	/** A delegate that is invoked for each key down event not handled by the browser, return true if event is handled to prevent it from bubbling up. */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnUnhandledKeyDown, const FKeyEvent& /*KeyEvent*/);
	virtual FOnUnhandledKeyDown& OnUnhandledKeyDown() = 0;

	/** A delegate that is invoked for each up down event not handled by the browser, return true if event is handled to prevent it from bubbling up. */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnUnhandledKeyUp, const FKeyEvent& /*KeyEvent*/);
	virtual FOnUnhandledKeyUp& OnUnhandledKeyUp() = 0;

	/** A delegate that is invoked for each key char event not handled by the browser, return true if event is handled to prevent it from bubbling up. */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnUnhandledKeyChar, const FCharacterEvent& /*CharacterEvent*/);
	virtual FOnUnhandledKeyChar& OnUnhandledKeyChar() = 0;
	/** A delegate that is invoked when drag is detected in an area specified as a drag region on the web page. */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnDragWindow, const FPointerEvent& /*MouseEvent*/)
	virtual FOnDragWindow& OnDragWindow() = 0;

	/** A delegate that is invoked to check the visibility of the native browser */
	DECLARE_DELEGATE_RetVal(bool, FOnCheckVisibility);
	virtual FOnCheckVisibility& OnCheckVisibility()
	{
		return OnCheckVisibilityDelegate;
	}

	virtual bool CheckVisibility()
	{
		return !OnCheckVisibilityDelegate.IsBound() || OnCheckVisibilityDelegate.Execute();
	}

protected:

	/** Virtual Destructor. */
	virtual ~IWebBrowserWindow() = default;

private:
	/** Delegate for veritying the window's visibility  */
	FOnCheckVisibility OnCheckVisibilityDelegate;

};
