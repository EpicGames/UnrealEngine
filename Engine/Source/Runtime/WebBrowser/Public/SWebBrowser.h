// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "SWebBrowserView.h"

class IWebBrowserAdapter;
class IWebBrowserDialog;
class IWebBrowserWindow;
class SEditableTextBox;
struct FWebNavigationRequest;
enum class EWebBrowserDialogEventResponse;

DECLARE_DELEGATE_RetVal(bool, FOnSuppressContextMenu);

class SWebBrowser
	: public SCompoundWidget
{
public:
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnBeforeBrowse, const FString&, const FWebNavigationRequest& /*Request*/)
	DECLARE_DELEGATE_RetVal_ThreeParams(bool, FOnLoadUrl, const FString& /*Method*/, const FString& /*Url*/, FString& /* Response */)
	DECLARE_DELEGATE_RetVal_OneParam(EWebBrowserDialogEventResponse, FOnShowDialog, const TWeakPtr<IWebBrowserDialog>&);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnDragWindow, const FPointerEvent& /* MouseEvent */);

	SLATE_BEGIN_ARGS(SWebBrowser)
		: _InitialURL(TEXT("https://www.google.com"))
		, _ShowControls(true)
		, _ShowAddressBar(false)
		, _ShowErrorMessage(true)
		, _SupportsTransparency(false)
		, _SupportsThumbMouseButtonNavigation(true)
		, _ShowInitialThrobber(true)
		, _BackgroundColor(255,255,255,255)
		, _BrowserFrameRate(24)
		, _PopupMenuMethod(TOptional<EPopupMethod>())
		, _UseSmoothResizing(true)
		, _ViewportSize(FVector2D::ZeroVector)
	{ }

		/** A reference to the parent window. */
		SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)

		/** URL that the browser will initially navigate to. The URL should include the protocol, eg http:// */
		SLATE_ARGUMENT(FString, InitialURL)

		/** Optional string to load contents as a web page. */
		SLATE_ARGUMENT(TOptional<FString>, ContentsToLoad)

		/** Custom Application part of the user agent for HTTP requests. If not set, WebBrowserInitSettings.ProductVersion will be used. */
		SLATE_ARGUMENT(TOptional<FString>, UserAgentApplication)

		/** Whether to show standard controls like Back, Forward, Reload etc. */
		SLATE_ARGUMENT(bool, ShowControls)

		/** Whether to show an address bar. */
		SLATE_ARGUMENT(bool, ShowAddressBar)

		/** Whether to show an error message in case of loading errors. */
		SLATE_ARGUMENT(bool, ShowErrorMessage)

		/** Should this browser window support transparency. */
		SLATE_ARGUMENT(bool, SupportsTransparency)

		/** Whether to allow forward and back navigation via the mouse thumb buttons. */
		SLATE_ARGUMENT(bool, SupportsThumbMouseButtonNavigation)

		/** Whether to show a throbber overlay during browser initialization. */
		SLATE_ARGUMENT(bool, ShowInitialThrobber)

		/** Opaque background color used before a document is loaded and when no document color is specified. */
		SLATE_ARGUMENT(FColor, BackgroundColor)

		/** The frames per second rate that the browser will attempt to use. */
		SLATE_ARGUMENT(int , BrowserFrameRate)

		/** Override the popup menu method used for popup menus. If not set, parent widgets will be queried instead. */
		SLATE_ARGUMENT(TOptional<EPopupMethod>, PopupMenuMethod)

		/** Whether we should use a workaround for CEF bugs. */
		SLATE_ARGUMENT(bool, UseSmoothResizing);
		
		/** Desired size of the web browser viewport. */
		SLATE_ATTRIBUTE(FVector2D, ViewportSize);

		/** Called when document loading completed. */
		SLATE_EVENT(FSimpleDelegate, OnLoadCompleted)

		/** Called when document loading failed. */
		SLATE_EVENT(FSimpleDelegate, OnLoadError)

		/** Called when document loading started. */
		SLATE_EVENT(FSimpleDelegate, OnLoadStarted)

		/** Called when document title changed. */
		SLATE_EVENT(FOnTextChanged, OnTitleChanged)

		/** Called when the Url changes. */
		SLATE_EVENT(FOnTextChanged, OnUrlChanged)

		/** Called before a popup window happens */
		SLATE_EVENT(FOnBeforePopupDelegate, OnBeforePopup)

		/** Called when the browser requests the creation of a new window */
		SLATE_EVENT(FOnCreateWindowDelegate, OnCreateWindow)

		/** Called when a browser window close event is detected */
		SLATE_EVENT(FOnCloseWindowDelegate, OnCloseWindow)

		/** Called before browser navigation. */
		SLATE_EVENT(FOnBeforeBrowse, OnBeforeNavigation)

		/** Called to allow bypassing page content on load. */
		SLATE_EVENT(FOnLoadUrl, OnLoadUrl)

		/** Called when the browser needs to show a dialog to the user. */
		SLATE_EVENT(FOnShowDialog, OnShowDialog)

		/** Called to dismiss any dialogs shown via OnShowDialog. */
		SLATE_EVENT(FSimpleDelegate, OnDismissAllDialogs)

		SLATE_EVENT(FOnSuppressContextMenu, OnSuppressContextMenu);

		/** Called when drag is detected in a web page area tagged as a drag region. */
		SLATE_EVENT(FOnDragWindow, OnDragWindow);

		/** Called for each console message */
		SLATE_EVENT(FOnConsoleMessageDelegate, OnConsoleMessage);

	SLATE_END_ARGS()


	/** Default constructor. */
	WEBBROWSER_API SWebBrowser();

	WEBBROWSER_API ~SWebBrowser();

	virtual bool SupportsKeyboardFocus() const override {return true;}
	virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override;

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	
	/**
	 * Construct the widget.
	 *
	 * @param InArgs  Declaration from which to construct the widget.
	 */
	WEBBROWSER_API void Construct(const FArguments& InArgs, const TSharedPtr<IWebBrowserWindow>& InWebBrowserWindow = nullptr);

	/**
	 * Load the specified URL.
	 *
	 * @param NewURL New URL to load.
	 */
	WEBBROWSER_API void LoadURL(FString NewURL);

	/**
	* Load a string as data to create a web page.
	*
	* @param Contents String to load.
	* @param DummyURL Dummy URL for the page.
	*/
	WEBBROWSER_API void LoadString(FString Contents, FString DummyURL);

	/** Reload the current page. */
	WEBBROWSER_API void Reload();

	/** Stop loading the page. */
	WEBBROWSER_API void StopLoad();

	/** Get the current title of the web page. */
	WEBBROWSER_API FText GetTitleText() const;

	/**
	 * Gets the currently loaded URL.
	 *
	 * @return The URL, or empty string if no document is loaded.
	 */
	WEBBROWSER_API FString GetUrl() const;

	/**
	 * Gets the URL that appears in the address bar, this may not be the URL that is currently loaded in the frame.
	 *
	 * @return The address bar URL.
	 */
	WEBBROWSER_API FText GetAddressBarUrlText() const;

	/** Whether the document finished loading. */
	WEBBROWSER_API bool IsLoaded() const;

	/** Whether the document is currently being loaded. */
	WEBBROWSER_API bool IsLoading() const;

	/** Execute javascript on the current window */
	WEBBROWSER_API void ExecuteJavascript(const FString& ScriptText);

	/**
	 * Gets the source of the main frame as raw HTML.
	 *
	 * This method has to be called asynchronously by passing a callback function, which will be called at a later point when the
	 * result is ready.
	 * @param	Callback	A callable that takes a single string reference for handling the result.
	 */
	WEBBROWSER_API void GetSource(TFunction<void (const FString&)> Callback) const;

	/**
	 * Expose a UObject instance to the browser runtime.
	 * Properties and Functions will be accessible from JavaScript side.
	 * As all communication with the rendering procesis asynchronous, return values (both for properties and function results) are wrapped into JS Future objects.
	 *
	 * @param Name The name of the object. The object will show up as window.ue.{Name} on the javascript side. Note: All object names, function names, and property names will be converted to lower case when bound on the javascript side.  If there is an existing object of the same name, this object will replace it. If bIsPermanent is false and there is an existing permanent binding, the permanent binding will be restored when the temporary one is removed.
	 * @param Object The object instance.
	 * @param bIsPermanent If true, the object will be visible to all pages loaded through this browser widget, otherwise, it will be deleted when navigating away from the current page. Non-permanent bindings should be registered from inside an OnLoadStarted event handler in order to be available before JS code starts loading.
	 */
	WEBBROWSER_API void BindUObject(const FString& Name, UObject* Object, bool bIsPermanent = true);

	/**
	 * Remove an existing script binding registered by BindUObject.
	 *
	 * @param Name The name of the object to remove.
	 * @param Object The object will only be removed if it is the same object as the one passed in.
	 * @param bIsPermanent Must match the bIsPermanent argument passed to BindUObject.
	 */
	WEBBROWSER_API void UnbindUObject(const FString& Name, UObject* Object, bool bIsPermanent = true);

	WEBBROWSER_API void BindAdapter(const TSharedRef<IWebBrowserAdapter>& Adapter);

	WEBBROWSER_API void UnbindAdapter(const TSharedRef<IWebBrowserAdapter>& Adapter);

	WEBBROWSER_API void BindInputMethodSystem(ITextInputMethodSystem* TextInputMethodSystem);

	WEBBROWSER_API void UnbindInputMethodSystem();

	/** Returns true if the browser can navigate backwards. */
	WEBBROWSER_API bool CanGoBack() const;

	/** Navigate backwards. */
	WEBBROWSER_API void GoBack();

	/** Returns true if the browser can navigate forwards. */
	WEBBROWSER_API bool CanGoForward() const;

	/** Navigate forwards. */
	WEBBROWSER_API void GoForward();

	/** Set parent SWindow for this browser. */
	WEBBROWSER_API void SetParentWindow(TSharedPtr<SWindow> Window);

	/** Show the throbber and hide the web browser content. */
	WEBBROWSER_API void ShowThrobber();

	/** Hide the throbber and show the web browser content. */
	WEBBROWSER_API void HideThrobber();

private:

	/** Navigate backwards. */
	WEBBROWSER_API FReply OnBackClicked();

	/** Navigate forwards. */
	WEBBROWSER_API FReply OnForwardClicked();

	/** Get text for reload button depending on status */
	WEBBROWSER_API FText GetReloadButtonText() const;

	/** Reload or stop loading */
	WEBBROWSER_API FReply OnReloadClicked();

	/** Invoked whenever text is committed in the address bar. */
	WEBBROWSER_API void OnUrlTextCommitted( const FText& NewText, ETextCommit::Type CommitType );

	/** Get whether the page viewport should be visible */
	WEBBROWSER_API EVisibility GetViewportVisibility() const;

	/** Get whether loading throbber should be visible */
	WEBBROWSER_API EVisibility GetLoadingThrobberVisibility() const;

private:

	/** The actual web browser view */
	TSharedPtr<SWebBrowserView> BrowserView;

	/** Editable text widget used for an address bar */
	TSharedPtr< SEditableTextBox > InputText;

	/** A delegate that is invoked when document loading completed. */
	FSimpleDelegate OnLoadCompleted;

	/** A delegate that is invoked when document loading failed. */
	FSimpleDelegate OnLoadError;

	/** A delegate that is invoked when document loading started. */
	FSimpleDelegate OnLoadStarted;

	/** A delegate that is invoked when document title changed. */
	FOnTextChanged OnTitleChanged;

	/** A delegate that is invoked when document address changed. */
	FOnTextChanged OnUrlChanged;

	/** A delegate that is invoked when the browser attempts to pop up a new window */
	FOnBeforePopupDelegate OnBeforePopup;

	/** A delegate that is invoked when the browser writes console output */
	FOnConsoleMessageDelegate OnConsoleMessage;

	/** A delegate that is invoked when the browser requests a UI window for another browser it spawned */
	FOnCreateWindowDelegate OnCreateWindow;

	/** A delegate that is invoked when a window close event is detected */
	FOnCloseWindowDelegate OnCloseWindow;

	/** A delegate that is invoked prior to browser navigation */
	FOnBeforeBrowse OnBeforeNavigation;

	/** A delegate that is invoked when loading a resource, allowing the application to provide contents directly */
	FOnLoadUrl OnLoadUrl;

	/** A delegate that is invoked when when the browser needs to present a dialog to the user */
	FOnShowDialog OnShowDialog;

	/** A delegate that is invoked when when the browser needs to dismiss all dialogs */
	FSimpleDelegate OnDismissAllDialogs;

	/** The initial throbber setting */
	bool bShowInitialThrobber;

	/** Whether to force the throbber to appear */
	bool bForceThrobber = false;

	/** The WebBrowserWindow itself. */
	TSharedPtr<IWebBrowserWindow> WebBrowserWindow;

	/** Pre-shutdown delegate handle, to unbind the Input Method System. */
	FDelegateHandle InputMethodSystemSlatePreShutdownDelegateHandle;
};
