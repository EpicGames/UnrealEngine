// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/DragAndDrop.h"
#include "Input/PopupMethodReply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/Application/IMenu.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/SViewport.h"
#include "IWebBrowserSingleton.h"

class FWebBrowserViewport;
class IWebBrowserAdapter;
class IWebBrowserDialog;
class IWebBrowserPopupFeatures;
class IWebBrowserWindow;
struct FWebNavigationRequest;
enum class EWebBrowserDialogEventResponse;
enum class EWebBrowserDocumentState;
enum class EWebBrowserConsoleLogSeverity;

DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnBeforePopupDelegate, FString, FString);
DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnCreateWindowDelegate, const TWeakPtr<IWebBrowserWindow>&, const TWeakPtr<IWebBrowserPopupFeatures>&);
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnCloseWindowDelegate, const TWeakPtr<IWebBrowserWindow>&);
DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<IToolTip>, FOnCreateToolTip, const FText&);
DECLARE_DELEGATE_FourParams(FOnConsoleMessageDelegate, const FString& /*Message*/, const FString& /*Source*/, int32 /*Line*/, EWebBrowserConsoleLogSeverity /*Severity*/);
DECLARE_DELEGATE(FOnFloatingCloseButtonPressedDelegate);

#if WITH_CEF3 || PLATFORM_APPLE
typedef SViewport SWebBrowserWidget;
#else
typedef SWidget SWebBrowserWidget;
#endif

class SWebBrowserView
	: public SCompoundWidget
{
public:
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnBeforeBrowse, const FString& /*Url*/, const FWebNavigationRequest& /*Request*/)
	DECLARE_DELEGATE_RetVal_ThreeParams(bool, FOnLoadUrl, const FString& /*Method*/, const FString& /*Url*/, FString& /* Response */)
	DECLARE_DELEGATE_RetVal_OneParam(EWebBrowserDialogEventResponse, FOnShowDialog, const TWeakPtr<IWebBrowserDialog>&);
	DECLARE_DELEGATE_RetVal(bool, FOnSuppressContextMenu);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnDragWindow, const FPointerEvent& /* MouseEvent */);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnUnhandledKeyDown, const FKeyEvent& /*KeyEvent*/);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnUnhandledKeyUp, const FKeyEvent& /*KeyEvent*/);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnUnhandledKeyChar, const FCharacterEvent& /*CharacterEvent*/);


	SLATE_BEGIN_ARGS(SWebBrowserView)
		: _InitialURL(TEXT("https://www.google.com"))
		, _ShowErrorMessage(true)
		, _SupportsTransparency(false)
		, _InterceptLoadRequests(true)
		, _SupportsThumbMouseButtonNavigation(true)
		, _BackgroundColor(255,255,255,255)
		, _BrowserFrameRate(24)
		, _PopupMenuMethod(TOptional<EPopupMethod>())
		, _ContextSettings()
		, _AltRetryDomains(TArray<FString>())
		, _EnablePaymentRequest(false)
		, _ViewportSize(FVector2D::ZeroVector)
	{ }

		/** A reference to the parent window. */
		SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)

		/** URL that the browser will initially navigate to. */
		SLATE_ARGUMENT(FString, InitialURL)

		/** Optional string to load contents as a web page. */
		SLATE_ARGUMENT(TOptional<FString>, ContentsToLoad)

		/** Custom Application part of the user agent for HTTP requests. If not set, WebBrowserInitSettings.ProductVersion will be used. */
		SLATE_ARGUMENT(TOptional<FString>, UserAgentApplication)

		/** Whether to show an error message in case of loading errors. */
		SLATE_ARGUMENT(bool, ShowErrorMessage)

		/** Should this browser window support transparency. */
		SLATE_ARGUMENT(bool, SupportsTransparency)

		/** Should this browser window intercept resource loading requests. If false the BrowserContext will instead. Defaults to True. */
		SLATE_ARGUMENT(bool, InterceptLoadRequests)

		/** Whether to allow forward and back navigation via the mouse thumb buttons. */
		SLATE_ARGUMENT(bool, SupportsThumbMouseButtonNavigation)

		/** Opaque background color used before a document is loaded and when no document color is specified. */
		SLATE_ARGUMENT(FColor, BackgroundColor)

		/** The frames per second rate that the browser will attempt to use. */
		SLATE_ARGUMENT(int, BrowserFrameRate)

		/** Override the popup menu method used for popup menus. If not set, parent widgets will be queried instead. */
		SLATE_ARGUMENT(TOptional<EPopupMethod>, PopupMenuMethod)

		/** Override the default global context settings for this specific window. If not set, the global default will be used. */
		SLATE_ARGUMENT(TOptional<FBrowserContextSettings>, ContextSettings)

		/** Domains to retry if original domain cannot connect. */
		SLATE_ARGUMENT(TArray<FString>, AltRetryDomains)

		/** Opt in the W3C PaymentRequest API for this browser. */
		SLATE_ARGUMENT(bool, EnablePaymentRequest)

		/** Desired size of the web browser viewport. */
		SLATE_ATTRIBUTE(FVector2D, ViewportSize);

		/** Called when document loading completed. */
		SLATE_EVENT(FSimpleDelegate, OnLoadCompleted)

		/** Called when document loading failed. */
		SLATE_EVENT(FSimpleDelegate, OnLoadError)

		/** Called when document loading started. */
		SLATE_EVENT(FSimpleDelegate, OnLoadStarted)

		/** Called when document title changed. */
		SLATE_EVENT(FOnTextChanged, OnTitleChanged)

		/** Called when the Url changes. */
		SLATE_EVENT(FOnTextChanged, OnUrlChanged)

		/** Called before a popup window happens */
		SLATE_EVENT(FOnBeforePopupDelegate, OnBeforePopup)

		/** Called when the browser requests the creation of a new window */
		SLATE_EVENT(FOnCreateWindowDelegate, OnCreateWindow)

		/** Called when a browser window close event is detected */
		SLATE_EVENT(FOnCloseWindowDelegate, OnCloseWindow)

		/** Called when a browser window requests to close through the floating close button*/
		SLATE_EVENT(FOnFloatingCloseButtonPressedDelegate, OnFloatingCloseButtonPressed)

		/** Called before browser navigation. */
		SLATE_EVENT(FOnBeforeBrowse, OnBeforeNavigation)

		/** Called to allow bypassing page content on load. */
		SLATE_EVENT(FOnLoadUrl, OnLoadUrl)

		/** Called when the browser needs to show a dialog to the user. */
		SLATE_EVENT(FOnShowDialog, OnShowDialog)

		/** Called to dismiss any dialogs shown via OnShowDialog. */
		SLATE_EVENT(FSimpleDelegate, OnDismissAllDialogs)

		/** Called to allow suppression of the browser context menu. */
		SLATE_EVENT(FOnSuppressContextMenu, OnSuppressContextMenu);

		/** Called to allow overriding of ToolTip widget construction. */
		SLATE_EVENT(FOnCreateToolTip, OnCreateToolTip)

		/** Called when drag is detected in a web page area tagged as a drag region. */
		SLATE_EVENT(FOnDragWindow, OnDragWindow)

		/** Called to allow the handling of any key down events not handled by the browser. */
		SLATE_EVENT(FOnUnhandledKeyDown, OnUnhandledKeyDown)

		/** Called to allow the handling of any key up events not handled by the browser. */
		SLATE_EVENT(FOnUnhandledKeyUp, OnUnhandledKeyUp)

		/** Called to allow the handling of any key char events not handled by the browser. */
		SLATE_EVENT(FOnUnhandledKeyChar, OnUnhandledKeyChar)

		/** Called for each console message */
		SLATE_EVENT(FOnConsoleMessageDelegate, OnConsoleMessage)

	SLATE_END_ARGS()


	/** Default constructor. */
	WEBBROWSER_API SWebBrowserView();

	WEBBROWSER_API ~SWebBrowserView();

	virtual bool SupportsKeyboardFocus() const override {return true;}

	virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override;

	WEBBROWSER_API virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	WEBBROWSER_API virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	WEBBROWSER_API virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	WEBBROWSER_API virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	/**
	 * Construct the widget.
	 *
	 * @param InArgs  Declaration from which to construct the widget.
	 */
	WEBBROWSER_API void Construct(const FArguments& InArgs, const TSharedPtr<IWebBrowserWindow>& InWebBrowserWindow = nullptr);

	WEBBROWSER_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	/**
	 * Load the specified URL.
	 *
	 * @param NewURL New URL to load.
	 */
	WEBBROWSER_API void LoadURL(FString NewURL);

	/**
	* Load a string as data to create a web page.
	*
	* @param Contents String to load.
	* @param DummyURL Dummy URL for the page.
	*/
	WEBBROWSER_API void LoadString(FString Contents, FString DummyURL);

	/** Reload the current page. */
	WEBBROWSER_API void Reload();

	/** Stop loading the page. */
	WEBBROWSER_API void StopLoad();

	/** Get the current title of the web page. */
	WEBBROWSER_API FText GetTitleText() const;

	/**
	 * Gets the currently loaded URL.
	 *
	 * @return The URL, or empty string if no document is loaded.
	 */
	WEBBROWSER_API FString GetUrl() const;

	/**
	 * Gets the URL that appears in the address bar, this may not be the URL that is currently loaded in the frame.
	 *
	 * @return The address bar URL.
	 */
	WEBBROWSER_API FText GetAddressBarUrlText() const;

	/** Whether the document finished loading. */
	WEBBROWSER_API bool IsLoaded() const;

	/** Whether the document is currently being loaded. */
	WEBBROWSER_API bool IsLoading() const;

	/** Whether the browser widget is done initializing. */
	WEBBROWSER_API bool IsInitialized() const;

	/** Execute javascript on the current window */
	WEBBROWSER_API void ExecuteJavascript(const FString& ScriptText);

	/**
	 * Gets the source of the main frame as raw HTML.
	 *
	 * This method has to be called asynchronously by passing a callback function, which will be called at a later point when the
	 * result is ready.
	 * @param	Callback	A callable that takes a single string reference for handling the result.
	 */
	WEBBROWSER_API void GetSource(TFunction<void (const FString&)> Callback) const ;

	/**
	 * Expose a UObject instance to the browser runtime.
	 * Properties and Functions will be accessible from JavaScript side.
	 * As all communication with the rendering procesis asynchronous, return values (both for properties and function results) are wrapped into JS Future objects.
	 *
	 * @param Name The name of the object. The object will show up as window.ue.{Name} on the javascript side. If there is an existing object of the same name, this object will replace it. If bIsPermanent is false and there is an existing permanent binding, the permanent binding will be restored when the temporary one is removed.
	 * @param Object The object instance.
	 * @param bIsPermanent If true, the object will be visible to all pages loaded through this browser widget, otherwise, it will be deleted when navigating away from the current page. Non-permanent bindings should be registered from inside an OnLoadStarted event handler in order to be available before JS code starts loading.
	 */
	WEBBROWSER_API void BindUObject(const FString& Name, UObject* Object, bool bIsPermanent = true);

	/**
	 * Remove an existing script binding registered by BindUObject.
	 *
	 * @param Name The name of the object to remove.
	 * @param Object The object will only be removed if it is the same object as the one passed in.
	 * @param bIsPermanent Must match the bIsPermanent argument passed to BindUObject.
	 */
	WEBBROWSER_API void UnbindUObject(const FString& Name, UObject* Object, bool bIsPermanent = true);

	WEBBROWSER_API void BindAdapter(const TSharedRef<IWebBrowserAdapter>& Adapter);

	WEBBROWSER_API void UnbindAdapter(const TSharedRef<IWebBrowserAdapter>& Adapter);

	WEBBROWSER_API void BindInputMethodSystem(ITextInputMethodSystem* TextInputMethodSystem);

	WEBBROWSER_API void UnbindInputMethodSystem();

	/** Returns true if the browser can navigate backwards. */
	WEBBROWSER_API bool CanGoBack() const;

	/** Navigate backwards. */
	WEBBROWSER_API void GoBack();

	/** Returns true if the browser can navigate forwards. */
	WEBBROWSER_API bool CanGoForward() const;

	/** Navigate forwards. */
	WEBBROWSER_API void GoForward();

	/** Set parent SWindow for this browser. */
	WEBBROWSER_API void SetParentWindow(TSharedPtr<SWindow> Window);

	/** Update the underlying browser widget to match the KB focus in slate.
		This is used to work around a CEF bug that loses focus state on navigations*/
	WEBBROWSER_API void SetBrowserKeyboardFocus();

	/** Close the underlying browser object before we destruct this view.
	    This will block until that object is fully destroyed.
		Calling this is optional, CEF has object lifetime requirements that mean on shutdown you must destroy browsers before exit.*/
	WEBBROWSER_API void CloseBrowser();

	/** Shows floating close button.*/
	WEBBROWSER_API void ShowFloatingCloseButton(bool bShow, bool bDraggable);

	/** Gets the internal web browser window */
	WEBBROWSER_API TSharedPtr<IWebBrowserWindow> GetBrowserWindow() const;

private:

	WEBBROWSER_API void SetupParentWindowHandlers();

	/** Callback for document loading state changes. */
	WEBBROWSER_API void HandleBrowserWindowDocumentStateChanged(EWebBrowserDocumentState NewState);

	/** Callback to tell slate we want to update the contents of the web view based on changes inside the view. */
	WEBBROWSER_API void HandleBrowserWindowNeedsRedraw();

	/** Callback for document title changes. */
	WEBBROWSER_API void HandleTitleChanged(FString NewTitle);

	/** Callback for loaded url changes. */
	WEBBROWSER_API void HandleUrlChanged(FString NewUrl);

	/** Callback for showing browser tool tips. */
	WEBBROWSER_API void HandleToolTip(FString ToolTipText);

	/**
	 * A delegate that is executed prior to browser navigation.
	 *
	 * @return true if the navigation was handled an no further action should be taken by the browser, false if the browser should handle.
	 */
	WEBBROWSER_API bool HandleBeforeNavigation(const FString& Url, const FWebNavigationRequest& Request);

	WEBBROWSER_API bool HandleLoadUrl(const FString& Method, const FString& Url, FString& OutResponse);

	/**
	 * A delegate that is executed when the browser requests window creation.
	 *
	 * @return true if if the window request was handled, false if the browser requesting the new window should be closed.
	 */
	WEBBROWSER_API bool HandleCreateWindow(const TWeakPtr<IWebBrowserWindow>& NewBrowserWindow, const TWeakPtr<IWebBrowserPopupFeatures>& PopupFeatures);

	/**
	 * A delegate that is executed when closing the browser window.
	 *
	 * @return true if if the window close was handled, false otherwise.
	 */
	WEBBROWSER_API bool HandleCloseWindow(const TWeakPtr<IWebBrowserWindow>& BrowserWindow);

	/** Callback for handling requests from floating close button */
	WEBBROWSER_API void HandleFloatingCloseButtonPressed();

	/** Callback for showing dialogs to the user */
	WEBBROWSER_API EWebBrowserDialogEventResponse HandleShowDialog(const TWeakPtr<IWebBrowserDialog>& DialogParams);

	/** Callback for dismissing any dialogs previously shown  */
	WEBBROWSER_API void HandleDismissAllDialogs();

	/** Callback for popup window permission */
	WEBBROWSER_API bool HandleBeforePopup(FString URL, FString Target);

	/** Callback for showing a popup menu */
	WEBBROWSER_API void HandleShowPopup(const FIntRect& PopupSize);

	/** Callback for hiding the popup menu */
	WEBBROWSER_API void HandleDismissPopup();

	/** Callback from the popup menu notifiying it has been dismissed */
	WEBBROWSER_API void HandleMenuDismissed(TSharedRef<IMenu>);

	virtual FPopupMethodReply OnQueryPopupMethod() const override
	{
		return PopupMenuMethod.IsSet()
			? FPopupMethodReply::UseMethod(PopupMenuMethod.GetValue())
			: FPopupMethodReply::Unhandled();
	}

	WEBBROWSER_API void HandleWindowDeactivated();
	WEBBROWSER_API void HandleWindowActivated();
	WEBBROWSER_API bool UnhandledKeyDown(const FKeyEvent& KeyEvent);
	WEBBROWSER_API bool UnhandledKeyUp(const FKeyEvent& KeyEvent);
	WEBBROWSER_API bool UnhandledKeyChar(const FCharacterEvent& CharacterEvent);

	WEBBROWSER_API bool HandleDrag(const FPointerEvent& MouseEvent);
	WEBBROWSER_API void HandleConsoleMessage(const FString& Message, const FString& Source, int32 Line, EWebBrowserConsoleLogSeverity Serverity);

private:

	/** Interface for dealing with a web browser window. */
	TSharedPtr<IWebBrowserWindow> BrowserWindow;
	/** The slate window that contains this widget. This must be stored weak otherwise we create a circular reference. */
	mutable TWeakPtr<SWindow> SlateParentWindowPtr;
	/** Viewport interface for rendering the web page. */
	TSharedPtr<FWebBrowserViewport> BrowserViewport;
	/** Viewport interface for rendering popup menus. */
	TSharedPtr<FWebBrowserViewport>	MenuViewport;
	/** The implementation dependent widget that renders the browser contents. */
	TSharedPtr<SWebBrowserWidget> BrowserWidget;

	TArray<TSharedRef<IWebBrowserAdapter>> Adapters;

	/**
	 * An interface pointer to a menu object presenting a popup.
	 * Pointer is null when a popup is not visible.
	 */
	TWeakPtr<IMenu> PopupMenuPtr;

	/** Can be set to override the popup menu method used for popup menus. If not set, parent widgets will be queried instead. */
	TOptional<EPopupMethod> PopupMenuMethod;

	/** The url that appears in the address bar which can differ from the url of the loaded page */
	FText AddressBarUrl;

	/** A delegate that is invoked when document loading completed. */
	FSimpleDelegate OnLoadCompleted;

	/** A delegate that is invoked when document loading failed. */
	FSimpleDelegate OnLoadError;

	/** A delegate that is invoked when document loading started. */
	FSimpleDelegate OnLoadStarted;

	/** A delegate that is invoked when document title changed. */
	FOnTextChanged OnTitleChanged;

	/** A delegate that is invoked when document address changed. */
	FOnTextChanged OnUrlChanged;

	/** A delegate that is invoked when the browser attempts to pop up a new window */
	FOnBeforePopupDelegate OnBeforePopup;

	/** A delegate that is invoked when the browser requests a UI window for another browser it spawned */
	FOnCreateWindowDelegate OnCreateWindow;

	/** A delegate that is invoked when a window close event is detected */
	FOnCloseWindowDelegate OnCloseWindow;

	/** A delegate that is invoked when the floating close button is pressed */
	FOnFloatingCloseButtonPressedDelegate OnFloatingCloseButtonPressed;

	/** A delegate that is invoked prior to browser navigation */
	FOnBeforeBrowse OnBeforeNavigation;

	/** A delegate that is invoked when loading a resource, allowing the application to provide contents directly */
	FOnLoadUrl OnLoadUrl;

	/** A delegate that is invoked when when the browser needs to present a dialog to the user */
	FOnShowDialog OnShowDialog;

	/** A delegate that is invoked when when the browser needs to dismiss all dialogs */
	FSimpleDelegate OnDismissAllDialogs;

	FOnSuppressContextMenu OnSuppressContextMenu;

	/** A delegate that is invoked when when the browser wishes to create a tooltip */
	FOnCreateToolTip OnCreateToolTip;

	/** A delegate that is invoked when the browser detects drag event in within drag region */
	FOnDragWindow OnDragWindow;

	/** A delegate for handling key down events not handled by browser. */
	FOnUnhandledKeyDown OnUnhandledKeyDown;

	/** A delegate for handling key up events not handled by browser. */
	FOnUnhandledKeyUp OnUnhandledKeyUp;

	/** A delegate for handling key char events not handled by browser. */
	FOnUnhandledKeyChar OnUnhandledKeyChar;

	/** A delegate that is invoked for each console message */
	FOnConsoleMessageDelegate OnConsoleMessage;

protected:
	WEBBROWSER_API bool HandleSuppressContextMenu();

};
