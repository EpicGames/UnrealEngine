// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class IWebBrowserSingleton;

/**
 * WebBrowser initialization settings, can be used to override default init behaviors.
 */
struct FWebBrowserInitSettings
{
public:
	/**
	 * Default constructor. Initializes all members with default behavior values.
	 */
	WEBBROWSER_API FWebBrowserInitSettings();

	// The string which is appended to the browser's user-agent value.
	FString ProductVersion;
};

/**
 * WebBrowserModule interface
 */
class IWebBrowserModule : public IModuleInterface
{
public:
	/**
	 * Get or load the Web Browser Module
	 * 
	 * @return The loaded module
	 */
	static inline IWebBrowserModule& Get()
	{
		return FModuleManager::LoadModuleChecked< IWebBrowserModule >("WebBrowser");
	}
	
	/**
	 * Check whether the module has already been loaded
	 * 
	 * @return True if the module is loaded
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("WebBrowser");
	}

	/**
	 * Returns a string suitable for use as the application (or product) part of a browser's user agent string.
	 * Includes the formatted application/product name and version, followed by the engine name and version.
	 *
	 * @param ApplicationName the application/product name
	 * @param ApplicationVersion the application/product version, will default to 1.0 if empty
	 * @return The user agent application string
	 */
	WEBBROWSER_API static FString MakeUserAgentApplication(const FString& ApplicationName, const FString& ApplicationVersion = FString());

	/**
	 * Customize initialization settings. You must call this before the first GetSingleton call, in order to override init settings.
	 * 
	 * @param WebBrowserInitSettings The custom settings.
	 * @return true if the settings were used to initialize the singleton. False if the call was ignored due to singleton already existing.
	 */
	virtual bool CustomInitialize(const FWebBrowserInitSettings& WebBrowserInitSettings) = 0;

	/**
	 * Get the Web Browser Singleton
	 * 
	 * @return The Web Browser Singleton
	 */
	virtual IWebBrowserSingleton* GetSingleton() = 0;


	/**
	 * Check whether the web module loaded its requirements successfully
	 *
	 * @return True if the module load worked
	 */
	virtual bool IsWebModuleAvailable() const = 0;
};
