// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Rendering/RenderingCommon.h"

class FSlateShaderResource;
class IWebBrowserWindow;

/**
 * A Slate viewport to display a Web Browser Window
 */
class FWebBrowserViewport : public ISlateViewport
{
public:
	/**
	 * Default Constructor
	 * 
	 * @param InWebBrowserWindow The Web Browser Window this viewport will display
	 * @param InViewportWidget The Widget displaying this viewport (needed to capture mouse)
	 * @param InIsPopup Used to initialize a viewport for showing browser popup menus instead of the main window.
	 */
	FWebBrowserViewport(TSharedPtr<IWebBrowserWindow> InWebBrowserWindow, bool InIsPopup = false)
		: WebBrowserWindow(InWebBrowserWindow)
		, bIsPopup(InIsPopup)
	{ }

	/**
	 * Destructor.
	 */
	~FWebBrowserViewport( ) = default;

	// ISlateViewport interface
	WEBBROWSER_API virtual FIntPoint GetSize() const override;
	WEBBROWSER_API virtual FSlateShaderResource* GetViewportRenderTargetTexture() const override;
	WEBBROWSER_API virtual void Tick( const FGeometry& AllottedGeometry, double InCurrentTime, float DeltaTime ) override;
	WEBBROWSER_API virtual bool RequiresVsync() const override;
	WEBBROWSER_API virtual FCursorReply OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) override;
	WEBBROWSER_API virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	WEBBROWSER_API virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	WEBBROWSER_API virtual void OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	WEBBROWSER_API virtual void OnMouseLeave( const FPointerEvent& MouseEvent ) override;
	WEBBROWSER_API virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	WEBBROWSER_API virtual FReply OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	WEBBROWSER_API virtual FReply OnTouchGesture( const FGeometry& MyGeometry, const FPointerEvent& GestureEvent ) override;
	WEBBROWSER_API virtual FReply OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override;
	WEBBROWSER_API virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	WEBBROWSER_API virtual FReply OnKeyUp( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	WEBBROWSER_API virtual FReply OnKeyChar( const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent ) override;
	WEBBROWSER_API virtual FReply OnFocusReceived( const FFocusEvent& InFocusEvent ) override;
	WEBBROWSER_API virtual void OnFocusLost( const FFocusEvent& InFocusEvent ) override;

private:
	/** The web browser this viewport will display */
	TSharedPtr<IWebBrowserWindow>	WebBrowserWindow;
	/** Whether this viewport is showing the browser window or a popup menu widget */
	bool const						bIsPopup;
};
