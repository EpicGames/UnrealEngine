// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Misc/Guid.h"
#include "Templates/EnableIf.h"
#include "Templates/IsPointer.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"

#include "WebJSFunction.generated.h"

class FWebJSScripting;
class UObject;
class UStruct;

struct FWebJSParam
{

	struct IStructWrapper
	{
		virtual ~IStructWrapper() = default;
		virtual UStruct* GetTypeInfo() = 0;
		virtual const void* GetData() = 0;
		virtual IStructWrapper* Clone() = 0;
	};

	template <typename T> struct FStructWrapper
		: public IStructWrapper
	{
		T StructValue;
		FStructWrapper(const T& InValue)
			: StructValue(InValue)
		{}
		virtual ~FStructWrapper() = default;
		virtual UStruct* GetTypeInfo() override
		{
			return T::StaticStruct();
		}
		virtual const void* GetData() override
		{
			return &StructValue;
		}
		virtual IStructWrapper* Clone() override
		{
			return new FStructWrapper<T>(StructValue);
		}
	};

	FWebJSParam() : Tag(PTYPE_NULL) {}
	FWebJSParam(bool Value) : Tag(PTYPE_BOOL), BoolValue(Value) {}
	FWebJSParam(int8 Value) : Tag(PTYPE_INT), IntValue(Value) {}
	FWebJSParam(int16 Value) : Tag(PTYPE_INT), IntValue(Value) {}
	FWebJSParam(int32 Value) : Tag(PTYPE_INT), IntValue(Value) {}
	FWebJSParam(uint8 Value) : Tag(PTYPE_INT), IntValue(Value) {}
	FWebJSParam(uint16 Value) : Tag(PTYPE_INT), IntValue(Value) {}
	FWebJSParam(uint32 Value) : Tag(PTYPE_DOUBLE), DoubleValue(Value) {}
	FWebJSParam(int64 Value) : Tag(PTYPE_DOUBLE), DoubleValue(Value) {}
	FWebJSParam(uint64 Value) : Tag(PTYPE_DOUBLE), DoubleValue(Value) {}
	FWebJSParam(double Value) : Tag(PTYPE_DOUBLE), DoubleValue(Value) {}
	FWebJSParam(float Value) : Tag(PTYPE_DOUBLE), DoubleValue(Value) {}
	FWebJSParam(const FString& Value) : Tag(PTYPE_STRING), StringValue(new FString(Value)) {}
	FWebJSParam(const FText& Value) : Tag(PTYPE_STRING), StringValue(new FString(Value.ToString())) {}
	FWebJSParam(const FName& Value) : Tag(PTYPE_STRING), StringValue(new FString(Value.ToString())) {}
	FWebJSParam(const TCHAR* Value) : Tag(PTYPE_STRING), StringValue(new FString(Value)) {}
	FWebJSParam(UObject* Value) : Tag(PTYPE_OBJECT), ObjectValue(Value) {}
	template <typename T> FWebJSParam(const T& Value,
		typename TEnableIf<!TIsPointer<T>::Value, UStruct>::Type* InTypeInfo=T::StaticStruct())
		: Tag(PTYPE_STRUCT)
		, StructValue(new FStructWrapper<T>(Value))
	{}
	template <typename T> FWebJSParam(const TArray<T>& Value)
		: Tag(PTYPE_ARRAY)
	{
		ArrayValue = new TArray<FWebJSParam>();
		ArrayValue->Reserve(Value.Num());
		for(T Item : Value)
		{
			ArrayValue->Add(FWebJSParam(Item));
		}
	}
	template <typename T> FWebJSParam(const TMap<FString, T>& Value)
		: Tag(PTYPE_MAP)
	{
		MapValue = new TMap<FString, FWebJSParam>();
		MapValue->Reserve(Value.Num());
		for(const auto& Pair : Value)
		{
			MapValue->Add(Pair.Key, FWebJSParam(Pair.Value));
		}
	}
	template <typename K, typename T> FWebJSParam(const TMap<K, T>& Value)
		: Tag(PTYPE_MAP)
	{
		MapValue = new TMap<FString, FWebJSParam>();
		MapValue->Reserve(Value.Num());
		for(const auto& Pair : Value)
		{
			MapValue->Add(Pair.Key.ToString(), FWebJSParam(Pair.Value));
		}
	}
	WEBBROWSER_API FWebJSParam(const FWebJSParam& Other);
	WEBBROWSER_API FWebJSParam(FWebJSParam&& Other);
	WEBBROWSER_API ~FWebJSParam();

	enum { PTYPE_NULL, PTYPE_BOOL, PTYPE_INT, PTYPE_DOUBLE, PTYPE_STRING, PTYPE_OBJECT, PTYPE_STRUCT, PTYPE_ARRAY, PTYPE_MAP } Tag;
	union
	{
		bool BoolValue;
		double DoubleValue;
		int32 IntValue;
		UObject* ObjectValue;
		const FString* StringValue;
		IStructWrapper* StructValue;
		TArray<FWebJSParam>* ArrayValue;
		TMap<FString, FWebJSParam>* MapValue;
	};

};

class FWebJSScripting;

/** Base class for JS callback objects. */
USTRUCT()
struct FWebJSCallbackBase
{
	GENERATED_USTRUCT_BODY()
	FWebJSCallbackBase()
	{}

	bool IsValid() const
	{
		return ScriptingPtr.IsValid();
	}


protected:
	FWebJSCallbackBase(TSharedPtr<FWebJSScripting> InScripting, const FGuid& InCallbackId)
		: ScriptingPtr(InScripting)
		, CallbackId(InCallbackId)
	{}

	WEBBROWSER_API void Invoke(int32 ArgCount, FWebJSParam Arguments[], bool bIsError = false) const;

private:

	TWeakPtr<FWebJSScripting> ScriptingPtr;
	FGuid CallbackId;
};


/**
 * Representation of a remote JS function.
 * FWebJSFunction objects represent a JS function and allow calling them from native code.
 * FWebJSFunction objects can also be added to delegates and events using the Bind/AddLambda method.
 */
USTRUCT()
struct FWebJSFunction
	: public FWebJSCallbackBase
{
	GENERATED_USTRUCT_BODY()

	FWebJSFunction()
		: FWebJSCallbackBase()
	{}

	FWebJSFunction(TSharedPtr<FWebJSScripting> InScripting, const FGuid& InFunctionId)
		: FWebJSCallbackBase(InScripting, InFunctionId)
	{}

	template<typename ...ArgTypes> void operator()(ArgTypes... Args) const
	{
		FWebJSParam ArgArray[sizeof...(Args)] = {FWebJSParam(Args)...};
		Invoke(sizeof...(Args), ArgArray);
	}
};

/** 
 *  Representation of a remote JS async response object.
 *  UFUNCTIONs taking a FWebJSResponse will get it passed in automatically when called from a web browser.
 *  Pass a result or error back by invoking Success or Failure on the object.
 *  UFunctions accepting a FWebJSResponse should have a void return type, as any value returned from the function will be ignored.
 *  Calling the response methods does not have to happen before returning from the function, which means you can use this to implement asynchronous functionality.
 *
 *  Note that the remote object will become invalid as soon as a result has been delivered, so you can only call either Success or Failure once.
 */
USTRUCT()
struct FWebJSResponse
	: public FWebJSCallbackBase
{
	GENERATED_USTRUCT_BODY()

	FWebJSResponse()
		: FWebJSCallbackBase()
	{}

	FWebJSResponse(TSharedPtr<FWebJSScripting> InScripting, const FGuid& InCallbackId)
		: FWebJSCallbackBase(InScripting, InCallbackId)
	{}

	/**
	 * Indicate successful completion without a return value.
	 * The remote Promise's then() handler will be executed without arguments.
	 */
	void Success() const
	{
		Invoke(0, nullptr, false);
	}

	/**
	 * Indicate successful completion passing a return value back.
	 * The remote Promise's then() handler will be executed with the value passed as its single argument.
	 */
	template<typename T>
	void Success(T Arg) const
	{
		FWebJSParam ArgArray[1] = {FWebJSParam(Arg)};
		Invoke(1, ArgArray, false);
	}

	/**
	 * Indicate failed completion, passing an error message back to JS.
	 * The remote Promise's catch() handler will be executed with the value passed as the error reason.
	 */
	template<typename T>
	void Failure(T Arg) const
	{
		FWebJSParam ArgArray[1] = {FWebJSParam(Arg)};
		Invoke(1, ArgArray, true);
	}


};
