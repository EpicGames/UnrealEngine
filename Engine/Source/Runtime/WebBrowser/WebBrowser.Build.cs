// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class WebBrowser : ModuleRules
{
	public WebBrowser(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"ApplicationCore",
				"RHI",
				"InputCore",
				"Serialization",
				"HTTP"
			}
		);

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
				"Slate",
				"SlateCore"
            }
        );

        if (Target.Platform == UnrealTargetPlatform.Android ||
		    Target.IsInPlatformGroup(UnrealPlatformGroup.IOS) || 
		    Target.Platform == UnrealTargetPlatform.Mac)
		{
			// We need these on mobile for external texture support
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"JsonUtilities",
					"WebBrowserTexture",
					"Json"
				}
			);

			if(Target.Platform != UnrealTargetPlatform.Mac) 
			{
				PrivateDependencyModuleNames.Add("Launch");
			} 
			else 
			{
				PrivateDependencyModuleNames.Add("RenderCore");

				PublicFrameworks.Add("WebKit");
			}

			// We need this one on Android for URL decoding
			PrivateDependencyModuleNames.Add("HTTP");

			CircularlyReferencedDependentModules.Add("WebBrowserTexture");
		}
		
		if (Target.bCompileAgainstEngine)
		{
			if (Target.Platform == UnrealTargetPlatform.Win64)
			{
				PrivateDependencyModuleNames.Add("RenderCore");
			}
		
			PrivateDependencyModuleNames.Add("Engine");
		}

		if (Target.Platform == UnrealTargetPlatform.Android)
		{
			PublicSystemLibraries.Add("libjnigraphics");

			string ModulePath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);
			AdditionalPropertiesForReceipt.Add("AndroidPlugin", Path.Combine(ModulePath, "WebBrowser_UPL.xml"));
		}

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			PrivateIncludePathModuleNames.AddRange([
				"D3D11RHI",
				"D3D12RHI"
			]);

			AddEngineThirdPartyPrivateStaticDependencies(Target, "DX11");
		}

		if (Target.Platform == UnrealTargetPlatform.Win64
		||  Target.Platform == UnrealTargetPlatform.Mac
		||  Target.Platform == UnrealTargetPlatform.Linux)
		{
			PrivateDependencyModuleNames.Add("CEF3Utils");
			AddEngineThirdPartyPrivateStaticDependencies(Target, "CEF3");

			if (Target.Type != TargetType.Server)
			{
				if (Target.Platform == UnrealTargetPlatform.Mac || Target.Platform == UnrealTargetPlatform.Linux)
				{
					RuntimeDependencies.Add("$(EngineDir)/Binaries/" + Target.Platform.ToString() + "/EpicWebHelper");
				}
				else
				{
					if (Target.Architecture == UnrealArch.Arm64)
					{
						RuntimeDependencies.Add("$(EngineDir)/Binaries/" + Target.Platform.ToString() + "/EpicWebHelperarm64.exe");
					}
					else
					{
						RuntimeDependencies.Add("$(EngineDir)/Binaries/" + Target.Platform.ToString() + "/EpicWebHelper.exe");
					}
				}
			}
		}
		PrivateDefinitions.Add("PLATFORM_SPECIFIC_WEB_BROWSER=" + (bPlatformSpecificWebBrowser ? "1" : "0"));
	}
	protected virtual bool bPlatformSpecificWebBrowser { get { return false; } }
}
