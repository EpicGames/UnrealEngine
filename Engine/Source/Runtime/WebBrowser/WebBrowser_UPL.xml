<?xml version="1.0" encoding="utf-8"?>
<!--WebBrowser optional additions-->
<root xmlns:android="http://schemas.android.com/apk/res/android">
	<!-- init section is always evaluated once per architecture -->
	<init>
		<log text="WebBrowser init"/>

		<setBoolFromProperty result="bSupportWebViewPaymentRequest" ini="Engine" section="/Script/AndroidRuntimeSettings.AndroidRuntimeSettings" property="bSupportWebViewPaymentRequest" default="false"/>

		<if condition="bSupportWebViewPaymentRequest">
			<true>
				<log text="Including WebView PaymentRequest API support."/>
			</true>
		</if>
	</init>

	<androidManifestUpdates>
		<if condition="bSupportWebViewPaymentRequest">
			<true>
				<addElements tag="manifest">
					<queries>
						<intent>
							<action android:name="org.chromium.intent.action.PAY" />
						</intent>
						<intent>
							<action android:name="org.chromium.intent.action.IS_READY_TO_PAY" />
						</intent>
						<intent>
							<action android:name="org.chromium.intent.action.UPDATE_PAYMENT_DETAILS" />
						</intent>
					</queries>
				</addElements>
			</true>
		</if>
	</androidManifestUpdates>
</root>
